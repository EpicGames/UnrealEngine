// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Unix/UnixSystemIncludes.h" // IWYU pragma: export
#include "Templates/Atomic.h"

#include <ucontext.h> // IWYU pragma: export

/** Passed in through sigqueue for gathering of a callstack from a signal */
struct ThreadStackUserData
{
	// If we want a backtrace or a callstack
	// Backtrace is just a list of program counters and callstack is a symbolicated backtrace
	bool bCaptureCallStack;
	union 
	{
		ANSICHAR* CallStack;
		uint64* BackTrace;
	};

	int32 BackTraceCount;
	SIZE_T CallStackSize;
	TAtomic<bool> bDone;
};

struct FUnixCrashContext : public FGenericCrashContext
{
	/** Signal number */
	int32 Signal;
	
	/** Additional signal info */
	siginfo_t* Info;
	
	/** Thread context */
	ucontext_t*	Context;

	/** Whether backtrace was already captured */
	bool bCapturedBacktrace;

	/** Symbols received via backtrace_symbols(), if any (note that we will need to clean it up) */
	char ** BacktraceSymbols;

	/** Memory reserved for "exception" (signal) info */
	TCHAR SignalDescription[256];

	/** Memory reserved for minidump-style callstack info */
	char MinidumpCallstackInfo[16384];

	/** Fake siginfo used when handling ensure(), etc */
	static CORE_API __thread siginfo_t	FakeSiginfoForDiagnostics;

	/** The PC of the first function used when handling a crash. Used to figure out the number of frames to ignore */
	uint64* FirstCrashHandlerFrame = nullptr;

	/** The PC of where the error being reported occurred. Note that this could be different
	 * from FirstCrashHandlerFrame */
	void* ErrorFrame = nullptr;

	FUnixCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage)
		:	FGenericCrashContext(InType, InErrorMessage)
		,	Signal(0)
		,	Info(nullptr)
		,	Context(nullptr)
		,	bCapturedBacktrace(false)
		,	BacktraceSymbols(nullptr)
	{
		SignalDescription[ 0 ] = 0;
		MinidumpCallstackInfo[ 0 ] = 0;
	}

	CORE_API ~FUnixCrashContext();

	/**
	 * Inits the crash context from data provided by a signal handler.
	 *
	 * @param InSignal number (SIGSEGV, etc)
	 * @param InInfo additional info (e.g. address we tried to read, etc)
	 * @param InContext thread context
	 */
	CORE_API void InitFromSignal(int32 InSignal, siginfo_t* InInfo, void* InContext);

	/**
	 * Inits the crash context from some diagnostic handler (ensure, stall, etc...)
	 *
	 * @param InAddress address where the event happened, for optional callstack trimming
	 */
	CORE_API void InitFromDiagnostics(const void* InAddress = nullptr);

	/**
	 * Populates crash context stack trace and a few related fields for the calling thread
	 *
	 */
	CORE_API void CaptureStackTrace(void* ErrorProgramCounter);

	/**
	 * Populates crash context stack trace and a few related fields for another thread
	 *
	 * @param ThreadId The thread to capture the stack trace from
	 */
	CORE_API void CaptureThreadStackTrace(uint32_t ThreadId);

	/**
	 * Generates a new crash report containing information needed for the crash reporter and launches it; may not return.
	 *
	 * @return If the crash type is not continuable, the function will not return
	 */
	CORE_API void GenerateCrashInfoAndLaunchReporter() const;

	/**
	 * Sets whether this crash represents a non-crash event like an ensure
	 */
	void SetType(ECrashContextType InType) { Type = InType; }

	/**
	 * Sets the FirstCrashHandlerFrame only if it has not been set before
	 */
	CORE_API void SetFirstCrashHandlerFrame(uint64* ProgramCounter);

	CORE_API virtual void GetPortableCallStack(const uint64* StackFrames, int32 NumStackFrames, TArray<FCrashStackFrame>& OutCallStack) const override;

	CORE_API void AddPlatformSpecificProperties() const;

	/**
	 * Captures callstacks from all threads for inclusion in crash reports.
	 * Must be called after CaptureStackTrace() so the crashing thread's stack is available.
	 */
	CORE_API void CaptureAllThreadContexts();

protected:
	CORE_API virtual bool GetPlatformAllThreadContextsString(FString& OutStr) const override;

	/**
	 * Dumps all the data from crash context to the "minidump" report.
	 *
	 * @param DiagnosticsPath Path to put the file to
	 */
	CORE_API void GenerateReport(const FString & DiagnosticsPath) const;

private:
	/** Accumulated XML string for all thread contexts, populated by CaptureAllThreadContexts() */
	mutable FString AllThreadContextsString;
};

typedef FUnixCrashContext FPlatformCrashContext;

namespace UnixCrashReporterTracker
{
	/**
	 * Initialize persistent data and tickers
	 */
	void PreInit();

	/**
	 * Only call this function from a forked child process. The child process cannot be responsible for a sibling process.
	 *
	 * This removes a valid Crash Reporter tracker from the calling process.
	 */
	void RemoveValidCrashReportTickerForChildProcess();
}
