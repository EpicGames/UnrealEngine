// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	UnixPlatformFile.h: Unix platform File functions
==============================================================================================*/

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformFile.h" // IWYU pragma: export
#include "Misc/DateTime.h"

class IMappedFileHandle;
template <typename FuncType> class TFunctionRef;

/**
 * Unix File I/O implementation
**/
class FUnixPlatformFile : public IPhysicalPlatformFile
{
protected:
	CORE_API virtual FString NormalizeFilename(const TCHAR* Filename, bool bIsForWriting);
	CORE_API virtual FString NormalizeDirectory(const TCHAR* Directory, bool bIsForWriting);
public:
	//~ For visibility of overloads we don't override
	using IPhysicalPlatformFile::IterateDirectory;
	using IPhysicalPlatformFile::IterateDirectoryStat;

	AUTORTFM_OPEN CORE_API virtual bool FileExists(const TCHAR* Filename) override;
	CORE_API virtual int64 FileSize(const TCHAR* Filename) override;
	CORE_API virtual bool DeleteFile(const TCHAR* Filename) override;
	CORE_API virtual bool IsReadOnly(const TCHAR* Filename) override;
	CORE_API virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override;
	CORE_API virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;


	CORE_API virtual FDateTime GetTimeStamp(const TCHAR* Filename) override;

	CORE_API virtual void SetTimeStamp(const TCHAR* Filename, const FDateTime DateTime) override;

	CORE_API virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override;
	CORE_API virtual FString GetFilenameOnDisk(const TCHAR* Filename) override;

	CORE_API virtual ESymlinkResult IsSymlink(const TCHAR* Filename) override;

	CORE_API virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	CORE_API virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override;
	CORE_API virtual FOpenMappedResult OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0) override;
	CORE_API virtual FOpenMappedResult OpenMappedEx2(const TCHAR* Filename, EOpenMappedFlags OpenOptions = EOpenMappedFlags::None, int64 MaximumSize = 0) override;

	CORE_API virtual bool DirectoryExists(const TCHAR* Directory) override;
	CORE_API virtual bool CreateDirectory(const TCHAR* Directory) override;
	CORE_API virtual bool DeleteDirectory(const TCHAR* Directory) override;

	CORE_API virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override;

	CORE_API bool CreateDirectoriesFromPath(const TCHAR* Path);

	CORE_API virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) override;
	CORE_API virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) override;

	CORE_API virtual bool CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None) override;

protected:
	CORE_API bool IterateDirectoryCommon(const TCHAR* Directory, const TFunctionRef<bool(struct dirent*)>& Visitor);

	/** We're logging an error message. */
	bool bLoggingError = false;
};
