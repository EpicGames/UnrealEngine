// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"

/**
 * Functions for operating on GUIDs (UUIDs).
 */
struct FUnixPlatformGuid
{
	/**
	 * Create a GUID (UUID). The UUID will be generated by one of the standard algorithms in RFC 9562.
	 * Historically this function used the random-data-only algorithm (UUID v4) until Unreal Engine 5.8, where it switched to time-and-random-data
	 * (UUID v7).
	 *
	 * @return New GUID, or Invalid() (Nil UUID—all zeroes) on error.
	 */
	AUTORTFM_OPEN static CORE_API FGuid CreateGuid();

	/**
	 * Create a GUID (UUID) using the time-and-random-data method (UUID version 7).
	 *
	 * @return New GUID, or Invalid() (Nil UUID—all zeroes) on error.
	 */
	AUTORTFM_OPEN static CORE_API FGuid CreateGuidFromTimeAndRandom();

	/**
	 * Create a GUID (UUID) using the random data only method (UUID version 4).
	 *
	 * @return New GUID, or Invalid() (Nil UUID—all zeroes) on error.
	 */
	AUTORTFM_OPEN static CORE_API FGuid CreateGuidFromRandomOnly();
};
