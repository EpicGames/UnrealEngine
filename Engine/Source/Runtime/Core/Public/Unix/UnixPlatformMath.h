// Copyright Epic Games, Inc. All Rights Reserved.
// IWYU pragma: begin_exports

/*=============================================================================================
	LinuxPlatformMath.h: Linux platform Math functions
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Clang/ClangPlatformMath.h"
#include "Linux/LinuxSystemIncludes.h"
#include "Math/UnrealPlatformMathSSE4.h"

/**
 * Linux implementation of the Math OS functions
 */
struct FLinuxPlatformMath : public TUnrealPlatformMathSSE4Base<FClangPlatformMath>
{
#if PLATFORM_ENABLE_VECTORINTRINSICS
	static UE_FORCEINLINE_HINT bool IsNaN( float A ) { return isnan(A) != 0; }
	static UE_FORCEINLINE_HINT bool IsNaN( double A ) { return isnan(A) != 0; }
	static UE_FORCEINLINE_HINT bool IsFinite( float A ) { return isfinite(A); }
	static UE_FORCEINLINE_HINT bool IsFinite( double A ) { return isfinite(A); }
#endif
};

typedef FLinuxPlatformMath FPlatformMath;

// IWYU pragma: end_exports
