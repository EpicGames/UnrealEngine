// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	UnixPlatformMemory.h: Unix platform memory functions
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/PlatformCrt.h"

#include <malloc.h>

class FString;

#ifndef COMPILE_FORK_PAGE_PROTECTOR
	#define COMPILE_FORK_PAGE_PROTECTOR 0
#endif

/**
 *	Unix implementation of the FGenericPlatformMemoryStats.
 */
struct FPlatformMemoryStats : public FGenericPlatformMemoryStats
{};

/**
 *	Struct for more detailed stats that are slower to gather. Useful when using ForkAndWait().
 */
struct FExtendedPlatformMemoryStats
{
	/** */
	SIZE_T Shared_Clean;

	/** Shared memory used */
	SIZE_T Shared_Dirty;

	/** */
	SIZE_T Private_Clean;

	/** Private memory used */
	SIZE_T Private_Dirty;

	/** How much private memory is being saved by KSM, minus memory used by KSM to do so */
	SIZE_T KSM_Process_Profit;

	/** */
	bool bIs_KSM_Enabled;

	/** How many bytes have been copied from shared to private as a result of copy-on-write */
	SIZE_T Shared_Memory_COW_Bytes;
};

/**
* Unix implementation of the memory OS functions
**/
struct FUnixPlatformMemory : public FGenericPlatformMemory
{
	/**
	 * Unix representation of a shared memory region
	 */
	struct FUnixSharedMemoryRegion : public FSharedMemoryRegion
	{
		/** Returns file descriptor of a shared memory object */
		int GetFileDescriptor() const { return Fd; }

		/** Returns true if we need to unlink this region on destruction (no other process will be able to access it) */
		bool NeedsToUnlinkRegion() const { return bCreatedThisRegion; }

		FUnixSharedMemoryRegion(const FString& InName, uint32 InAccessMode, void* InAddress, SIZE_T InSize, int InFd, bool bInCreatedThisRegion)
			:	FSharedMemoryRegion(InName, InAccessMode, InAddress, InSize)
			,	Fd(InFd)
			,	bCreatedThisRegion(bInCreatedThisRegion)
		{}

	protected:

		/** File descriptor of a shared region */
		int				Fd;

		/** Whether we created this region */
		bool			bCreatedThisRegion;
	};

	//~ Begin FGenericPlatformMemory Interface
	static CORE_API void Init();
	static CORE_API class FMalloc* BaseAllocator();
	static CORE_API FPlatformMemoryStats GetStats();
	static CORE_API FExtendedPlatformMemoryStats GetExtendedStats();
	static CORE_API const FPlatformMemoryConstants& GetConstants();
	static CORE_API bool PageProtect(void* const Ptr, const SIZE_T Size, const bool bCanRead, const bool bCanWrite);
	static CORE_API void* BinnedAllocFromOS(SIZE_T Size);
	static CORE_API void BinnedFreeToOS(void* Ptr, SIZE_T Size);

	static CORE_API bool GetForkedPageAllocationInfo(TArray<FForkedPageAllocation>& OutPageAllocationInfos);

	class FPlatformVirtualMemoryBlock : public FBasicVirtualMemoryBlock
	{
	public:

		FPlatformVirtualMemoryBlock()
		{
		}

		FPlatformVirtualMemoryBlock(void *InPtr, uint32 InVMSizeDivVirtualSizeAlignment)
			: FBasicVirtualMemoryBlock(InPtr, InVMSizeDivVirtualSizeAlignment)
		{
		}
		FPlatformVirtualMemoryBlock(const FPlatformVirtualMemoryBlock& Other) = default;
		FPlatformVirtualMemoryBlock& operator=(const FPlatformVirtualMemoryBlock& Other) = default;

		CORE_API void Commit(size_t InOffset, size_t InSize);
		CORE_API void Decommit(size_t InOffset, size_t InSize);
		CORE_API void FreeVirtual();

		UE_FORCEINLINE_HINT void CommitByPtr(void *InPtr, size_t InSize)
		{
			Commit(size_t(((uint8*)InPtr) - ((uint8*)Ptr)), InSize);
		}

		UE_FORCEINLINE_HINT void DecommitByPtr(void *InPtr, size_t InSize)
		{
			Decommit(size_t(((uint8*)InPtr) - ((uint8*)Ptr)), InSize);
		}

		UE_FORCEINLINE_HINT void Commit()
		{
			Commit(0, GetActualSize());
		}

		UE_FORCEINLINE_HINT void Decommit()
		{
			Decommit(0, GetActualSize());
		}

		UE_FORCEINLINE_HINT size_t GetActualSize() const
		{
			return VMSizeDivVirtualSizeAlignment * GetVirtualSizeAlignment();
		}

		static CORE_API FPlatformVirtualMemoryBlock AllocateVirtual(size_t Size, size_t InAlignment = FPlatformVirtualMemoryBlock::GetVirtualSizeAlignment());
		static CORE_API size_t GetCommitAlignment();
		static CORE_API size_t GetVirtualSizeAlignment();
	};

	static CORE_API FSharedMemoryRegion * MapNamedSharedMemoryRegion(const FString& InName, bool bCreate, uint32 AccessMode, SIZE_T Size);
	static CORE_API bool UnmapNamedSharedMemoryRegion(FSharedMemoryRegion * MemoryRegion);
	static CORE_API bool GetLLMAllocFunctions(void*(*&OutAllocFunction)(size_t), void(*&OutFreeFunction)(void*, size_t), int32& OutAlignment);
	[[noreturn]] static CORE_API void OnOutOfMemory(uint64 Size, uint32 Alignment);
	//~ End FGenericPlatformMemory Interface

	static CORE_API bool HasForkPageProtectorEnabled();

	static CORE_API bool CanOverallocateVirtualMemory();

	static CORE_API void IncSharedMemoryCOWBytes(size_t Bytes);
};

typedef FUnixPlatformMemory FPlatformMemory;



