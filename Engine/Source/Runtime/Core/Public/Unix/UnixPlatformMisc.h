// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	UnixPlatformMisc.h: Unix platform misc functions
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Misc/Build.h"

#define UE_DEBUG_BREAK_IMPL() FUnixPlatformMisc::UngrabAllInput(); PLATFORM_BREAK()

class Error;
struct FGenericCrashContext;

/**
 * Unix implementation of the misc OS functions
 */
struct FUnixPlatformMisc : public FGenericPlatformMisc
{
	static CORE_API void PlatformPreInit();
	static CORE_API void PlatformInit();
	static CORE_API void PlatformTearDown();
	static CORE_API void SetGracefulTerminationHandler();
	static CORE_API void SetCrashHandler(void (* CrashHandler)(const FGenericCrashContext& Context));
	static CORE_API int32 GetMaxPathLength();

	static CORE_API FString GetEnvironmentVariable(const TCHAR* VariableName);
	static CORE_API void SetEnvironmentVar(const TCHAR* VariableName, const TCHAR* Value);
	static CORE_API TArray<uint8> GetMacAddress();
	static CORE_API bool IsRunningOnBattery();

#if !UE_BUILD_SHIPPING
	static CORE_API bool IsDebuggerPresent();
#endif // !UE_BUILD_SHIPPING

	static CORE_API void LowLevelOutputDebugString(const TCHAR *Message);

	static CORE_API void RequestExit(bool Force, const TCHAR* CallSite = nullptr);
	static CORE_API void RequestExitWithStatus(bool Force, uint8 ReturnCode, const TCHAR* CallSite = nullptr);
	static CORE_API const TCHAR* GetSystemErrorMessage(TCHAR* OutBuffer, int32 BufferCount, int32 Error);
	static CORE_API uint32 GetLastError();

	static CORE_API void NormalizePath(FString& InPath);
	static CORE_API void NormalizePath(FStringBuilderBase& InPath);

	static const TCHAR* GetPathVarDelimiter()
	{
		return TEXT(":");
	}

	static CORE_API EAppReturnType::Type MessageBoxExt(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption);

	UE_FORCEINLINE_HINT static void MemoryBarrier()
	{
		__sync_synchronize();
	}

	static CORE_API int32 NumberOfCores();
	static CORE_API int32 NumberOfCoresIncludingHyperthreads();
	static CORE_API FString GetOperatingSystemId();
	static CORE_API bool GetDiskTotalAndFreeSpace(const FString& InPath, uint64& TotalNumberOfBytes, uint64& NumberOfFreeBytes);
	static CORE_API bool GetPageFaultStats(FPageFaultStats& OutStats, EPageFaultFlags Flags=EPageFaultFlags::All);
	static CORE_API bool GetBlockingIOStats(FProcessIOStats& OutStats, EInputOutputFlags Flags=EInputOutputFlags::All);
	static CORE_API bool GetContextSwitchStats(FContextSwitchStats& OutStats, EContextSwitchFlags Flags=EContextSwitchFlags::All);

	/**
	 * Determines the shader format for the platform
	 *
	 * @return	Returns the shader format to be used by that platform
	 */
	static CORE_API const TCHAR* GetNullRHIShaderFormat();

	static CORE_API bool HasCPUIDInstruction();
	static CORE_API FString GetCPUVendor();
	static CORE_API FString GetCPUBrand();
	static CORE_API uint32 GetCPUModelID();
	static CORE_API uint32 GetCPUMicrocodeRevision();

	/**
	 * Uses cpuid instruction to get the vendor string
	 *
	 * @return	CPU info bitfield
	 *
	 *			Bits 0-3	Stepping ID
	 *			Bits 4-7	Model
	 *			Bits 8-11	Family
	 *			Bits 12-13	Processor type (Intel) / Reserved (AMD)
	 *			Bits 14-15	Reserved
	 *			Bits 16-19	Extended model
	 *			Bits 20-27	Extended family
	 *			Bits 28-31	Reserved
	 */
	static CORE_API uint32 GetCPUInfo();

	static CORE_API FString GetPrimaryGPUBrand();

	/**
	 * Ungrabs input (useful before breaking into debugging)
	 */
	static CORE_API void UngrabAllInput();

	/**
	 * Returns whether the program has been started remotely (e.g. over SSH)
	 */
	static CORE_API bool HasBeenStartedRemotely();

#if ENABLE_PGO_PROFILE
	static bool StartNewPGOCollection(const FString& AbsoluteFileName);
	static bool IsPGIActive();
	static bool StopPGOCollectionAndCloseFile();
#endif

	/**
	 * Determines if return code has been overriden and returns it.
	 *
	 * @param OverriddenReturnCodeToUsePtr pointer to an variable that will hold an overriden return code, if any. Can be null.
	 *
	 * @return true if the error code has been overriden, false if not
	 */
	static CORE_API bool HasOverriddenReturnCode(uint8 * OverriddenReturnCodeToUsePtr);
	static CORE_API void GetOSVersions(FString& out_OSVersionLabel, FString& out_OSSubVersionLabel);
	static CORE_API FString GetOSVersion();
	static CORE_API FString GetLoginId();

	static CORE_API void CreateGuid(FGuid& Result);

	static CORE_API IPlatformChunkInstall* GetPlatformChunkInstall();

	static CORE_API bool SetStoredValues(const FString& InStoreId, const FString& InSectionName, const TMap<FString, FString>& InKeyValues);

	static CORE_API int32 NumberOfWorkerThreadsToSpawn();

	/* Explicitly call this function to setup syscall filters based on a file passed in from the command line
	 * -allowsyscallfilterfile=PATH_TO_FILE
	 */
	static CORE_API bool SetupSyscallFilters();
};
