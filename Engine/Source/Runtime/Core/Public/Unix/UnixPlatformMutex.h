// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PThreadsRecursiveMutex.h"
#include "HAL/PThreadsSharedMutex.h"
#include "Misc/Timespan.h"

class FString;

namespace UE
{

/** A system-wide mutex for Unix. Uses exclusive file locking. */
class FUnixSystemWideMutex final
{
public:
	FUnixSystemWideMutex(const FUnixSystemWideMutex&) = delete;
	FUnixSystemWideMutex& operator=(const FUnixSystemWideMutex&) = delete;

	/** Construct a named, system-wide mutex and attempt to get access/ownership of it. */
	CORE_API explicit FUnixSystemWideMutex(const FString& InName, FTimespan InTimeout = FTimespan::Zero());

	/** Destructor releases system-wide mutex if it is currently owned. */
	CORE_API ~FUnixSystemWideMutex();

	/**
	 * Does the calling thread have ownership of the system-wide mutex?
	 *
	 * @return True if obtained. WARNING: Returns true for an owned but previously abandoned locks so shared resources can be in undetermined states. You must handle shared data robustly.
	 */
	CORE_API bool IsValid() const;

	/** Releases system-wide mutex if it is currently owned. */
	CORE_API void Release();

private:
	int32 FileHandle;
};

using FPlatformRecursiveMutex = FPThreadsRecursiveMutex;
using FPlatformSharedMutex = FPThreadsSharedMutex;
using FPlatformSystemWideMutex = FUnixSystemWideMutex;

} // UE
