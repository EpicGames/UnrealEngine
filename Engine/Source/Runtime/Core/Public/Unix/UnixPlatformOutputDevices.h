// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	UnixPlatformOutputDevices.h: Unix platform OutputDevices functions
==============================================================================================*/

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformOutputDevices.h"

class FOutputDevice;
class FOutputDeviceError;

struct FUnixOutputDevices : public FGenericPlatformOutputDevices
{
	static CORE_API FString GetAbsoluteLogFilename();
	static CORE_API FOutputDevice* GetEventLog();
	static CORE_API FOutputDeviceError* GetError();
};

typedef FUnixOutputDevices FPlatformOutputDevices;
