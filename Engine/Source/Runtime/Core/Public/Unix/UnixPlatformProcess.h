// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	UnixPlatformProcess.h: Unix platform Process functions
==============================================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "PosixOS/PosixOSPlatformProcess.h" // IWYU pragma: export
#include "HAL/PlatformMisc.h"
#include "Misc/AssertionMacros.h"
#include "Unix/UnixSystemIncludes.h" // IWYU pragma: export

class Error;

/** Wrapper around Unix pid_t. Should not be copyable as changes in the process state won't be properly propagated to all copies. */
struct FProcState
{
	/** Default constructor. */
	inline FProcState()
		:	ProcessId(0)
		,	bIsRunning(false)
		,	bHasBeenWaitedFor(false)
		,	ReturnCode(-1)
		,	bFireAndForget(false)
	{
	}

	/** Initialization constructor. */
	explicit FProcState(pid_t InProcessId, bool bInFireAndForget);

	/** Destructor. */
	~FProcState();

	/** Getter for process id */
	UE_FORCEINLINE_HINT pid_t GetProcessId() const
	{
		return ProcessId;
	}

	/**
	 * Returns whether this process is running.
	 *
	 * @return true if running
	 */
	bool	IsRunning();

	/**
	 * Returns child's return code (only valid to call if not running)
	 *
	 * @param ReturnCode set to return code if not NULL (use the value only if method returned true)
	 *
	 * @return return whether we have the return code (we don't if it crashed)
	 */
	bool	GetReturnCode(int32* ReturnCodePtr);

	/**
	 * Waits for the process to end.
	 * Has a side effect (stores child's return code).
	 */
	void	Wait();

protected:  // the below is not a public API!

	// FProcState should not be copyable since it breeds problems (e.g. one copy could have wait()ed, but another would not know it)

	/** Copy constructor - should not be publicly accessible */
	inline FProcState(const FProcState& Other)
		:	ProcessId(Other.ProcessId)
		,	bIsRunning(Other.bIsRunning)  // assume it is
		,	bHasBeenWaitedFor(Other.bHasBeenWaitedFor)
		,	ReturnCode(Other.ReturnCode)
		,	bFireAndForget(Other.bFireAndForget)
	{
		checkf(false, TEXT("FProcState should not be copied"));
	}

	/** Assignment operator - should not be publicly accessible */
	inline FProcState& operator=(const FProcState& Other)
	{
		checkf(false, TEXT("FProcState should not be copied"));
		return *this;
	}

	friend struct FUnixPlatformProcess;

	// -------------------------

	/** Process id */
	pid_t	ProcessId;

	/** Whether the process has finished or not (cached) */
	bool	bIsRunning;

	/** Whether the process's return code has been collected */
	bool	bHasBeenWaitedFor;

	/** Return code of the process (if negative, means that process did not finish gracefully but was killed/crashed*/
	int32	ReturnCode;

	/** Whether this child is fire-and-forget */
	bool	bFireAndForget;
};

/** FProcHandle can be copied (and thus passed by value). */
struct FProcHandle
{
	/** Child proc state set from CreateProc() call */
	FProcState* 		ProcInfo;

	/** Process start time captured during OpenProcess(), used with OpenedPid to
	  * uniquely identify the process and detect PID reuse. Value is clock ticks
	  * since boot from /proc/<pid>/stat field 22. Zero if unavailable.
	  */
	uint64				OpenedStartTime;

	/** Pid of external process opened with OpenProcess() call. */
	pid_t				OpenedPid;

	constexpr FProcHandle()
	:	ProcInfo(nullptr), OpenedStartTime(0), OpenedPid(-1)
	{
	}

	FProcHandle(FProcState* InHandle)
	:	ProcInfo(InHandle), OpenedStartTime(0), OpenedPid(-1)
	{
	}

	FProcHandle(pid_t InProcPid)
	:	ProcInfo(nullptr), OpenedStartTime(0), OpenedPid(InProcPid)
	{
	}

	/** Accessors. */
	UE_FORCEINLINE_HINT pid_t Get() const
	{
		return ProcInfo ? ProcInfo->GetProcessId() : OpenedPid;
	}

	/** Resets the handle to invalid */
	inline void Reset()
	{
		ProcInfo = nullptr;
		OpenedStartTime = 0;
		OpenedPid = -1;
	}

	/** Checks the validity of handle */
	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return ProcInfo != nullptr || OpenedPid != -1;
	}

	// the below is not part of FProcHandle API and is specific to Unix implementation
	UE_FORCEINLINE_HINT FProcState* GetProcessInfo() const
	{
		return ProcInfo;
	}
};

/** Wrapper around Unix file descriptors */
struct FPipeHandle
{
	FPipeHandle(int Fd, int PairFd)
		:	PipeDesc(Fd)
		,	PairDesc(PairFd)
	{
	}

	CORE_API ~FPipeHandle();

	/**
	 * Reads until EOF.
	 */
	FString Read();

	/**
	 * Reads until EOF.
	 */
	bool ReadToArray(TArray<uint8> & Output);

	/**
	 * Returns raw file handle.
	 */
	int GetHandle() const
	{
		return PipeDesc;
	}

	/**
	 * Returns the raw file handle of the other endpoint of PipeDesc
	 */
	int GetPairHandle() const
	{
		return PairDesc;
	}

protected:

	int	PipeDesc;
	int	PairDesc;
};

/**
 * Unix implementation of the Process OS functions
 */
struct FUnixPlatformProcess : public FPosixOSPlatformProcess
{
	struct FProcEnumInfo;

	/**
	 * Process enumerator.
	 */
	class FProcEnumerator
	{
	public:
		// Constructor
		CORE_API FProcEnumerator();
		FProcEnumerator(const FProcEnumerator&) = delete;
		FProcEnumerator& operator=(const FProcEnumerator&) = delete;

		// Destructor
		CORE_API ~FProcEnumerator();

		// Gets current process enumerator info.
		CORE_API FProcEnumInfo GetCurrent() const;
		
		/**
		 * Moves current to the next process.
		 *
		 * @returns True if succeeded. False otherwise.
		 */
		CORE_API bool MoveNext();
	private:
		// Private implementation data.
		struct FProcEnumData* Data;
	};

	/**
	 * Process enumeration info structure.
	 */
	struct FProcEnumInfo
	{
		friend FUnixPlatformProcess::FProcEnumerator::FProcEnumerator();

		// Gets process PID.
		CORE_API uint32 GetPID() const;

		// Gets parent process PID.
		CORE_API uint32 GetParentPID() const;

		// Gets process name. I.e. exec name.
		CORE_API FString GetName() const;

		// Gets process full image path. I.e. full path of the exec file.
		CORE_API FString GetFullPath() const;

	private:
		// Private constructor.
		FProcEnumInfo(uint32 InPID);

		// Current process PID.
		uint32 PID;
	};

	static CORE_API void* GetDllHandle( const TCHAR* Filename );
	static CORE_API void FreeDllHandle( void* DllHandle );
	static CORE_API void* GetDllExport( void* DllHandle, const TCHAR* ProcName );
	static CORE_API const TCHAR* ComputerName();
	static CORE_API const TCHAR* UserName(bool bOnlyAlphaNumeric = true);
	static CORE_API const TCHAR* UserTempDir();
	static CORE_API const TCHAR* UserDir();
	static CORE_API const TCHAR* UserSettingsDir();
	static CORE_API const TCHAR* ApplicationSettingsDir();
	static CORE_API FString GetApplicationSettingsDir(const ApplicationSettingsContext& Settings);
	static CORE_API void SetCurrentWorkingDirectoryToBaseDir();
	static CORE_API FString GetCurrentWorkingDirectory();
	static CORE_API FString GenerateApplicationPath(const FString& AppName, EBuildConfiguration BuildConfiguration);
	static CORE_API FString GetApplicationName( uint32 ProcessId );
	static CORE_API bool SetProcessLimits(EProcessResource::Type Resource, uint64 Limit);
	static CORE_API const TCHAR* ExecutablePath();
	static CORE_API const TCHAR* ExecutableName(bool bRemoveExtension = true);
	static CORE_API const TCHAR* GetModulePrefix();
	static CORE_API const TCHAR* GetModuleExtension();
	static CORE_API void ClosePipe( void* ReadPipe, void* WritePipe );
	static CORE_API bool CreatePipe(void*& ReadPipe, void*& WritePipe, bool bWritePipeLocal = false);
	static CORE_API FString ReadPipe( void* ReadPipe );
	static CORE_API bool ReadPipeToArray(void* ReadPipe, TArray<uint8> & Output);
	static CORE_API bool WritePipe(void* WritePipe, const FString& Message, FString* OutWritten = nullptr);
	static CORE_API bool WritePipe(void* WritePipe, const uint8* Data, const int32 DataLength, int32* OutDataLength = nullptr);
	static CORE_API class FRunnableThread* CreateRunnableThread();
	static CORE_API const FString GetModulesDirectory();
	static CORE_API bool CanLaunchURL(const TCHAR* URL);
	static CORE_API void LaunchURL(const TCHAR* URL, const TCHAR* Parms, FString* Error);
	static CORE_API FProcHandle CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild = nullptr);
	static CORE_API FProcHandle CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild);
	static CORE_API TTuple<FProcHandle, UE::HAL::EProcessId> CreateProc(const UE::HAL::FProcessStartInfo& StartInfo);
	static CORE_API FProcHandle OpenProcess(uint32 ProcessID);
	static CORE_API bool IsProcRunning( FProcHandle & ProcessHandle );
	static CORE_API void WaitForProc( FProcHandle & ProcessHandle );
	static CORE_API void CloseProc( FProcHandle & ProcessHandle );
	static CORE_API void TerminateProc( FProcHandle & ProcessHandle, bool KillTree = false );
	static CORE_API EWaitAndForkResult WaitAndFork();
	static CORE_API uint32 GetCurrentProcessId();
	static CORE_API uint32 GetCurrentCoreNumber();
	static CORE_API bool GetProcReturnCode( FProcHandle & ProcHandle, int32* ReturnCode );
	static CORE_API bool Daemonize();
	static CORE_API bool IsApplicationRunning( uint32 ProcessId );
	static CORE_API bool IsApplicationRunning( const TCHAR* ProcName );
	static CORE_API bool ExecProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr, const TCHAR* OptionalWorkingDirectory = NULL, bool bShouldEndWithParentProcess = false);
	static CORE_API void ExploreFolder( const TCHAR* FilePath );
	static CORE_API bool LaunchFileInDefaultExternalApplication( const TCHAR* FileName, const TCHAR* Parms = NULL, ELaunchVerb::Type Verb = ELaunchVerb::Open, bool bPromptToOpenOnFailure = true);
	static CORE_API void OnChildEndFramePostFork();
	static CORE_API int32 TranslateThreadPriority(EThreadPriority Priority);
	static CORE_API void SetThreadNiceValue( uint32_t ThreadId, int32 NiceValue );
	static CORE_API void SetThreadPriority( EThreadPriority NewPriority );
	static CORE_API bool GetPerFrameProcessorUsage(uint32 ProcessId, float& ProcessUsageFraction, float& IdleUsageFraction);


	/**
	 * @brief Returns user home directory (i.e. $HOME).
	 *
	 * Like other directory functions, cannot return nullptr!
	 */
	static CORE_API const TCHAR* UserHomeDir();

private:
	static FProcHandle CreateProcInternal(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild, bool bInheritHandles);
};
