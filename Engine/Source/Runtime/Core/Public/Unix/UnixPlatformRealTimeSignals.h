// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Central location to define all real time signal in use

// Used in UnixPlatformProcess.cpp for forking
#define WAIT_AND_FORK_QUEUE_SIGNAL SIGRTMIN + 1
#define WAIT_AND_FORK_RESPONSE_SIGNAL SIGRTMIN + 2

// Used in UnixSignalGameHitchHeartBeat.cpp for a hitch signal handler
#define HEART_BEAT_SIGNAL SIGRTMIN + 3

// Skip SIGRTMIN + 4 default signal used in VTune

// Using in UnixPlatformCrashContext.cpp/UnixPlatformStackWalk.cpp for gathering current callstack from a thread
#define THREAD_CALLSTACK_GENERATOR SIGRTMIN + 5
