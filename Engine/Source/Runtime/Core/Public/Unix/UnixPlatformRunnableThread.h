// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	UnixRunnableThread.h: Unix platform threading functions
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "HAL/PThreadRunnableThread.h"
#include <signal.h> // for SIGSTKSZ

class Error;

/**
* Unix implementation of the Process OS functions
**/
class FRunnableThreadUnix : public FRunnableThreadPThread
{
	struct EConstants
	{
		enum
		{
			UnixThreadNameLimit = 15,			// the limit for thread name is just 15 chars :( http://man7.org/linux/man-pages/man3/pthread_setname_np.3.html

			CrashHandlerStackSize = SIGSTKSZ + 192 * 1024,	// should be at least SIGSTKSIZE, plus 192K because we do logging and symbolication in crash handler
			CrashHandlerStackSizeMin = SIGSTKSZ + 8 * 1024  // minimum allowed stack size
		};
	};

	/** Each thread needs a separate stack for the signal handler, so possible stack overflows in the thread are handled */
	void* ThreadCrashHandlingStack;

	/** Address of stack guard page - if nullptr, the page wasn't set */
	void* StackGuardPageAddress;

	/** Baseline priority (nice value). See explanation in SetPriority(). */
	int BaselineNiceValue;

	/** Whether the value of BaselineNiceValue has been obtained through getpriority(). See explanation in SetPriority(). */
	bool bGotBaselineNiceValue;

public:

	/** Separate stack for the signal handler (so possible stack overflows don't go unnoticed), for the main thread specifically. */
	static void *MainThreadSignalHandlerStack;

	static void *AllocCrashHandlerStack();
	static void FreeCrashHandlerStack(void *StackBuffer);
	static uint64 GetCrashHandlerStackSize();

	FRunnableThreadUnix()
		:	FRunnableThreadPThread()
		,	ThreadCrashHandlingStack(nullptr)
		,	StackGuardPageAddress(nullptr)
		,	BaselineNiceValue(0)
		,	bGotBaselineNiceValue(false)
	{
	}

	~FRunnableThreadUnix()
	{
		// Call the parent destructor body before the parent does it - see comment on that function for explanation why.
		FRunnableThreadPThread_DestructorBody();
	}

	/**
	 * Sets up an alt stack for signal (including crash) handling on this thread.
	 *
	 * This includes guard page at the end of the stack to make running out of stack more obvious.
	 * Should be run in the context of the thread.
	 *
	 * @param StackBuffer pointer to the beginning of the stack buffer (note: on x86_64 will be the bottom of the stack, not its beginning)
	 * @param StackSize size of the stack buffer
	 * @param OutStackGuardPageAddress pointer to the variable that will receive the address of the guard page. Can be null. Will not be set if guard page wasn't successfully set.
	 *
	 * @return true if setting the alt stack succeeded. Inability to set guard page will not affect success of the operation.
	 */
	static bool SetupSignalHandlerStack(void* StackBuffer, const size_t StackBufferSize, void** OutStackGuardPageAddress);

protected:

	/** on Unix, this translates to ranges of setpriority(). Note that not all range may be available*/
	int32 TranslateThreadPriority(EThreadPriority Priority) override;

	void SetThreadPriority(EThreadPriority NewPriority) override;

	void SetThreadPriority(pthread_t InThread, EThreadPriority NewPriority) override;

private:

	/**
	 * Allows a platform subclass to setup anything needed on the thread before running the Run function
	 */
	void PreRun() override;

	void PostRun() override;

	/**
	 * Allows platforms to adjust stack size
	 */
	uint32 AdjustStackSize(uint32 InStackSize) override;
};
