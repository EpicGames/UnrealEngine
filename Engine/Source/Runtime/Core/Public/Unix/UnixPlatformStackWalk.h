// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	UnixStackWalk.h: Unix platform stack walk functions
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"

struct FGenericCrashContext;

struct FUnixPlatformStackWalk : public FGenericPlatformStackWalk
{
	static CORE_API void ProgramCounterToSymbolInfo( uint64 ProgramCounter, FProgramCounterSymbolInfo& out_SymbolInfo );
	static CORE_API bool ProgramCounterToHumanReadableString( int32 CurrentCallDepth, uint64 ProgramCounter, ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, FGenericCrashContext* Context = nullptr );
	static CORE_API uint32 CaptureStackBackTrace( uint64* BackTrace, uint32 MaxDepth, void* Context = nullptr );
	// Fast stack backtrace, only for tracing.
	// Will NOT work from signal handlers, is NOT suitable for crashreporting. Will not see inline functions.
	static CORE_API uint32 CaptureStackBackTraceViaFramePointerWalking(uint64* BackTrace, uint32 MaxDepth);
	static CORE_API void StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, void* Context = nullptr );
	static CORE_API void StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, void* ProgramCounter, void* Context = nullptr );
	static CORE_API void StackWalkAndDumpEx(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 Flags, void* Context = nullptr);
	static CORE_API void StackWalkAndDumpEx(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, void* ProgramCounter, uint32 Flags, void* Context = nullptr);
	static CORE_API uint32 CaptureThreadStackBackTrace(uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context = nullptr);
	static CORE_API void ThreadStackWalkAndDump(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 ThreadId);
	static CORE_API int32 GetProcessModuleCount();
	static CORE_API int32 GetProcessModuleSignatures(FStackWalkModuleInfo *ModuleSignatures, const int32 ModuleSignaturesSize);
};

typedef FUnixPlatformStackWalk FPlatformStackWalk;
