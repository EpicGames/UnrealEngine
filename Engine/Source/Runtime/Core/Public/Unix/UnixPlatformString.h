// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericWidePlatformString.h"

/**
 * Unix string implementation
 */
struct FUnixPlatformString : public FGenericWidePlatformString
{
	template<typename CHAR>
	static inline int32 Strlen(const CHAR* String)
	{
		if(!String)
			return 0;

		int Len = 0;
		while(*String++)
		{
			++Len;
		}

		return Len;
	}
};

typedef FUnixPlatformString FPlatformString;

// Format specifiers to be able to print values of these types correctly, for example when using UE_LOG.
// SIZE_T format specifier
#define SIZE_T_FMT "zu"
// SIZE_T format specifier for lowercase hexadecimal output
#define SIZE_T_x_FMT "zx"
// SIZE_T format specifier for uppercase hexadecimal output
#define SIZE_T_X_FMT "zX"

// SSIZE_T format specifier
#define SSIZE_T_FMT "lld"
// SSIZE_T format specifier for lowercase hexadecimal output
#define SSIZE_T_x_FMT "llx"
// SSIZE_T format specifier for uppercase hexadecimal output
#define SSIZE_T_X_FMT "llX"

// UPTRINT format specifier for decimal output
#define UPTRINT_FMT "llu"
// UPTRINT format specifier for lowercase hexadecimal output
#define UPTRINT_x_FMT "llx"
// UPTRINT format specifier for uppercase hexadecimal output
#define UPTRINT_X_FMT "llX"

// PTRINT format specifier for decimal output
#define PTRINT_FMT SSIZE_T_FMT
// PTRINT format specifier for lowercase hexadecimal output
#define PTRINT_x_FMT SSIZE_T_x_FMT
// PTRINT format specifier for uppercase hexadecimal output
#define PTRINT_X_FMT SSIZE_T_X_FMT

// int64 format specifier for decimal output
#define INT64_FMT "lld"
// int64 format specifier for lowercase hexadecimal output
#define INT64_x_FMT "llx"
// int64 format specifier for uppercase hexadecimal output
#define INT64_X_FMT "llX"

// uint64 format specifier for decimal output
#define UINT64_FMT "llu"
// uint64 format specifier for lowercase hexadecimal output
#define UINT64_x_FMT "llx"
// uint64 format specifier for uppercase hexadecimal output
#define UINT64_X_FMT "llX"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "GenericPlatform/StandardPlatformString.h"
#endif
