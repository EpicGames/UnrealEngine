// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	UnixPlatformTLS.h: Unix platform TLS (Thread local storage and thread ID) functions
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Unix/UnixSystemIncludes.h" // IWYU pragma: export
#include "Misc/AssertionMacros.h"
#include "GenericPlatform/GenericPlatformTLS.h"

#if defined(_GNU_SOURCE)
	#include <sys/syscall.h>
#endif // _GNU_SOURCE

/**
 * Unix implementation of the TLS OS functions
 */
struct FUnixTLS : public FGenericPlatformTLS
{
	/**
	 * Returns the currently executing thread's id
	 */
	static inline uint32 GetCurrentThreadId(void)
	{
		// note: cannot use pthread_self() without updating the rest of API to opaque (or at least 64-bit) thread handles
#if defined(_GNU_SOURCE)

		// syscall() is relatively heavy and the whole editor grinds to a halt. 
		// Cache thread id in TLS, but allocate TLS slots dynamically for non-monolithic case since limit on dynamic libs with static per-thread variables (DTV_SURPLUS) is low
	#if !IS_MONOLITHIC
		uint32 ThreadIdTLS = static_cast<uint32>(reinterpret_cast<UPTRINT>(GetTlsValue(ThreadIdTLSKey)));
	#endif
		if (UNLIKELY(ThreadIdTLS == 0))
		{
			pid_t ThreadId = static_cast<pid_t>(syscall(SYS_gettid));
			static_assert(sizeof(pid_t) <= sizeof(uint32), "pid_t is larger than uint32, reconsider implementation of GetCurrentThreadId()");
			ThreadIdTLS = static_cast<uint32>(ThreadId);
			checkf(ThreadIdTLS != 0, TEXT("ThreadId is 0 - reconsider implementation of GetCurrentThreadId() (syscall changed?)"));

	#if !IS_MONOLITHIC
			SetTlsValue(ThreadIdTLSKey, reinterpret_cast<void *>(static_cast<UPTRINT>(ThreadIdTLS)));
	#endif
		}
		return ThreadIdTLS;

#else
		// better than nothing...
		static_assert(sizeof(uint32) == sizeof(pthread_t), "pthread_t cannot be converted to uint32 one to one - different number of bits. Review FUnixTLS::GetCurrentThreadId() implementation.");
		return static_cast< uint32 >(pthread_self());
#endif
	}

	static void ClearThreadIdTLS(void)
	{
#if IS_MONOLITHIC
		ThreadIdTLS = 0;
#else
		SetTlsValue(ThreadIdTLSKey, nullptr);
#endif
	}

	/**
	 * Allocates a thread local store slot
	 */
	static uint32 AllocTlsSlot(void)
	{
		// allocate a per-thread mem slot
		pthread_key_t Key = 0;
		if (pthread_key_create(&Key, nullptr) != 0)
		{
			return static_cast<uint32>(INDEX_NONE); // matches the Windows TlsAlloc() retval.
		}

		// pthreads can return an arbitrary key, yet we reserve INDEX_NONE as an invalid one. Handle this very unlikely case
		// by allocating another one first (so we get another value) and releasing existing key.
		if (static_cast<uint32>(Key) == static_cast<uint32>(INDEX_NONE))
		{
			pthread_key_t NewKey = 0;
			int SecondKeyAllocResult = pthread_key_create(&NewKey, nullptr);
			// discard the previous one
			pthread_key_delete((pthread_key_t)Key);

			if (SecondKeyAllocResult != 0)
			{
				// could not alloc the second key, treat this as an error
				return static_cast<uint32>(INDEX_NONE); // matches the Windows TlsAlloc() retval.
			}

			// check that we indeed got something different
			checkf(NewKey != static_cast<uint32>(INDEX_NONE), TEXT("Could not allocate a usable TLS slot id."));

			Key = NewKey;
		}

		return Key;
	}

	/**
	 * Sets a value in the specified TLS slot
	 *
	 * @param SlotIndex the TLS index to store it in
	 * @param Value the value to store in the slot
	 */
	static UE_FORCEINLINE_HINT void SetTlsValue(uint32 SlotIndex,void* Value)
	{
		pthread_setspecific((pthread_key_t)SlotIndex, Value);
	}

	/**
	 * Reads the value stored at the specified TLS slot
	 *
	 * @return the value stored in the slot
	 */
	static UE_FORCEINLINE_HINT void* GetTlsValue(uint32 SlotIndex)
	{
		return pthread_getspecific((pthread_key_t)SlotIndex);
	}

	/**
	 * Frees a previously allocated TLS slot
	 *
	 * @param SlotIndex the TLS index to store it in
	 */
	static UE_FORCEINLINE_HINT void FreeTlsSlot(uint32 SlotIndex)
	{
		pthread_key_delete((pthread_key_t)SlotIndex);
	}

private:

#if IS_MONOLITHIC
	/** per-thread static variable - cannot have that in a non-monolithic build because it eats into a precious OS limit. */
	static CORE_API __thread uint32 ThreadIdTLS;
#else
	/** TLS key to store TID */
	static CORE_API uint32 ThreadIdTLSKey;
#endif
};

typedef FUnixTLS FPlatformTLS;
