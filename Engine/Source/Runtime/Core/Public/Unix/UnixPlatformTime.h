// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	UnixPlatformTime.h: Unix platform Time functions
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformTime.h"

#include <time.h> // IWYU pragma: export

/**
 * Unix implementation of the Time OS functions
 */
struct FUnixTime : public FGenericPlatformTime
{
	static CORE_API double InitTiming();

	static inline double Seconds()
	{
		struct timespec ts;
		clock_gettime(GetClockSource(), &ts);
		return static_cast<double>(ts.tv_sec) + static_cast<double>(ts.tv_nsec) / 1e9;
	}

	static inline uint32 Cycles()
	{
		struct timespec ts;
		clock_gettime(GetClockSource(), &ts);
		return static_cast<uint32>(static_cast<uint64>(ts.tv_sec) * (uint64)1e6 + static_cast<uint64>(ts.tv_nsec) / 1000ULL);
	}

	static inline uint64 Cycles64()
	{
		struct timespec ts;
		clock_gettime(GetClockSource(), &ts);
		return static_cast<uint64>(static_cast<uint64>(ts.tv_sec) * (uint64)1e7 + static_cast<uint64>(ts.tv_nsec) / 100ULL);
	}

	static CORE_API bool UpdateCPUTime(float DeltaSeconds);
	static CORE_API bool UpdateThreadCPUTime(float = 0.0);
	static CORE_API void AutoUpdateGameThreadCPUTime(double UpdateInterval);

	static CORE_API FCPUTime GetCPUTime();
	static CORE_API FCPUTime GetThreadCPUTime();
	static CORE_API double GetLastIntervalThreadCPUTimeInSeconds();

	/**
	 * Calibration log to be printed at later time
	 */
	static CORE_API void PrintCalibrationLog();

	/**
	 * Return the clock source, initiating clock selection (calibration/benchmarking) first if necessary.
	 */
	static inline clockid_t GetClockSource()
	{
		if (UNLIKELY(ClockSource < 0))
		{
			ClockSource = CalibrateAndSelectClock();
		}

		return ClockSource;
	}

private:

	/** Clock source to use */
	static CORE_API clockid_t ClockSource;

	/**
	 * Benchmarks clock_gettime() and returns the best available clock.
	 */
	static CORE_API clockid_t CalibrateAndSelectClock();
};

typedef FUnixTime FPlatformTime;
