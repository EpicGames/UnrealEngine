// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreGlobals.h"
#include "CoreTypes.h"

#include <time.h> // IWYU pragma: export

class FUnixSignalGameHitchHeartBeat
{
public:
	static CORE_API FUnixSignalGameHitchHeartBeat* Singleton;

	/** Gets the heartbeat singleton */
	static CORE_API FUnixSignalGameHitchHeartBeat& Get();
	static CORE_API FUnixSignalGameHitchHeartBeat* GetNoInit();

	/**
	* Called at the start of a frame to register the time we are looking to detect a hitch
	*/
	CORE_API void FrameStart(bool bSkipThisFrame = false);

	CORE_API double GetFrameStartTime();
	CORE_API double GetCurrentTime();

	/**
	* Suspend heartbeat hitch detection. Must call ResumeHeartBeat later to resume.
	*/
	CORE_API void SuspendHeartBeat();

	/**
	* Resume heartbeat hitch detection. Call only after first calling SuspendHeartBeat.
	*/
	CORE_API void ResumeHeartBeat();

	/**
	* Check if started suspended.
	*/
	CORE_API bool IsStartedSuspended();

	CORE_API void Restart();
	CORE_API void Stop();

	CORE_API void PostFork();

	CORE_API void InitSettings();

private:
	FUnixSignalGameHitchHeartBeat();
	~FUnixSignalGameHitchHeartBeat();

	void Init();

    double HitchThresholdS = -1.0;
	double StartTime = 0.0;
	bool bHasCmdLine = false;
	bool bDisabled = false;
	int32 SuspendCount = 0;
	bool bTimerCreated = false;
	bool bStartSuspended = false;
	timer_t TimerId = 0;
};
