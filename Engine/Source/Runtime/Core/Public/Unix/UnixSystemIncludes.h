// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Set up compiler pragmas, etc

#include "CoreTypes.h"

// IWYU pragma: begin_exports
#include "Unix/UnixPlatformCompilerSetup.h"

#include <string.h>
#include <alloca.h>
#include <stdlib.h>
#include <stdio.h>
#include <float.h>
#include <ctype.h>
#include <stdarg.h>
#include <stdint.h>
#include <limits.h>
#include <sys/time.h>
#include <math.h>
#include <wchar.h>
#include <wctype.h>
#include <errno.h>
#include <unistd.h>
#include <pthread.h>
#include <sys/types.h>
#include <sys/stat.h>
#include <pwd.h>
#include <fcntl.h>
#include <dirent.h>
#include <dlfcn.h>
#include <utime.h>
#include <execinfo.h>
#include <signal.h>
#if PLATFORM_ENABLE_VECTORINTRINSICS && !PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	#include <xmmintrin.h>
#endif
#include <sys/utsname.h>
#include <libgen.h>

struct tagRECT
{
	int32 left;
	int32 top;
	int32 right;
	int32 bottom;
};
typedef struct tagRECT RECT;

// In glibc 2.17, __secure_getenv was renamed to secure_getenv.
#if defined(__GLIBC__) && (__GLIBC__ == 2) && (__GLIBC_MINOR__ < 17)
#  define secure_getenv __secure_getenv
#endif

// IWYU pragma: end_exports
