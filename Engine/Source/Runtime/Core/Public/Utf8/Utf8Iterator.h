// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "CoreTypes.h"

/**
 * Iterator for decoding a UTF-8 encoded string view as Unicode codepoints (UTF-32).
 *
 * Usage:
 *   for (FUtf8Iterator It(MyStringView); It; ++It)
 *   {
 *       UTF32CHAR Codepoint = *It;
 *   }
 *
 * Invalid UTF-8 sequences are decoded following the Unicode Standard's "maximal subpart"
 * rule: each maximal initial subsequence of bytes that could be the beginning of a
 * well-formed sequence is replaced by exactly one U+FFFD replacement character.  In
 * practice this means that a lone invalid or out-of-range lead byte is consumed by
 * itself, and any continuation bytes (0x80–0xBF) that follow it are each decoded as
 * independent errors, each producing their own replacement character.
 *
 * Sequences truncated at the end of the view also produce a replacement character and
 * advance the iterator to the end of the view.
 *
 * Zero bytes within the view are treated as ordinary data and do not terminate iteration.
 *
 * The iterator is non-copyable. Construct it with an explicit FUtf8StringView.
 */
class FUtf8Iterator
{
public:
	UE_NONCOPYABLE(FUtf8Iterator)

	/** The Unicode replacement character emitted for any invalid or truncated sequence. */
	static constexpr UTF32CHAR ReplacementCharacter = (UTF32CHAR)0xFFFDu;

	/** Construct an iterator positioned at the start of the given UTF-8 view. */
	[[nodiscard]] CORE_API explicit FUtf8Iterator(FUtf8StringView InView);

	/**
	 * Returns true while the iterator is positioned at a valid codepoint.
	 * Becomes false once all code unit sequences in the view have been consumed.
	 */
	[[nodiscard]] UE_REWRITE explicit operator bool() const
	{
		return ViewEnd != nullptr;
	}

	/**
	 * Returns the codepoint at the current position.
	 * Only valid when operator bool() returns true.
	 */
	[[nodiscard]] UE_REWRITE UTF32CHAR operator*() const
	{
		return CachedCodepoint;
	}

	/** Advance to the next codepoint. */
	CORE_API void operator++();
	UE_REWRITE void operator++(int)
	{
		++(*this);
	}

private:
	/** Decode the sequence at Pos, store the result in CachedCodepoint, and
	 *  advance Pos to the first byte of the following sequence. */
	void DecodeNext();

	const UTF8CHAR* Pos;
	const UTF8CHAR* ViewEnd;
	UTF32CHAR       CachedCodepoint;
};
