// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compression/CompressedBuffer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "IO/IoHash.h"
#include "Misc/ConfigCacheIni.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Virtualization/VirtualizationTypes.h"

class FPackagePath;
class FText;
class UObject;
struct FAnalyticsEventAttribute;

namespace UE::Virtualization
{

/** Profiling data containing all activity relating to payloads. */
struct FPayloadActivityInfo
{
	struct FActivity
	{
		/** The number of payloads that have been involved by the activity. */
		int64 PayloadCount = 0;
		/** The total size of all payloads involved in the activity, in bytes. */
		int64 TotalBytes = 0;
		/** The total number of cycles spent on the activity across all threads. */
		int64 CyclesSpent = 0;
	};

	FActivity Pull;
	FActivity Push;
	FActivity Cache;
};

/** Profiling data containing all activity relating to a single backend */
struct FBackendStats
{
	/** Name of the backend as provided by the current VA graph found in ini:Engine */
	FString ConfigName;
	/** Full debug name of the backend */
	FString DebugName;
	
	/** The storage type of the backend */
	EStorageType Type;

	/** Payload activity for the current backend */
	FPayloadActivityInfo PayloadActivity;
};

/** Info about a rehydration operation */
struct FRehydrationInfo
{
	/** Size of the package before it was rehydrated */
	uint64 OriginalSize = 0;
	/** Size of the package after rehydration, this does not including padding if any was applied */
	uint64 RehydratedSize = 0;
	/** The number of payloads that needed to be rehydrated */
	int32 NumPayloadsRehydrated = 0;
};

/** 
 * This interface can be implemented and passed to a FPushRequest as a way of providing the payload 
 * to the virtualization system for a push operation but deferring the loading of the payload from 
 * disk until it is actually needed. In some cases this allows the loading of the payload to be 
 * skipped entirely (if the payload is already in all backends for example) or can prevent memory
 * spikes caused by loading a large number of payloads for a batched push request.
 * 
 * Note that if the backend graph contains multiple backends then payloads may be requested 
 * multiple times. It will be up to the provider implementation to decide if a requested
 * payload should be cached in case of future access or not. The methods are not const in order
 * to make it easier for derived classes to cache the results if needed without the use of
 * mutable.
 */
class IPayloadProvider
{
public:
	IPayloadProvider() = default;
	virtual ~IPayloadProvider() = default;

	/** 
	 * Should return the payload for the given FIoHash. If the provider fails to find the payload
	 * then it should return a null FCompressedBuffer to indicate the error.
	 */
	virtual FCompressedBuffer RequestPayload(const FIoHash& Identifier) = 0;

	/** Returns the current size of the payload on disk. */
	virtual uint64 GetPayloadSize(const FIoHash& Identifier) = 0;
};

/** The result of a push operation on a single payload */
struct FPushResult
{
	/** 
	 * Records the status of the push, negative values indicate an error, positive values indicate 
	 * that the payload is stored in the target backend(s) and a zero value means that the operation 
	 * has not yet run 
	 */
	enum class EStatus : int8
	{
		/** The push operation caused an error */
		Error			= -5,
		/** The push operation storage type is disabled */
		ProcessDisabled	= -4,
		/** There were no backends that supported the push operation */
		NoBackend		= -3,
		/** The push operation was run on an invalid payload id */
		Invalid			= -2,
		/** The payload was rejected by the filtering system */
		Filtered		= -1,
		/** The push operation has not yet been run */
		Pending			= 0,
		/** The payload was already present in the target backend(s) */
		AlreadyExisted	= 1,
		/** The payload was pushed to the backend(s) */
		Pushed			= 2
	};

	FPushResult() = default;
	~FPushResult() = default;

	static FPushResult GetAsError()
	{
		return FPushResult(EStatus::Error);
	}

	static FPushResult GetAsProcessDisabled()
	{
		return FPushResult(EStatus::ProcessDisabled);
	}

	static FPushResult GetAsNoBackend()
	{
		return FPushResult(EStatus::NoBackend);
	}

	static FPushResult GetAsInvalid()
	{
		return FPushResult(EStatus::Invalid);
	}

	static FPushResult GetAsFiltered(EPayloadFilterReason Reason)
	{
		return FPushResult(EStatus::Filtered, Reason);
	}

	static FPushResult GetAsAlreadyExists()
	{
		return FPushResult(EStatus::AlreadyExisted);
	}

	static FPushResult GetAsPushed()
	{
		return FPushResult(EStatus::Pushed);
	}

	/** Returns true if the payload was actually uploaded to the target backend(s) */
	bool WasPushed() const
	{
		return Status == EStatus::Pushed;
	}

	/** Returns true if the payload experienced an error during upload */
	bool WasError() const
	{
		return Status < EStatus::Pending;
	}

	/** Returns true if the payload is stored in the target backend(s) after the operation completed */
	bool IsVirtualized() const
	{
		return Status > EStatus::Pending;
	}

	/** Returns true if the payload was rejected by the filtering system */
	bool IsFiltered() const
	{
		if (Status == EStatus::Filtered)
		{
			// Quick sanity check, if the payload was filtered there must be a reason
			check(FilterReason != EPayloadFilterReason::None);
			return true;
		}
		else
		{
			return false;
		}
	}

	/** Returns why the payload was rejected by the filtering system (if there was one) */
	EPayloadFilterReason GetFilterReason() const
	{
		return FilterReason;
	}

private:

	FPushResult(EStatus InStatus)
		: Status(InStatus)
	{
		check(Status != EStatus::Pending);
	}

	FPushResult(EStatus InStatus, EPayloadFilterReason InReason)
		: Status(InStatus)
		, FilterReason(InReason)
	{
		check(Status != EStatus::Pending);
		check(FilterReason != EPayloadFilterReason::None);
	}

	EStatus Status = EStatus::Pending;
	EPayloadFilterReason FilterReason = EPayloadFilterReason::None;
};

/** 
 * Data structure representing a request to push a payload to a backend storage system. 
 * Note that a request can either before for payload already in memory (in which case the
 * payload should be passed into the constructor as a FCompressedBuffer) or by a 
 * IPayloadProvider which will provide the payload on demand.
*/
struct FPushRequest
{
	FPushRequest() = delete;
	~FPushRequest() = default;

	/** 
	 * Create a request for a payload already in memory 
	 *
	 * @param InIdentifier	The hash of the payload in its uncompressed form
	 * @param InPayload		The payload, this can be in any compressed format that the caller wishes.
	 * @param InContext		Content showing where the payload came from. If it comes from a package then this should be the package path
	 */
	FPushRequest(const FIoHash& InIdentifier, const FCompressedBuffer& InPayload, const FString& InContext)
		: Identifier(InIdentifier)
		, Payload(InPayload)
		, Context(InContext)
	{

	}

	/**
	 * Create a request for a payload already in memory
	 *
	 * @param InIdentifier	The hash of the payload in its uncompressed form
	 * @param InPayload		The payload, this can be in any compressed format that the caller wishes.
	 * @param InContext		Content showing where the payload came from. If it comes from a package then this should be the package path
	 */
	FPushRequest(const FIoHash& InIdentifier, FCompressedBuffer&& InPayload, FString&& InContext)
		: Identifier(InIdentifier)
		, Payload(InPayload)
		, Context(InContext)
	{

	}

	/**
	 * Create a request for a payload to be loaded on demand
	 *
	 * @param InIdentifier	The hash of the payload in its uncompressed form
	 * @param InProvider	The provider that will load the payload when requested. The providers lifespan must exceed that of the FPushRequest
	 * @param InContext		Content showing where the payload came from. If it comes from a package then this should be the package path
	 */
	FPushRequest(const FIoHash& InIdentifier, IPayloadProvider& InProvider, FString&& InContext)
		: Identifier(InIdentifier)
		, Provider(&InProvider)
		, Context(InContext)
	{

	}

	/** Return the identifer used in the request */
	const FIoHash& GetIdentifier() const
	{
		return Identifier;
	}

	/** Returns the size of the payload when it was on disk */
	uint64 GetPayloadSize() const
	{
		if (Provider != nullptr)
		{
			return Provider->GetPayloadSize(Identifier);
		}
		else
		{
			return Payload.GetCompressedSize();
		}
	}
	
	/** Returns the payload */
	FCompressedBuffer GetPayload() const
	{
		if (Provider != nullptr)
		{
			return Provider->RequestPayload(Identifier);
		}
		else
		{
			return Payload;
		}
	}

	/** Returns the context of the payload */
	const FString& GetContext() const
	{
		return Context;
	}

	void ResetResult()
	{
		Result = FPushResult();
	}

	void SetResult(FPushResult InResult)
	{
		Result = InResult;
	}

	const FPushResult& GetResult() const
	{
		return Result;
	}

private:
	/** The identifier of the payload */
	FIoHash Identifier;

	/** The payload data */
	FCompressedBuffer Payload;

	/** Provider to retrieve the payload from */
	IPayloadProvider* Provider = nullptr;

	/** A string containing context for the payload, typically a package name */
	FString Context;

	FPushResult Result;
};

/** Data structure representing a payload pull request */
struct FPullRequest
{
private:
	enum class EStatus : int8
	{
		Error = -1,
		Pending = 0,
		Success = 1
	};

public:
	FPullRequest(const FIoHash& InIdentifier)
		: Identifier(InIdentifier)
	{

	}

	const FIoHash& GetIdentifier() const
	{
		return Identifier;
	}

	const FCompressedBuffer& GetPayload() const
	{
		return Payload;
	}

	bool IsSuccess() const
	{
		return Status == EStatus::Success;
	}

	void SetError()
	{
		Status = EStatus::Error;
		Payload.Reset();
	}

	void SetPayload(const FCompressedBuffer& InPayload)
	{
		Payload = InPayload;
		Status = Payload ? EStatus::Success : EStatus::Error;
	}

	void SetPayload(FCompressedBuffer&& InPayload)
	{
		Payload = MoveTemp(InPayload);
		Status = Payload ? EStatus::Success : EStatus::Error;
	}

private:
	FIoHash Identifier;
	FCompressedBuffer Payload;
	EStatus Status = EStatus::Pending;
};

/** Contains the results of the package virtualization process */
struct FVirtualizationResult
{
	/** A list of errors encountered during the process */
	TArray<FText> Errors;

	/** A list of tags associated with the group of packages that were virtualized */
	TArray<FText> DescriptionTags;

	/** A list of packages that were actually virtualized and had locally stored payloads removed */
	TArray<FString> VirtualizedPackages;
	/** A list of packages that were checked out of revision control during the process */
	TArray<FString> CheckedOutPackages;

	/** The length of time that the process took in seconds */
	double TimeTaken = 0.0;

	/** The total size of the packages that actually were virtualized before they were modified */
	uint64 PreOperationSize = 0;
	/** The total size of the packages that actually were virtualized after they have been modified */
	uint64 PostOperationSize = 0;

	/* Returns if the virtualization process succeeded or not */
	bool WasSuccessful() const
	{
		return Errors.IsEmpty();
	}

	/** Add an error message to the result */
	void AddError(const FText& ErrorMsg)
	{
		Errors.Add(ErrorMsg);
	}

	/** Add an error message to the result */
	void AddError(FText&& ErrorMsg)
	{
		Errors.Add(MoveTemp(ErrorMsg));
	}

	/** Returns how many errors the process has currently encountered */
	int32 GetNumErrors() const
	{
		return Errors.Num();
	}

};

/** Contains the results of the package virtualization process */
struct FRehydrationResult
{
	/** A list of errors encountered during the process */
	TArray<FText> Errors;

	/** A list of packages that were actually rehydrated and had payloads added to local storage */
	TArray<FString> RehydratedPackages;
	/** A list of packages that were checked out of revision control during the process */
	TArray<FString> CheckedOutPackages;

	/** The length of time that the process took in seconds */
	double TimeTaken = 0.0;

	/** The total size of the packages that actually were rehydrated before they were modified */
	uint64 PreOperationSize = 0;
	/** The total size of the packages that actually were rehydrated after they have been modified */
	uint64 PostOperationSize = 0;

	/* Returns if the rehydration process succeeded or not */
	bool WasSuccessful() const
	{
		return Errors.IsEmpty();
	}

	/** Add an error message to the result */
	void AddError(const FText& ErrorMsg)
	{
		Errors.Add(ErrorMsg);
	}

	/** Add an error message to the result */
	void AddError(FText&& ErrorMsg)
	{
		Errors.Add(MoveTemp(ErrorMsg));
	}

	/** Returns how many errors the process has currently encountered */
	int32 GetNumErrors() const
	{
		return Errors.Num();
	}
};

/** 
 * The set of parameters to be used when initializing the virtualization system. The 
 * members must remain valid for the duration of the call to ::Initialize. It is not
 * expected that any virtualization system will store a reference to the members, if
 * they want to retain the data then they will make their own copies.
 */
struct FInitParams
{
	FInitParams(FStringView InProjectName, const FConfigFile& InConfigFile)
		: ProjectName(InProjectName)
		, ConfigFile(InConfigFile)
	{

	}

	/** The name of the current project (will default to FApp::GetProjectName()) */
	FStringView ProjectName;

	/** The config file to load the settings from (will default to GEngineIni) */
	const FConfigFile& ConfigFile;
};

/**
 * Creates the global IVirtualizationSystem if it has not already been set up. This can be called explicitly
 * during process start up but it will also be called by IVirtualizationSystem::Get if it detects that the
 * IVirtualizationSystem has not yet been set up.
 * 
 * This version will use the default values of FInitParams.
 */
CORE_API void Initialize(EInitializationFlags Flags);

/**
 * This version of ::Initialize takes parameters via the FInitParams structure.
 */
CORE_API void Initialize(const FInitParams& InitParams, EInitializationFlags Flags);

/**
 * Returns if the system should be initialized before the slate system has been initialized or not.
 * 
 * Originally the VA system would be initialized as early as possible, but this would occur before
 * slate has been set up and prevents us from initiating interactive dialogs. Now that we have
 * the firm rule that engine content cannot be virtualized it should be safe for us to initialize
 * the system after slate allowing dialogs.
 * 
 * This function exists so that a project could opt into the original behavior by setting the 
 * config file value "engine.ini:[Core.ContentVirtualization]:InitPreSlate=true"
 * This is only provided as a way for teams to quickly fix any issue that the later initialization
 * might cause and is not a long term feature. This will be deprecated and removed in future releases.
 * 
 * Note that this is only really supported in the editor as we use the global config file, not one
 * provided. Standalone programs should just initialize the system as needed.
 */
CORE_API bool ShouldInitializePreSlate();

/**
 * Shutdowns the global IVirtualizationSystem if it exists. 
 * Calling this is optional as the system will shut itself down along with the rest of the engine.
 */
CORE_API void Shutdown();

/** Used to customize the analytics events being sent via the virtualization system */
enum class EAnalyticsFlags : uint32
{
	None = 0,
	/** The event needs to be completely handled/flushed by the listening systems before returning.
		Commonly used when sending events right before terminating the process */
	Flush = 1 << 0
};

using FAnalyticsRecordEvent = TTSMulticastDelegate<void(const FString&, const TArray<FAnalyticsEventAttribute>&, EAnalyticsFlags)>;

/** 
 * This event is used by the virtualization system to send analytics events. Telemetry systems that wish to process asset virtualization
 * events should register to this. Note that these events could be raised on different threads so the listener will need to be threadsafe.
 */
CORE_API FAnalyticsRecordEvent& GetAnalyticsRecordEvent();

/** 
 * The base interface for the virtualization system. An Epic maintained version can be found in the Virtualization module.
 * To implement your own, simply derived from this interface and then use the
 * UE_REGISTER_VIRTUALIZATION_SYSTEM macro in the cpp to register it as an option. 
 * You can then set the config file option [Core.ContentVirtualization]SystemName=FooBar, where FooBar should be the 
 * SystemName parameter you used when registering with the macro.
 * 
 * Special Cases:
 * SystemName=Off		-	This is the default set up and means a project will not use content virtualization
 *							Note that calling IVirtualizationSystem::Get() will still return a valid 
 *							IVirtualizationSystem implementation, but all push and pull operations will result 
 *							in failure and IsEnabled will always return false.
 * SystemName=Default	-	This will cause the default Epic implementation to be used @see VirtualizationManager
 */
class IVirtualizationSystem
{
public:
	IVirtualizationSystem() = default;
	virtual ~IVirtualizationSystem() = default;

	/**
	 * Initialize the system from the parameters given in the FInitParams structure.
	 * The system can only rely on the members of FInitParams to be valid for the duration of the method call, so
	 * if a system needs to retain information longer term then it should make it's own copy of the required data.
	 * 
	 * NOTE: Although it is relatively easy to access cached FConfigFiles, systems should use the one provided 
	 * by InitParams to ensure that the correct settings are parsed.
	 * 
	 * @param InitParam The parameters used to initialize the system
	 * @return				True if the system was initialized correctly, otherwise false. Note that if the method
	 *						returns false then the system will be deleted and the default FNullVirtualizationSystem
	 *						will be used instead.
	 */
	virtual bool Initialize(const FInitParams& InitParams) = 0;

	/** Returns true if a virtualization system has been initialized and false if not */
	CORE_API static bool IsInitialized();

	/** Returns the name of the currently initialized system, will be NAME_None if initialization failed or the system is not enabled */
	CORE_API static FName GetSystemName();

	/** 
	 * Gain access to the current virtualization system active for the project. If the system has not yet been 
	 * initialized then calling this method will initialize it.
	 */
	CORE_API static IVirtualizationSystem& Get();

	/** Poll to see if content virtualization is enabled or not. */
	virtual bool IsEnabled() const = 0;

	/** Poll to see if pushing virtualized content to the given backend storage type is enabled or not. */
	virtual bool IsPushingEnabled(EStorageType StorageType) const = 0;

	/** 
	 * Run checks to see if the payload can be virtualized or not.
	 * 
	 * @param	Owner	The UObject that contains the payload, this can be nullptr if the payload is not owned by one
	 * 
	 * @return	The reasons why the payload can not be virtualized, encoded as a bitfield of EPayloadFilterReason. 
	 *			This bitfield will be EPayloadFilterReason::None (0) if no reason was found.
	 */
	virtual EPayloadFilterReason FilterPayload(const UObject* Owner) const = 0;

	/** 
	 * Poll to see if the virtualization process failing when submitting a collection of files to source
	 * control should block that submit or allow it to continue.
	 * 
	 * NOTE: This is a bit of an odd ball option to be included in the VirtualizationSystem API but at
	 * the moment we don't really have a better place for it. This might be removed or replaced in future
	 * releases.
	 * 
	 * @return		True if the calling code should continue with submitting files if the virtualization
	 *				process failed. If it returns false then the calling code should prevent the submit.
	 */
	virtual bool AllowSubmitIfVirtualizationFailed() const = 0;
	
	/**
	 * Push one or more payloads to a backend storage system. @See FPushRequest.
	 * 
	 * @param	Requests	A collection of one or more payload push requests
	 * @param	StorageType	The type of storage to push the payload to, @See EStorageType for details.
	 * 
	 * @return	When StorageType is EStorageType::Cache this will return true as long each payload
	 *			ends up being stored in at least one of the cache backends.
	 *			When StorageType is EStorageType::Persistent this will only return true if each payload
	 *			endd up being stored in *all* of the backends.
	 *			This is because the cache backends are not considered essential and it is not the end of
	 *			the world if a payload is missing but the persistent backends must be reliable.
	 */
	virtual bool PushData(TArrayView<FPushRequest> Requests, EStorageType StorageType) = 0;

	/**
	 * Push a payload to the virtualization backends.
	 *
	 * @param	Request			A single push request
	 * @param	PackageContext	Context for the payload being submitted, typically the name from the UPackage that owns it.
	 *
	 * @return	When StorageType is EStorageType::Cache this will return true as long each payload
	 *			ends up being stored in at least one of the cache backends.
	 *			When StorageType is EStorageType::Persistent this will only return true if each payload
	 *			endd up being stored in *all* of the backends.
	 *			This is because the cache backends are not considered essential and it is not the end of
	 *			the world if a payload is missing but the persistent backends must be reliable.
	 */
	bool PushData(FPushRequest Request, EStorageType StorageType)
	{
		return PushData(MakeArrayView(&Request, 1), StorageType);
	}

	/**
	 * Push a payload to the virtualization backends.
	 *
	 * @param	Id				The identifier of the payload being pushed.
	 * @param	Payload			The payload itself in FCompressedBuffer form, it is assumed that if the buffer is to
	 *							be compressed that it will have been done by the caller.
	 * @param	Context			Context for the payload being submitted, typically the name from the UPackage that owns it.
	 * @param	StorageType		The type of storage to push the payload to, @See EStorageType for details.
	 *
	 * @return	When StorageType is EStorageType::Cache this will return true as long each payload
	 *			ends up being stored in at least one of the cache backends.
	 *			When StorageType is EStorageType::Persistent this will only return true if each payload
	 *			endd up being stored in *all* of the backends.
	 *			This is because the cache backends are not considered essential and it is not the end of
	 *			the world if a payload is missing but the persistent backends must be reliable.
	 */
	bool PushData(const FIoHash& Id, const FCompressedBuffer& Payload, const FString& Context, EStorageType StorageType)
	{
		FPushRequest Request(Id, Payload, Context);
		return PushData(MakeArrayView(&Request, 1), StorageType);
	}

	/**
	 * Pull a number of payloads from the virtualization backends.
	 *
	 * @param	Requests	An array of payload pull requests. @see FPullRequest
	 * @return				True if the requests succeeded, false if one or more requests failed
	 */
	virtual bool PullData(TArrayView<FPullRequest> Requests) = 0;

	/** 
	 * Pull a single payload from the virtualization backends.
	 * 
	 * @param Id	The hash of the payload to pull
	 * @return		A valid buffer representing the payload if the pull was successful. 
	 *				An invalid buffer if the pull failed.
	 */
	FCompressedBuffer PullData(const FIoHash& Id)
	{
		FPullRequest Request(Id);

		if (PullData(MakeArrayView(&Request, 1)))
		{
			return Request.GetPayload();
		}
		else
		{
			return FCompressedBuffer();
		}
	}

	/**
	 * Pull a single payload from the virtualization backends.
	 *
	 * @param Request	A payload pull request. @see FPullRequest
	 * @return			True if the pull failed and the request will return a valid payload.
	 *					False if the pull failed and the request will return an invalid payload
	 */
	bool PullData(FPullRequest Request)
	{
		return PullData(MakeArrayView(&Request, 1));
	}

	/**
	 * Query if a number of payloads exist or not in the given storage type. 
	 * 
	 * @param	Ids					One or more payload identifiers to test
	 * @param	StorageType			The type of storage to push the payload to, @See EStorageType for details.
	 * @param	OutStatuses [out]	An array containing the results for each payload. @See FPayloadStatus
	 * 								If the operation succeeds the array will be resized to match the size of Ids. 
	 * 
	 * @return	True if the operation succeeded and the contents of OutStatuses is valid. False if errors were 
	 * 			encountered in which case the contents of OutStatuses should be ignored.
	 */
	virtual EQueryResult QueryPayloadStatuses(TArrayView<const FIoHash> Ids, EStorageType StorageType, TArray<EPayloadStatus>& OutStatuses) = 0;
	
	/**
	 * Runs the virtualization process on a set of packages. All of the packages will be parsed and any found to be containing locally stored
	 * payloads will have them removed but before they are removed they will be pushed to persistent storage.
	 * 
	 * Note that if errors occur some or all of the payloads could still get pushed to persistent storage, usually if the errors occur when
	 * trying to remove the now virtualized payload from the packages on disk.
	 * In addition, if errors do occur when removing the virtualized payloads, some packages may be virtualized successfully. In any case the
	 * packages should still be usable and safe to checkin after the process has run, even with failure cases.
	 *
	 * @param PackagePaths			An array of file paths to packages that should be virtualized. If a path resolves to a file that is not
	 *								a valid package then it will be silently skipped and will not be considered an error.
	 * @param Options				An enum bitfield containing various options for the process. @see EVirtualizationOptions
	 * @param OutResultInfo			A struct that will contain info about the process. @see FVirtualizationResult
	 *
	 * @return						A EVirtualizationResult enum with the status of the process. If the status is not EVirtualizationResult::Success
	 *								then the parameter OutErrors should contain at least one entry.
	 */
	virtual FVirtualizationResult TryVirtualizePackages(TConstArrayView<FString> PackagePaths, EVirtualizationOptions Options) = 0;

	/**
	 * Runs the rehydration process on a set of packages. This involves downloading virtualized payloads and placing them back in the trailer of
	 * the given packages.
	 * 
	 * @param PackagePaths		An array containing the absolute file paths of packages
	 * @param Options			An enum bitfield containing various options for the process. @see EVirtualizationOptions
	 * @param OutResultInfo		A struct that will contain info about the process. @see FVirtualizationResult
	 * 
	 * @return	A ERehydrationResult enum with the status of the process. If the status indicates any sort of failure then OutResultInfo.Errors should
	 *			contain at least one entry.
	 */
	virtual FRehydrationResult TryRehydratePackages(TConstArrayView<FString> PackagePaths, ERehydrationOptions Options) = 0;

	/**
	 * Rehydrates a number of packages into memory buffers.
	 * Note that if a package does not require rehydration we will still return the package in a memory buffer but it will be the same
	 * as the package on disk.
	 * 
	 * @param PackagePaths		An array containing the absolute file paths of packages.
	 * @param PaddingAlignment	Byte alignment to pad each package buffer too, a value of 0 will result in the buffers being the same size as the packages
	 * @param OutErrors			Any errors encountered while rehydration will be added here
	 * @param OutPackages		The rehydrated packages as memory buffers. Each entry should match the corresponding entry in PackagePaths. This array is
	 *							only guaranteed to be correct if the method returns ERehydrationResult::Success.
	 * @param OutInfo			Information about the rehydration process, each entry should match the corresponding entry in PackagePaths assuming that
	 *							the method returns ERehydrationResult::Success. This parameter is optional, if the information is not required then
	 *							pass in nullptr to skip.
	 * @return					ERehydrationResult::Success if the rehydration suceeeds and OutPackages/OutInfocan be trued, otherwise it will return an
	 *							error value. @see ERehydrationResult
	 */
	virtual ERehydrationResult TryRehydratePackages(TConstArrayView<FString> PackagePaths, uint64 PaddingAlignment, TArray<FText>& OutErrors, TArray<FSharedBuffer>& OutPackages, TArray<FRehydrationInfo>* OutInfo) = 0;

	/** When called the system should write any performance stats that it has been gathering to the log file */
	virtual void DumpStats() const = 0;

	using GetPayloadActivityInfoFuncRef = TFunctionRef<void(const FString& DebugName, const FString& ConfigName, const FPayloadActivityInfo& PayloadInfo)>;
	
	/** Access profiling info relating to payload activity per backend. Stats will only be collected if ENABLE_COOK_STATS is enabled.*/
	virtual TArray<FBackendStats> GetBackendStatistics() const = 0;
	
	/** Access profiling info for all backends combined. Stats will only be collected if ENABLE_COOK_STATS is enabled.*/
	virtual FPayloadActivityInfo GetSystemStatistics() const = 0;

	/** Gather analytics data. Stats will only be collected if ENABLE_COOK_STATS is enabled.*/
	virtual void GatherAnalytics(TArray<FAnalyticsEventAttribute>& Attributes) const =0;

	//* Notification messages
	enum ENotification
	{
		PushBegunNotification,
		PushEndedNotification,
		PushFailedNotification,

		PullBegunNotification,
		PullEndedNotification,
		PullFailedNotification,
	};

	/** Declare delegate for notifications*/
	using FOnNotification = TTSMulticastDelegate<void(ENotification, const FIoHash&)>;

	virtual FOnNotification& GetNotificationEvent() = 0;
};

namespace Private
{

/** 
 * Factory interface for creating virtualization systems. This is not intended to be derived from 
 * directly. Use the provided UE_REGISTER_VIRTUALIZATION_SYSTEM macro instead 
 */
class IVirtualizationSystemFactory : public IModularFeature
{
public:
	/** Creates and returns a new virtualization system instance */
	virtual TUniquePtr<IVirtualizationSystem> Create() = 0;

	/** Returns the name of the system that this factory created */
	virtual FName GetName() = 0;
};

} // namespace Private

/**
 * Registers a class derived from IVirtualizationSystem so that it can be set as the virtualization system for
 * the process to use.
 * 
 * @param SystemClass	The class derived from IVirtualizationSystem
 * @param SystemName	The name of the system that will be used to potentially select the system for use
 */
#define UE_REGISTER_VIRTUALIZATION_SYSTEM(SystemClass, SystemName) \
	class FVirtualizationSystem##Factory : public Private::IVirtualizationSystemFactory \
	{ \
	public: \
		FVirtualizationSystem##Factory() { IModularFeatures::Get().RegisterModularFeature(FName("VirtualizationSystem"), this); }\
		virtual ~FVirtualizationSystem##Factory() { IModularFeatures::Get().UnregisterModularFeature(FName("VirtualizationSystem"), this); } \
	private: \
		virtual TUniquePtr<IVirtualizationSystem> Create() override { return MakeUnique<SystemClass>(); } \
		virtual FName GetName() override { return FName(#SystemName); } \
	}; \
	static FVirtualizationSystem##Factory FVirtualizationSystem##Factory##Instance;

namespace Experimental
{

class IVirtualizationSourceControlUtilities : public IModularFeature
{
public:
	/**
	 * Given a package path this method will attempt to sync th e.upayload file that is compatible with
	 * the .uasset file of the package.
	 *
	 * We can make the following assumptions about the relationship between .uasset and .upayload files:
	 * 1) The .uasset may be submitted to perforce without the .upayload (if the payload is unmodified)
	 * 2) If the payload is modified then the .uasset and .upayload file must be submitted at the same time.
	 * 3) The caller has already checked the existing .upayload file (if any) to see if it contains the payload
	 * that they are looking for.
	 *
	 * If the above is true then we can sync the .upayload file to the same perforce changelist as the
	 * * .uasset and be sure that we have the correct version.
	 *
	 * Note that this has only been tested with perforce and so other source control solutions are currently
	 * unsupported.
	 */
	virtual bool SyncPayloadSidecarFile(const FPackagePath& PackagePath) = 0;
};

} // namespace Experimental

} // namespace UE::Virtualization
