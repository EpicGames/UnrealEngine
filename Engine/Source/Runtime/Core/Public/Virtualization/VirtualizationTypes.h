// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Virtualization
{

/** Describes the type of storage to use for a given action */
enum class EStorageType : int8
{
	/** Store in the local cache backends, this can be called from any thread */
	Cache = 0,
	/** Store in the persistent backends, this can only be called from the game thread due to limitations with ISourceControlModule. */
	Persistent
};

/**
 * The result of a query.
 * Success indicates that the query worked and that the results are valid and can be used.
 * Any other value indicates that the query failed in some manner and that the results cannot be trusted and should be discarded.
 */
enum class EQueryResult : int8
{
	/** The query succeeded and the results are valid */
	Success = 0,
	/** The query failed with an unspecified error */
	Failure_Unknown,
	/** The query failed because the current virtualization system has not implemented it */
	Failure_NotImplemented
};

/** Describes the status of a payload in regards to a backend storage system */
enum class EPayloadStatus : int8
{
	/** The payload id was not value */
	Invalid = -1,
	/** The payload was not found in any backend for the given storage type */
	NotFound = 0,
	/** The payload was found in at least one backend but was not found in all backends available for the given storage type */
	FoundPartial,
	/** The payload was found in all of the backends available for the given storage type */
	FoundAll
};

/** The result of the virtualization process */
enum class EVirtualizationResult : uint8
{
	/** The virtualization process ran with no problems to completion. */
	Success = 0,
	/** The process failed before completion and nothing was virtualized */
	Failed
};

/** The result of the re-hydration process */
enum class ERehydrationResult : uint8
{
	/** The re-hydration process ran with no problems to completion. */
	Success = 0,
	/** The process failed before completion and nothing was re-hydrate */
	Failed,
};

/** This enum describes the reasons why a payload may not be virtualized */
enum class EPayloadFilterReason : uint16
{
	/** Not filtered, the payload can be virtualized */
	None = 0,
	/** Filtered due to the asset type of the owning UObject */
	Asset = 1 << 0,
	/** Filtered due to the path of the owning UPackage */
	Path = 1 << 1,
	/** Filtered because the payload size is below the minimum size for virtualization */
	MinSize = 1 << 2,
	/** Filtered because the owning editor bulkdata had virtualization disabled programmatically */
	EditorBulkDataCode = 1 << 3,
	/** Filtered because the package is either a UMap or the owning editor bulkdata is under a UMapBuildDataRegistry */
	MapContent = 1 << 4,
};
constexpr uint16 NumPayloadFilterReasons = 6;

ENUM_CLASS_FLAGS(EPayloadFilterReason);

/** Options used when virtualizing packages */
enum class EVirtualizationOptions : uint32
{
	None = 0,
	/** Attempt to check out files from revision control if needed */
	Checkout = 1 << 0
};

ENUM_CLASS_FLAGS(EVirtualizationOptions);

/** Options used when rehydrating packages */
enum class ERehydrationOptions : uint32
{
	None = 0,
	/** Attempt to check out files from revision control if needed */
	Checkout = 1 << 0
};

ENUM_CLASS_FLAGS(ERehydrationOptions);

/** Optional flags that change how the VA system is initialized */
enum class EInitializationFlags : uint32
{
	/** No flags are set */
	None = 0,
	/** Forces the initialization to occur, ignoring and of the lazy initialization flags */
	ForceInitialize = 1 << 0
};

ENUM_CLASS_FLAGS(EInitializationFlags);

} //UE::Virtualization