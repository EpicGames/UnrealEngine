// Copyright Epic Games, Inc. All Rights Reserved.

// #TODO: redirect to platform-agnostic version for the time being. Eventually this will become an error
#include "HAL/Platform.h"
#if !PLATFORM_WINDOWS
	#include "Microsoft/AllowMicrosoftPlatformAtomics.h"
#else

#ifndef WINDOWS_PLATFORM_ATOMICS_GUARD
	#define WINDOWS_PLATFORM_ATOMICS_GUARD
#else
	#error Nesting AllowWindowsPlatformAtomics.h is not allowed!
#endif

#define InterlockedIncrement _InterlockedIncrement
#define InterlockedDecrement _InterlockedDecrement
#if !defined(InterlockedAdd) && !PLATFORM_WINDOWS_ARM64EC
	#define InterlockedAdd _InterlockedAdd
#endif
#define InterlockedExchange _InterlockedExchange
#define InterlockedExchangeAdd _InterlockedExchangeAdd
#define InterlockedCompareExchange _InterlockedCompareExchange
#define InterlockedAnd _InterlockedAnd
#define InterlockedOr _InterlockedOr
#define InterlockedXor _InterlockedXor
#define InterlockedCompareExchangePointer _InterlockedCompareExchangePointer
#define InterlockedExchange64 _InterlockedExchange64
#define InterlockedExchangeAdd64 _InterlockedExchangeAdd64
#define InterlockedCompareExchange64 _InterlockedCompareExchange64
#define InterlockedIncrement64 _InterlockedIncrement64
#define InterlockedDecrement64 _InterlockedDecrement64

#endif //PLATFORM_*
