// Copyright Epic Games, Inc. All Rights Reserved.

// #TODO: redirect to platform-agnostic version for the time being. Eventually this will become an error
#include "HAL/Platform.h"
#if !PLATFORM_WINDOWS
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#else


#include "Windows/WindowsHWrapper.h"

#ifndef WINDOWS_PLATFORM_TYPES_GUARD
	#define WINDOWS_PLATFORM_TYPES_GUARD
#else
	#error Nesting AllowWindowsPlatformTypes.h is not allowed!
#endif

#pragma warning( push )
#pragma warning( disable : 4459 )

// Disable warnings about malformed SAL annotations - this is probably due to the UINT macro below
#pragma warning( disable : 28285 ) // d3d12.h(6236) : warning C28285: For function 'SetComputeRoot32BitConstants' '_Param_(3)' syntax error in 'SAL_readableTo(elementCount(__formal(1,Num32BitValuesToSet)*sizeof(::UINT)))' near '::UINT)))'.

#define INT ::INT
#define UINT ::UINT
#define DWORD ::DWORD
#define FLOAT ::FLOAT

#define TRUE 1
#define FALSE 0

#endif //PLATFORM_*
