// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Should not be included

UE_DEPRECATED_HEADER(4.27, "#include Microsoft/COMPointer.h instead of Windows/COMPointer.h.")

#include "Microsoft/COMPointer.h"
