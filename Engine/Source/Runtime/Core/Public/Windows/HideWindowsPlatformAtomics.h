// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

// #TODO: redirect to platform-agnostic version for the time being. Eventually this will become an error
#include "HAL/Platform.h"
#if !PLATFORM_WINDOWS
	#include "Microsoft/HideMicrosoftPlatformAtomics.h"
#else

#ifdef WINDOWS_PLATFORM_ATOMICS_GUARD
	#undef WINDOWS_PLATFORM_ATOMICS_GUARD
#else
	#error Mismatched HideWindowsPlatformAtomics.h detected.
#endif

#undef InterlockedIncrement
#undef InterlockedDecrement
#undef InterlockedAdd
#undef InterlockedExchange
#undef InterlockedExchangeAdd
#undef InterlockedCompareExchange
#undef InterlockedCompareExchangePointer
#undef InterlockedExchange64
#undef InterlockedExchangeAdd64
#undef InterlockedCompareExchange64
#undef InterlockedIncrement64
#undef InterlockedDecrement64
#undef InterlockedAnd
#undef InterlockedOr
#undef InterlockedXor


#endif //PLATFORM_*
