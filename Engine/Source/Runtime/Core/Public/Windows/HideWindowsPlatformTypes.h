// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

// #TODO: redirect to platform-agnostic version for the time being. Eventually this will become an error
#include "HAL/Platform.h"
#if !PLATFORM_WINDOWS
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
#else

#ifdef WINDOWS_PLATFORM_TYPES_GUARD
	#undef WINDOWS_PLATFORM_TYPES_GUARD
#else
	#error Mismatched HideWindowsPlatformTypes.h detected.
#endif

#undef INT
#undef UINT
#undef DWORD
#undef FLOAT

#ifdef TRUE
	#undef TRUE
#endif

#ifdef FALSE
	#undef FALSE
#endif

#pragma warning( pop )

#endif //PLATFORM_*
