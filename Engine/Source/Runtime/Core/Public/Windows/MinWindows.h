// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#if !defined(WINDOWS_H_WRAPPER_GUARD)
#pragma message("WARNING: do not include Windows/MinWindows.h file directly. Use Windows/WindowsHWrapper.h instead")
#endif

// #TODO: redirect to platform-agnostic version for the time being. Eventually this will become an error
#include "HAL/Platform.h"
#if !PLATFORM_WINDOWS
	#include "Microsoft/MinWindows.h"
#else

#include "CoreTypes.h"

#if defined(_WINDOWS_) && !defined(UE_MINIMAL_WINDOWS_INCLUDE)
	#pragma message ( " " )
	#pragma message ( "You have included windows.h before MinWindows.h" )
	#pragma message ( "All useless stuff from the windows headers won't be excluded !!!" )
	#pragma message ( " " )
#endif // _WINDOWS_

#define UE_MINIMAL_WINDOWS_INCLUDE

// WIN32_LEAN_AND_MEAN excludes rarely-used services from windows headers.
#define WIN32_LEAN_AND_MEAN

// The below excludes some other unused services from the windows headers -- see windows.h for details.
#define NOGDICAPMASKS			// CC_*, LC_*, PC_*, CP_*, TC_*, RC_
//#define NOVIRTUALKEYCODES		// VK_*
//#define NOWINMESSAGES			// WM_*, EM_*, LB_*, CB_*
//#define NOWINSTYLES			// WS_*, CS_*, ES_*, LBS_*, SBS_*, CBS_*
//#define NOSYSMETRICS			// SM_*
// #define NOMENUS				// MF_*
//#define NOICONS				// IDI_*
//#define NOKEYSTATES			// MK_*
//#define NOSYSCOMMANDS			// SC_*
//#define NORASTEROPS			// Binary and Tertiary raster ops
//#define NOSHOWWINDOW			// SW_*
#define OEMRESOURCE				// OEM Resource values
#define NOATOM					// Atom Manager routines
//#define NOCLIPBOARD			// Clipboard routines
//#define NOCOLOR				// Screen colors
//#define NOCTLMGR				// Control and Dialog routines
//#define NODRAWTEXT				// DrawText() and DT_*
//#define NOGDI					// All GDI #defines and routines
#define NOKERNEL				// All KERNEL #defines and routines
//#define NOUSER				// All USER #defines and routines
//#define NONLS					// All NLS #defines and routines
//#define NOMB					// MB_* and MessageBox()
#define NOMEMMGR				// GMEM_*, LMEM_*, GHND, LHND, associated routines
#define NOMETAFILE				// typedef METAFILEPICT
#define NOMINMAX				// Macros min(a,b) and max(a,b)
//#define NOMSG					// typedef MSG and associated routines
#define NOOPENFILE				// OpenFile(), OemToAnsi, AnsiToOem, and OF_*
#define NOSCROLL				// SB_* and scrolling routines
#define NOSERVICE				// All Service Controller routines, SERVICE_ equates, etc.
#define NOSOUND					// Sound driver routines
//#define NOTEXTMETRIC			// typedef TEXTMETRIC and associated routines
//#define NOWH					// SetWindowsHook and WH_*
//#define NOWINOFFSETS			// GWL_*, GCL_*, associated routines
#define NOCOMM					// COMM driver routines
#define NOKANJI					// Kanji support stuff.
#define NOHELP					// Help engine interface.
#define NOPROFILER				// Profiler interface.
#define NODEFERWINDOWPOS		// DeferWindowPos routines
#define NOMCX					// Modem Configuration Extensions
#define NOCRYPT
#define NOTAPE
#define NOIMAGE
#define NOPROXYSTUB
#define NORPC

struct IUnknown;

// Finally now we can include windows.h
#include <Windows.h>


#endif //PLATFORM_*
