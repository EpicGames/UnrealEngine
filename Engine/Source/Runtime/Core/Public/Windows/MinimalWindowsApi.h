// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !PLATFORM_WINDOWS
	#error this file is for PLATFORM_WINDOWS only
#endif

// redirect to new file
#include "Microsoft/MinimalWindowsApi.h"
