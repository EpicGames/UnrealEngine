// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

// #TODO: redirect to platform-agnostic version for the time being. Eventually this will become an error
#include "HAL/Platform.h"
#if !PLATFORM_WINDOWS
	#include "Microsoft/PostWindowsApi.h"
#else

// this file should only be included from WindowsHWrapper.h
#if !defined(WINDOWS_H_WRAPPER_GUARD) 
#pragma message("WARNING: do not include Windows/PostWindowsApi.h directly. Use Windows/WindowsHWrapper.h or Windows/HideWindowsPlatformTypes.h instead") 
#endif

// Re-enable warnings
THIRD_PARTY_INCLUDES_END

// Hide Windows-only types (same as HideWindowsPlatformTypes.h)
#undef INT
#undef UINT
#undef DWORD
#undef FLOAT

// Undo any Windows defines.
#undef uint8
#undef uint16
#undef uint32
#undef int32
#undef float
#undef CDECL
#undef PF_MAX
#undef CaptureStackBackTrace
#undef CopyFile
#undef CreateDesktop
#undef CreateDirectory
#undef CreateFont
#undef DeleteFile
#undef DrawText
#undef FindWindow
#undef GetClassInfo
#undef GetClassName
#undef GetCommandLine
#undef GetCurrentTime
#undef GetEnvironmentVariable
#undef GetFileAttributes
#undef GetFreeSpace
#undef GetMessage
#undef GetNextSibling
#undef GetObject
#undef GetProp
#undef GetTempFileName
#undef IMediaEventSink
#undef IsMaximized
#undef IsMinimized
#undef LoadString
#undef MemoryBarrier
#undef MoveFile
#undef PlaySound
#undef PostMessage
#undef ReportEvent
#undef SendMessage
#undef SetPort
#undef SetProp
#undef UpdateResource
#undef Yield

// Undefine all the atomics. AllowWindowsPlatformAtomics/HideWindowsPlatformAtomics temporarily defining these macros.
#undef InterlockedIncrement
#undef InterlockedDecrement
#undef InterlockedAdd
#undef InterlockedExchange
#undef InterlockedExchangeAdd
#undef InterlockedCompareExchange
#undef InterlockedCompareExchangePointer
#undef InterlockedExchange64
#undef InterlockedExchangeAdd64
#undef InterlockedCompareExchange64
#undef InterlockedIncrement64
#undef InterlockedDecrement64
#undef InterlockedAnd
#undef InterlockedOr
#undef InterlockedXor

// Restore any previously defined macros
#pragma pop_macro("MAX_uint8")
#pragma pop_macro("MAX_uint16")
#pragma pop_macro("MAX_uint32")
#pragma pop_macro("MAX_int32")
#pragma pop_macro("TEXT")
#pragma pop_macro("TRUE")
#pragma pop_macro("FALSE")

// Restore the struct packing setting
PRAGMA_POP_PLATFORM_DEFAULT_PACKING

// Redefine CDECL to our version of the #define.  <AJS> Is this really necessary?
#define CDECL	    __cdecl					/* Standard C function */

// Make sure version is high enough for API to be defined. For CRITICAL_SECTION
#if !defined(_XTL_) && (_WIN32_WINNT < 0x0403)
	#error SetCriticalSectionSpinCount requires _WIN32_WINNT >= 0x0403
#endif

#endif //PLATFORM_*
