// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

// #TODO: redirect to platform-agnostic version for the time being. Eventually this will become an error
#include "HAL/Platform.h"
#if !PLATFORM_WINDOWS
	#include "Microsoft/PreWindowsApi.h"
#else

// this file should only be included from WindowsHWrapper.h
#if !defined(WINDOWS_H_WRAPPER_GUARD) 
#pragma message("WARNING: do not include Windows/PreWindowsApi.h directly. Use Windows/WindowsHWrapper.h or Windows/AllowWindowsPlatformTypes.h instead") 
#endif

// Disable the warning that the pack size is changed in this header. We do this globally for
// clang, since the mechanism of using a sub-header to modify packing generates a -Wpragma-pack
// warning about modifying packing alignemnt in a header.
#if !defined(__clang__)
	#pragma warning(disable:4103)
#endif	// __clang__


// The 10.0.18362.0 SDK introduces an error if the packing isn't the default for the platform.
PRAGMA_PUSH_PLATFORM_DEFAULT_PACKING

// Save these macros for later; Windows redefines them
#pragma push_macro("MAX_uint8")
#pragma push_macro("MAX_uint16")
#pragma push_macro("MAX_uint32")
#pragma push_macro("MAX_int32")
#pragma push_macro("TEXT")
#pragma push_macro("TRUE")
#pragma push_macro("FALSE")

// Undefine the TEXT macro for winnt.h to redefine it, unless it's already been included
#ifndef _WINNT_
#undef TEXT
#endif

// Disable all normal third party headers
THIRD_PARTY_INCLUDES_START

#endif //PLATFORM_*
