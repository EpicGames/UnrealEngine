// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/OutputDeviceConsole.h"

class FWindowsErrorOutputDevice : public FOutputDeviceError
{
public:
	/** Constructor, initializing member variables */
	CORE_API FWindowsErrorOutputDevice();

	/**
	 * Serializes the passed in data unless the current event is suppressed.
	 *
	 * @param	Data	Text to log
	 * @param	Event	Event name used for suppression purposes
	 */
	CORE_API virtual void Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category ) override;

	/**
	 * Error handling function that is being called from within the system wide global
	 * error handler, e.g. using structured exception handling on the PC.
	 */
	CORE_API virtual void HandleError() override;

protected:
	/**
	 * Callback to allow FWindowsApplicationErrorOutputDevice to restore the UI.
	 */
	CORE_API virtual void HandleErrorRestoreUI();
};

