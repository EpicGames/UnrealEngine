// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/OutputDevice.h"
#include "Windows/MinimalWindowsApi.h"

class FName;

/**
 * Output device that writes to Windows Event Log
 */
class FWindowsEventLogOutputDevice
	: public FOutputDevice
{
	/** Handle to the event log object */
	Windows::HANDLE EventLog;

public:
	/**
	 * Constructor, initializing member variables
	 */
	CORE_API FWindowsEventLogOutputDevice();

	/** Destructor that cleans up any remaining resources */
	CORE_API virtual ~FWindowsEventLogOutputDevice();

	CORE_API virtual void Serialize(const TCHAR* Buffer, ELogVerbosity::Type Verbosity, const class FName& Category) override;
	
	/** Does nothing */
	CORE_API virtual void Flush(void);

	/**
	 * Closes any event log handles that are open
	 */
	CORE_API virtual void TearDown(void);
};
