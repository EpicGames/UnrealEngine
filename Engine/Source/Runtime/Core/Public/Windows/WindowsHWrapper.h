// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// #TODO: redirect to platform-agnostic version for the time being. Eventually this will become an error
#include "HAL/Platform.h"
#if !PLATFORM_WINDOWS
	#include "Microsoft/WindowsHWrapper.h"
#else

#include "CoreTypes.h"
#include "HAL/PlatformMemory.h"

#if defined(WINDOWS_H_WRAPPER_GUARD)
#error WINDOWS_H_WRAPPER_GUARD already defined
#endif
#define WINDOWS_H_WRAPPER_GUARD

#include "Windows/PreWindowsApi.h"
#ifndef STRICT
#define STRICT
#endif
#include "Windows/MinWindows.h"
#include "Windows/PostWindowsApi.h"

#undef WINDOWS_H_WRAPPER_GUARD

#endif //PLATFORM_*