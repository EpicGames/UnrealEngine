// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include <sal.h>

#if defined(__INTEL_LLVM_COMPILER)
	#include "IntelICX/IntelICXPlatform.h"
#elif defined(__clang__)
	#include "Clang/ClangPlatform.h"
#else
	#include "Microsoft/MSVCPlatform.h"
#endif

#if WINVER <= 0x600
	#error "Windows Vista and earlier are no longer supported"
#endif

#ifndef PLATFORM_COMPILER_CLANG
#if defined(__clang__)
#define PLATFORM_COMPILER_CLANG			1
#else
#define PLATFORM_COMPILER_CLANG			0
#endif // defined(__clang__)
#endif

#include "GenericPlatform/GenericPlatform.h"	//for FGenericPlatformTypes

/**
* Windows specific types
**/
struct FWindowsPlatformTypes : public FGenericPlatformTypes
{
#ifdef _WIN64
	typedef unsigned __int64	SIZE_T;
	typedef __int64				SSIZE_T;
#else
	typedef unsigned long		SIZE_T;
	typedef long				SSIZE_T;
#endif

#if USE_UTF8_TCHARS
	typedef UTF8CHAR TCHAR;
#endif
};

typedef FWindowsPlatformTypes FPlatformTypes;

// Base defines, must define these for the platform, there are no defaults
#define PLATFORM_DESKTOP					1
#if defined( _MANAGED ) || defined( _M_CEE )
	#define PLATFORM_COMPILER_COMMON_LANGUAGE_RUNTIME_COMPILATION	1
#endif
#define PLATFORM_CAN_SUPPORT_EDITORONLY_DATA				1

// Base defines, defaults are commented out

#define PLATFORM_LITTLE_ENDIAN								1
#define PLATFORM_SUPPORTS_UNALIGNED_LOADS					1
#define PLATFORM_SUPPORTS_FLIP_TRACKING						1

#define PLATFORM_SUPPORTS_PRAGMA_PACK						1
#if (defined(__arm__) || defined(_M_ARM) || defined(__aarch64__) || defined(_M_ARM64) || defined(_M_ARM64EC))
	#define PLATFORM_CPU_ARM_FAMILY							1
	#define PLATFORM_ENABLE_VECTORINTRINSICS_NEON			1
	#define PLATFORM_ENABLE_VECTORINTRINSICS				1
#elif (defined(_M_IX86) || defined(_M_X64))
	#define PLATFORM_CPU_ARM_FAMILY							0
	#define PLATFORM_CPU_X86_FAMILY							1
	#define PLATFORM_ENABLE_VECTORINTRINSICS				1

#endif
// FMA3 support was added starting from AMD Piledriver (excluding Jaguar) and Intel Haswell (excluding Pentium and Celeron)
#ifndef PLATFORM_ALWAYS_HAS_FMA3
	#define PLATFORM_ALWAYS_HAS_FMA3						0
#endif

#define PLATFORM_USE_LS_SPEC_FOR_WIDECHAR					0
//#define PLATFORM_USE_SYSTEM_VSWPRINTF						1
#define PLATFORM_HAS_BSD_TIME								0
#define PLATFORM_USE_PTHREADS								0
#define PLATFORM_USES_UNFAIR_LOCKS							1
#define PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED				WINDOWS_MAX_PATH
#define PLATFORM_HAS_BSD_IPV6_SOCKETS						1
#define PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS			1
#define PLATFORM_USES_MICROSOFT_LIBC_FUNCTIONS				1
#define PLATFORM_IS_ANSI_MALLOC_THREADSAFE					1
#define PLATFORM_SUPPORTS_ASYMMETRIC_FENCES					1

#define PLATFORM_SUPPORTS_TBB								1
#define PLATFORM_SUPPORTS_MIMALLOC							1
#define PLATFORM_SUPPORTS_NAMED_PIPES						1
#define PLATFORM_COMPILER_HAS_TCHAR_WMAIN					1
#define PLATFORM_SUPPORTS_EARLY_MOVIE_PLAYBACK				(!WITH_EDITOR) // movies will start before engine is initialized
#define PLATFORM_USE_GENERIC_STRING_IMPLEMENTATION			0 // Can set this to 1 if you need to debug FGenericWidePlatformString::GetVarArgs on Windows
#define PLATFORM_SUPPORTS_VARIABLE_RATE_SHADING				1
#define PLATFORM_SUPPORTS_MESH_SHADERS						1
#define PLATFORM_SUPPORTS_WORKGRAPH_SHADERS					1
#define PLATFORM_SUPPORTS_BINDLESS_RENDERING				1
#define PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS			1
#define PLATFORM_USES_DYNAMIC_HDR_SETTING					1
#define PLATFORM_USES__ALIGNED_MALLOC						1

#if WITH_EDITOR
#define PLATFORM_FILE_READER_BUFFER_SIZE					(256*1024)
#endif

#define PLATFORM_SUPPORTS_STACK_SYMBOLS						1

#define PLATFORM_GLOBAL_LOG_CATEGORY						LogWindows

#define PLATFORM_SUPPORTS_BORDERLESS_WINDOW					1

#if defined(__clang__)
#define PLATFORM_RETURN_ADDRESS_FOR_CALLSTACKTRACING		PLATFORM_RETURN_ADDRESS
#define PLATFORM_USE_CALLSTACK_ADDRESS_POINTER				0
#else
#define PLATFORM_RETURN_ADDRESS_FOR_CALLSTACKTRACING		PLATFORM_RETURN_ADDRESS_POINTER
#define PLATFORM_USE_CALLSTACK_ADDRESS_POINTER				1
#endif

#define WINDOWS_USE_FEATURE_APPLICATIONMISC_CLASS			1
#define WINDOWS_USE_FEATURE_PLATFORMMISC_CLASS				1
#define WINDOWS_USE_FEATURE_PLATFORMHTTP_CLASS				1


// Q: Why is there a __nop() before __debugbreak()?
// A: VS' debug engine has a bug where it will silently swallow explicit
// breakpoint interrupts when single-step debugging either line-by-line or
// over call instructions. This can hide legitimate reasons to trap. Asserts
// for example, which can appear as if the did not fire, leaving a programmer
// unknowingly debugging an undefined process.
#define PLATFORM_BREAK() (__nop(), __debugbreak())
#define PLATFORM_BREAK_IS_EXPRESSION 1

// Intrinsics for 128-bit atomics on Windows platform requires Windows 8 or higher (WINVER>=0x0602)
// http://msdn.microsoft.com/en-us/library/windows/desktop/hh972640.aspx
#define PLATFORM_HAS_128BIT_ATOMICS							(WINVER >= 0x602)

#ifdef CDECL
#undef CDECL
#endif

// Function type macros.
#define VARARGS     __cdecl											/* Functions with variable arguments */
#define CDECL	    __cdecl											/* Standard C function */
#define STDCALL		__stdcall										/* Standard calling convention */
#define FORCEINLINE __forceinline									/* Force code to be inline */
#define FORCENOINLINE __declspec(noinline)							/* Force code to NOT be inline */

// When using clang-cl we will pick up the ClangPlatform.h definition of this macro before this, so check first!
#ifndef FUNCTION_NON_NULL_RETURN_START
#define FUNCTION_NON_NULL_RETURN_START _Ret_notnull_				/* Indicate that the function never returns nullptr. */
#endif

#define DECLARE_UINT64(x)	x

// Optimization macros (uses __pragma to enable inside a #define).
#if !defined(__clang__)
	#define PRAGMA_DISABLE_OPTIMIZATION_ACTUAL __pragma(optimize("",off))
	#define PRAGMA_ENABLE_OPTIMIZATION_ACTUAL  __pragma(optimize("",on))
#elif defined(_MSC_VER)		// Clang only supports __pragma with -fms-extensions
	#define PRAGMA_DISABLE_OPTIMIZATION_ACTUAL __pragma(clang optimize off)
	#define PRAGMA_ENABLE_OPTIMIZATION_ACTUAL  __pragma(clang optimize on)
#endif

#if !defined(__clang__)
	#define PLATFORM_EMPTY_BASES __declspec(empty_bases)
#endif

// Tells the compiler to put the decorated function in a certain section (aka. segment) of the executable.
#define PLATFORM_CODE_SECTION(Name) __declspec(code_seg(Name)) 

// Backwater of the spec. All compilers support this except microsoft, and they will soon
#if !defined(__clang__)		// Clang expects typename outside template
	#define TYPENAME_OUTSIDE_TEMPLATE
#endif

#pragma warning(disable : 4481) // nonstandard extension used: override specifier 'override'

#define ABSTRACT abstract

// Strings.
#define LINE_TERMINATOR TEXT("\r\n")
#define LINE_TERMINATOR_ANSI "\r\n"

// Alignment.
#if defined(__clang__)
	#define GCC_PACK(n) __attribute__((packed,aligned(n)))
	#define GCC_ALIGN(n) __attribute__((aligned(n)))
	#if defined(_MSC_VER)
		#define MS_ALIGN(n) __declspec(align(n)) // With -fms-extensions, Clang will accept either alignment attribute
	#endif
#else
	#define MS_ALIGN(n) __declspec(align(n))
#endif

// Pragmas
#if !defined(__clang__)
	#define MSVC_PRAGMA(Pragma) __pragma(Pragma)
#endif

// Prefetch
#define PLATFORM_CACHE_LINE_SIZE	64

// DLL export and import definitions
#define DLLEXPORT __declspec(dllexport)
#define DLLIMPORT __declspec(dllimport)

// disable this now as it is annoying for generic platform implementations
#pragma warning(disable : 4100) // unreferenced formal parameter


// Include code analysis features
#if PLATFORM_COMPILER_CLANG
	#include "Clang/ClangPlatformCodeAnalysis.h"
#elif PLATFORM_WINDOWS
	#include "Windows/WindowsPlatformCodeAnalysis.h"
#endif

// Other macros
#ifndef ENABLE_WIN_ALLOC_TRACKING
#define ENABLE_WIN_ALLOC_TRACKING 0
#endif

// If set, ShouldExpectLowIntegrityLevel defaults to true. This affects paths for local settings and storage.
#ifndef WINDOWS_LOWINTEGRITYLEVEL_EXPECT_DEFAULT
#define WINDOWS_LOWINTEGRITYLEVEL_EXPECT_DEFAULT 0
#endif

// If set, the engine will attempt to automatically migrate user data like settings from the default medium integrity ProjectUserDir path to the low integrity one.
// Migration is only attempted if the low integrity ProjectUserDir path is empty, e.g. first run after a game update that sets WINDOWS_LOWINTEGRITYLEVEL_EXPECT_DEFAULT = 1.
#ifndef WINDOWS_LOWINTEGRITYLEVEL_AUTOMIGRATE_USERDATA
#define WINDOWS_LOWINTEGRITYLEVEL_AUTOMIGRATE_USERDATA 0
#endif