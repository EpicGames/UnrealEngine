// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformAffinity.h"

class FWindowsPlatformAffinity : public FGenericPlatformAffinity
{
public:
	static EThreadPriority GetRenderingThreadPriority()
	{
		return TPri_AboveNormal;
	}

	static EThreadPriority GetRHIThreadPriority()
	{
		return TPri_AboveNormal;
	}

	static EThreadPriority GetGameThreadPriority()
	{
		return TPri_AboveNormal;
	}
};

typedef FWindowsPlatformAffinity FPlatformAffinityBase;
