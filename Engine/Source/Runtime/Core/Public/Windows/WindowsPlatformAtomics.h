// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformAtomics.h"
#include "Windows/WindowsSystemIncludes.h"
#include <intrin.h>

#include <atomic>

template <typename T>
class TStdAtomicSizeAssert {
	static_assert(sizeof(T) == sizeof(std::atomic<T>), "Unexpected atomic size");
	static_assert( alignof(T) == alignof(std::atomic<T>), "Unexpected atomic alignment");
};

inline TStdAtomicSizeAssert<int8> Assert8;
inline TStdAtomicSizeAssert<int16> Assert16;
inline TStdAtomicSizeAssert<int32> Assert32;
inline TStdAtomicSizeAssert<int64> Assert64;

/**
 * Windows implementation of the Atomics OS functions
 */
struct FWindowsPlatformAtomics
	: public FGenericPlatformAtomics
{
	static_assert(sizeof(int8)  == sizeof(char)      && alignof(int8)  == alignof(char),      "int8 must be compatible with char");
	static_assert(sizeof(int16) == sizeof(short)     && alignof(int16) == alignof(short),     "int16 must be compatible with short");
	static_assert(sizeof(int32) == sizeof(long)      && alignof(int32) == alignof(long),      "int32 must be compatible with long");
	static_assert(sizeof(int64) == sizeof(long long) && alignof(int64) == alignof(long long), "int64 must be compatible with long long");

	static FORCEINLINE int8 InterlockedIncrement( volatile int8* Value )
	{
		return (int8)::_InterlockedExchangeAdd8((char*)Value, 1) + 1;
	}

	static FORCEINLINE int16 InterlockedIncrement( volatile int16* Value )
	{
		return (int16)::_InterlockedIncrement16((short*)Value);
	}

	static FORCEINLINE int32 InterlockedIncrement( volatile int32* Value )
	{
		return (int32)::_InterlockedIncrement((long*)Value);
	}

	static FORCEINLINE int64 InterlockedIncrement( volatile int64* Value )
	{
		return (int64)::_InterlockedIncrement64((long long*)Value);
	}

	static FORCEINLINE int8 InterlockedDecrement( volatile int8* Value )
	{
		return (int8)::_InterlockedExchangeAdd8((char*)Value, -1) - 1;
	}

	static FORCEINLINE int16 InterlockedDecrement( volatile int16* Value )
	{
		return (int16)::_InterlockedDecrement16((short*)Value);
	}

	static FORCEINLINE int32 InterlockedDecrement( volatile int32* Value )
	{
		return (int32)::_InterlockedDecrement((long*)Value);
	}

	static FORCEINLINE int64 InterlockedDecrement( volatile int64* Value )
	{
		return (int64)::_InterlockedDecrement64((long long*)Value);
	}

	static FORCEINLINE int8 InterlockedAdd( volatile int8* Value, int8 Amount )
	{
		return (int8)::_InterlockedExchangeAdd8((char*)Value, (char)Amount);
	}

	static FORCEINLINE int16 InterlockedAdd( volatile int16* Value, int16 Amount )
	{
		return (int16)::_InterlockedExchangeAdd16((short*)Value, (short)Amount);
	}

	static FORCEINLINE int32 InterlockedAdd( volatile int32* Value, int32 Amount )
	{
		return (int32)::_InterlockedExchangeAdd((long*)Value, (long)Amount);
	}

	static FORCEINLINE int64 InterlockedAdd( volatile int64* Value, int64 Amount )
	{
		return (int64)::_InterlockedExchangeAdd64((int64*)Value, (int64)Amount);
	}

	static FORCEINLINE int8 InterlockedExchange( volatile int8* Value, int8 Exchange )
	{
		return (int8)::_InterlockedExchange8((char*)Value, (char)Exchange);
	}

	static FORCEINLINE int16 InterlockedExchange( volatile int16* Value, int16 Exchange )
	{
		return (int16)::_InterlockedExchange16((short*)Value, (short)Exchange);
	}

	static FORCEINLINE int32 InterlockedExchange( volatile int32* Value, int32 Exchange )
	{
		return (int32)::_InterlockedExchange((long*)Value, (long)Exchange);
	}

	static FORCEINLINE int64 InterlockedExchange( volatile int64* Value, int64 Exchange )
	{
		return (int64)::_InterlockedExchange64((long long*)Value, (long long)Exchange);
	}

	static FORCEINLINE void* InterlockedExchangePtr( void*volatile* Dest, void* Exchange )
	{
		#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) 
			if (IsAligned(Dest, alignof(void*)) == false)
			{
				HandleAtomicsFailure(TEXT("InterlockedExchangePointer requires Dest pointer to be aligned to %d bytes"), (int)alignof(void*));
			}
		#endif

		return ::_InterlockedExchangePointer(Dest, Exchange);
	}

	static FORCEINLINE int8 InterlockedCompareExchange( volatile int8* Dest, int8 Exchange, int8 Comparand )
	{
		return (int8)::_InterlockedCompareExchange8((char*)Dest, (char)Exchange, (char)Comparand);
	}

	static FORCEINLINE int16 InterlockedCompareExchange( volatile int16* Dest, int16 Exchange, int16 Comparand )
	{
		return (int16)::_InterlockedCompareExchange16((short*)Dest, (short)Exchange, (short)Comparand);
	}

	static FORCEINLINE int32 InterlockedCompareExchange( volatile int32* Dest, int32 Exchange, int32 Comparand )
	{
		return (int32)::_InterlockedCompareExchange((long*)Dest, (long)Exchange, (long)Comparand);
	}

	static FORCEINLINE int64 InterlockedCompareExchange( volatile int64* Dest, int64 Exchange, int64 Comparand )
	{
		#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (IsAligned(Dest, alignof(int64)) == false)
			{
				HandleAtomicsFailure(TEXT("InterlockedCompareExchange int64 requires Dest pointer to be aligned to %d bytes"), (int)alignof(int64));
			}
		#endif

		return (int64)::_InterlockedCompareExchange64(Dest, Exchange, Comparand);
	}

	static FORCEINLINE int8 InterlockedAnd(volatile int8* Value, const int8 AndValue)
	{
		return (int8)::_InterlockedAnd8((volatile char*)Value, (char)AndValue);
	}

	static FORCEINLINE int16 InterlockedAnd(volatile int16* Value, const int16 AndValue)
	{
		return (int16)::_InterlockedAnd16((volatile short*)Value, (short)AndValue);
	}

	static FORCEINLINE int32 InterlockedAnd(volatile int32* Value, const int32 AndValue)
	{
		return (int32)::_InterlockedAnd((volatile long*)Value, (long)AndValue);
	}

	static FORCEINLINE int64 InterlockedAnd(volatile int64* Value, const int64 AndValue)
	{
		return (int64)::_InterlockedAnd64((volatile long long*)Value, (long long)AndValue);
	}

	static FORCEINLINE int8 InterlockedOr(volatile int8* Value, const int8 OrValue)
	{
		return (int8)::_InterlockedOr8((volatile char*)Value, (char)OrValue);
	}

	static FORCEINLINE int16 InterlockedOr(volatile int16* Value, const int16 OrValue)
	{
		return (int16)::_InterlockedOr16((volatile short*)Value, (short)OrValue);
	}

	static FORCEINLINE int32 InterlockedOr(volatile int32* Value, const int32 OrValue)
	{
		return (int32)::_InterlockedOr((volatile long*)Value, (long)OrValue);
	}

	static FORCEINLINE int64 InterlockedOr(volatile int64* Value, const int64 OrValue)
	{
		return (int64)::_InterlockedOr64((volatile long long*)Value, (long long)OrValue);
	}

	static FORCEINLINE int8 InterlockedXor(volatile int8* Value, const int8 XorValue)
	{
		return (int8)::_InterlockedXor8((volatile char*)Value, (char)XorValue);
	}

	static FORCEINLINE int16 InterlockedXor(volatile int16* Value, const int16 XorValue)
	{
		return (int16)::_InterlockedXor16((volatile short*)Value, (short)XorValue);
	}

	static FORCEINLINE int32 InterlockedXor(volatile int32* Value, const int32 XorValue)
	{
		return (int32)::_InterlockedXor((volatile long*)Value, (int32)XorValue);
	}

	static FORCEINLINE int64 InterlockedXor(volatile int64* Value, const int64 XorValue)
	{
		return (int64)::_InterlockedXor64((volatile long long*)Value, (long long)XorValue);
	}

	static FORCEINLINE int8 AtomicRead(volatile const int8* Src)
	{
		return ((std::atomic<int8>*)(Src))->load();
	}

	static FORCEINLINE int16 AtomicRead(volatile const int16* Src)
	{
		return ((std::atomic<int16>*)(Src))->load();
	}

	static FORCEINLINE int32 AtomicRead(volatile const int32* Src)
	{
		return ((std::atomic<int32>*)(Src))->load();
	}

	static FORCEINLINE int64 AtomicRead(volatile const int64* Src)
	{
		return ((std::atomic<int64>*)(Src))->load();
	}

	static FORCEINLINE int8 AtomicRead_Relaxed(volatile const int8* Src)
	{
		return ((std::atomic<int8>*)(Src))->load(std::memory_order_relaxed);
	}

	static FORCEINLINE int16 AtomicRead_Relaxed(volatile const int16* Src)
	{
		return ((std::atomic<int16>*)(Src))->load(std::memory_order_relaxed);
	}

	static FORCEINLINE int32 AtomicRead_Relaxed(volatile const int32* Src)
	{
		return ((std::atomic<int32>*)(Src))->load(std::memory_order_relaxed);
	}

	static FORCEINLINE int64 AtomicRead_Relaxed(volatile const int64* Src)
	{
		return ((std::atomic<int64>*)(Src))->load(std::memory_order_relaxed);
	}

	static FORCEINLINE void AtomicStore(volatile int8* Src, int8 Val)
	{
		((std::atomic<int8>*)(Src))->store(Val);
	}

	static FORCEINLINE void AtomicStore(volatile int16* Src, int16 Val)
	{
		((std::atomic<int16>*)(Src))->store(Val);
	}

	static FORCEINLINE void AtomicStore(volatile int32* Src, int32 Val)
	{
		((std::atomic<int32>*)(Src))->store(Val);
	}

	static FORCEINLINE void AtomicStore(volatile int64* Src, int64 Val)
	{
		((std::atomic<int64>*)(Src))->store(Val);
	}

	static FORCEINLINE void AtomicStore_Relaxed(volatile int8* Src, int8 Val)
	{
		((std::atomic<int8>*)(Src))->store(Val, std::memory_order_relaxed);
	}

	static FORCEINLINE void AtomicStore_Relaxed(volatile int16* Src, int16 Val)
	{
		((std::atomic<int16>*)(Src))->store(Val, std::memory_order_relaxed);
	}

	static FORCEINLINE void AtomicStore_Relaxed(volatile int32* Src, int32 Val)
	{
		((std::atomic<int32>*)(Src))->store(Val, std::memory_order_relaxed);
	}

	static FORCEINLINE void AtomicStore_Relaxed(volatile int64* Src, int64 Val)
	{
		((std::atomic<int64>*)(Src))->store(Val, std::memory_order_relaxed);
	}

	/**
	 *	The function compares the Destination value with the Comparand value:
	 *		- If the Destination value is equal to the Comparand value, the Exchange value is stored in the address specified by Destination, 
	 *		- Otherwise, the initial value of the Destination parameter is stored in the address specified specified by Comparand.
	 *	
	 *	@return true if Comparand equals the original value of the Destination parameter, or false if Comparand does not equal the original value of the Destination parameter.
	 *	
	 *	Early AMD64 processors lacked the CMPXCHG16B instruction.
	 *	To determine whether the processor supports this operation, call the IsProcessorFeaturePresent function with PF_COMPARE64_EXCHANGE128.
	 */
#if	PLATFORM_HAS_128BIT_ATOMICS
	static FORCEINLINE bool InterlockedCompareExchange128( volatile FInt128* Dest, const FInt128& Exchange, FInt128* Comparand )
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (IsAligned(Dest,16) == false)
		{
			HandleAtomicsFailure(TEXT("InterlockedCompareExchange128 requires Dest pointer to be aligned to 16 bytes") );
		}
		if (IsAligned(Comparand,16) == false)
		{
			HandleAtomicsFailure(TEXT("InterlockedCompareExchange128 requires Comparand pointer to be aligned to 16 bytes") );
		}
#endif

		return ::_InterlockedCompareExchange128((int64 volatile *)Dest, Exchange.High, Exchange.Low, (int64*)Comparand) == 1;
	}
	/**
	* Atomic read of 128 bit value with a memory barrier
	*/
	static FORCEINLINE void AtomicRead128(const volatile FInt128* Src, FInt128* OutResult)
	{
		FInt128 Zero;
		Zero.High = 0;
		Zero.Low = 0;
		*OutResult = Zero;
		InterlockedCompareExchange128((FInt128*)Src, Zero, OutResult);
	}

#endif // PLATFORM_HAS_128BIT_ATOMICS

	static FORCEINLINE void* InterlockedCompareExchangePointer( void*volatile* Dest, void* Exchange, void* Comparand )
	{
		#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (IsAligned(Dest, alignof(void*)) == false)
			{
				HandleAtomicsFailure(TEXT("InterlockedCompareExchangePointer requires Dest pointer to be aligned to %d bytes"), (int)alignof(void*));
			}
		#endif

		return ::_InterlockedCompareExchangePointer(Dest, Exchange, Comparand);
	}

	/**
	* @return true, if the processor we are running on can execute compare and exchange 128-bit operation.
	* @see cmpxchg16b, early AMD64 processors don't support this operation.
	*/
	static FORCEINLINE bool CanUseCompareExchange128()
	{
		return !!Windows::IsProcessorFeaturePresent( WINDOWS_PF_COMPARE_EXCHANGE128 );
	}

protected:
	/**
	 * Handles atomics function failure.
	 *
	 * Since 'check' has not yet been declared here we need to call external function to use it.
	 *
	 * @param InFormat - The string format string.
	 */
	static CORE_API void HandleAtomicsFailure( const TCHAR* InFormat, ... );
};

#ifdef __clang__
#include "Clang/ClangPlatformAtomics.h"

typedef FClangPlatformAtomics FPlatformAtomics;
#else
typedef FWindowsPlatformAtomics FPlatformAtomics;
#endif // __clang__
