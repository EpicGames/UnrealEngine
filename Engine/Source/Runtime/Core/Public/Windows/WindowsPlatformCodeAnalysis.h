// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Not marking as deprecated as eventually we'll need to properly do 
// a per platform CodeAnalysis header, but for now just reroute to generic 
// Microsoft header which includes windows specific elements
#include "Microsoft/MicrosoftPlatformCodeAnalysis.h"