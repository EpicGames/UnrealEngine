// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Cause circular dependencies for header units

#include "Microsoft/MSVCPlatformCompilerPreSetup.h"

// This is needed when compiling with header units. By default _CRT_MEMCPY_S_INLINE is "static inline"
// "static inline" means that function is private to module which causes compile errors when other windows includes uses memcpy_s and friends
#ifndef _CRT_MEMCPY_S_INLINE
	#define _CRT_MEMCPY_S_INLINE inline
#endif
