// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#if !PLATFORM_WINDOWS
#error PLATFORM_WINDOWS not defined
#endif

//
// If LLVM Clang is used, we require at minimum v18 to compile
//
#if defined(__clang__)
static_assert(__clang_major__ >= 18, "LLVM Clang v18 or greater is required to build Unreal Engine.");
#endif

//
// We require at minimum Visual Studio 2022 v17.8 to compile
//
static_assert(_MSC_VER >= 1938, "Visual Studio 2022 v17.8 (MSVC 14.38) or greater is required to build Unreal Engine.");

//
// Manually enable all warnings as errors, except ones that are explicitly skipped.
// Warnings that we explicitly disable later are still included in there.
//

#pragma warning (error:      4001 4002 4003           4006 4007 4008      4010           4013      4015           4018 4019 4020      4022 4023 4024 4025 4026 4027 4028 4029 4030 4031 4032 4033 4034 4035 4036      4038           4041 4042           4045      4047 4048 4049      4051 4052 4053 4054 4055 4056 4057           4060                4064 4065 4066 4067 4068 4069                4073 4074 4075 4076 4077      4079 4080 4081      4083      4085 4086 4087 4088 4089 4090 4091 4092      4094      4096 4097 4098 4099)
#pragma warning (error: 4100 4101 4102 4103                          4109           4112 4113 4114 4115 4116 4117      4119 4120 4121 4122 4123 4124 4125      4127      4129 4130 4131 4132 4133                4137 4138           4141 4142 4143 4144 4145 4146                4150      4152 4153 4154 4155 4156 4157 4158 4159 4160 4161 4162 4163 4164      4166 4167 4168                4172      4174 4175 4176 4177 4178 4179 4180 4181 4182 4183      4185 4186 4187 4188 4189 4190 4191 4192 4193 4194 4195 4196 4197      4199)
#pragma warning (error: 4200 4201 4202 4203 4204 4205 4206 4207 4208      4210 4211 4212 4213 4214 4215 4216      4218      4220 4221      4223 4224      4226 4227 4228 4229 4230      4232 4233 4234 4235      4237 4238 4239 4240           4243      4245                     4250 4251                4255 4256      4258                     4263 4264           4267 4268 4269           4272 4273 4274 4275 4276 4277 4278 4279 4280 4281 4282 4283      4285 4286 4287 4288 4289 4290 4291      4293      4295      4297 4298 4299)
#pragma warning (error:      4301 4302 4303           4306      4308 4309 4310           4313 4314 4315 4316      4318 4319      4321 4322 4323 4324 4325 4326 4327 4328 4329 4330           4333 4334 4335 4336 4337 4338      4340           4343 4344      4346      4348                4352 4353      4355 4356 4357 4358 4359           4362      4364      4366 4367 4368 4369                     4374 4375 4376 4377 4378 4379 4380 4381 4382 4383 4384           4387      4389 4390 4391 4392 4393 4394 4395 4396 4397 4398 4399)
#pragma warning (error: 4400 4401 4402 4403 4404 4405 4406 4407 4408 4409 4410 4411      4413 4414 4415 4416 4417 4418 4419 4420           4423 4424 4425 4426 4427      4429 4430 4431           4434      4436      4438 4439 4440 4441 4442 4443      4445 4446 4447 4448 4449 4450 4451 4452 4453 4454 4455                     4460 4461 4462      4464 4465 4466 4467 4468 4469 4470           4473 4474 4475 4476 4477 4478      4480      4482 4483 4484 4485 4486 4487 4488 4489 4490 4491 4492 4493 4494 4495 4496 4497 4498 4499)
#pragma warning (error:           4502 4503      4505 4506      4508 4509      4511 4512 4513 4514 4515 4516 4517 4518 4519      4521 4522 4523           4526                4530 4531 4532 4533 4534 4535 4536 4537 4538      4540 4541 4542 4543 4544 4545 4546                4550 4551 4552 4553 4554      4556 4557 4558 4559      4561 4562      4564 4565 4566      4568 4569 4570      4572 4573      4575 4576      4578      4580 4581           4584 4585 4586 4587 4588 4589      4591      4593 4594 4595 4596 4597 4598     )
#pragma warning (error: 4600      4602 4603 4604      4606           4609 4610 4611 4612 4613      4615 4616      4618      4620 4621 4622 4623 4624 4625 4626 4627 4628 4629 4630 4631 4632 4633 4634 4635 4636 4637 4638 4639 4640 4641 4642      4644 4645 4646      4648 4649 4650      4652 4653 4654 4655 4656 4657 4658 4659      4661 4662                     4667      4669 4670 4671 4672 4673 4674      4676 4677 4678 4679 4680 4681 4682 4683 4684 4685 4686 4687 4688 4689 4690 4691      4693 4694 4695 4696      4698     )
#pragma warning (error: 4700           4703           4706           4709 4710 4711           4714 4715 4716 4717 4718 4719 4720 4721 4722 4723 4724 4725 4726 4727 4728 4729      4731 4732 4733 4734                     4739 4740 4741 4742 4743 4744 4745      4747      4749 4750 4751 4752 4753 4754 4755 4756 4757                4761           4764                               4771 4772           4775 4776 4777 4778                                    4786      4788 4789           4792 4793 4794                4798 4799)
#pragma warning (error: 4800 4801      4803 4804 4805 4806 4807 4808 4809 4810 4811 4812 4813      4815 4816 4817                4821 4822 4823                4827      4829                     4834 4835                4839 4840 4841 4842 4843 4844 4845 4846 4847 4848 4849                          4855                                                        4867      4869           4872                                    4880 4881 4882 4883                                                                                )
#pragma warning (error: 4900                     4905 4906                4910      4912 4913           4916 4917 4918      4920 4921                4925 4926 4927 4928 4929 4930 4931 4932      4934 4935 4936 4937 4938 4939                     4944 4945 4946 4947 4948 4949 4950 4951 4952 4953 4954 4955 4956 4957 4958 4959 4960 4961      4963 4964 4965 4966                4970 4971 4972 4973 4974                               4981                4985                4989 4990 4991 4992           4995      4997 4998 4999)
#pragma warning (error:                                                                                                                                                                                               5038                                                                                                                                                                                                                                                                                                                 )

//
// Skipped warnings, which are not disabled below. Most are disabled by default. Might be useful to look through, re-enable some and fix the code.
// NOTE: We don't use #pragma warning (default: ####) on these warnings because default enables warnings that default to disabled.
//

// 4005 - 'identifier' : macro redefinition																							https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4005
// 4061 - enumerator 'identifier' in switch of enum 'enumeration' is not explicitly handled by a case label							https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4061
// 4062 - enumerator 'identifier' in switch of enum 'enumeration' is not handled													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4062

// 4165 - 'HRESULT' is being converted to 'bool'; are you sure this is what you want?												https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4165

// 4242 - 'identifier' : conversion from 'type1' to 'type2', possible loss of data													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4242
// 4254 - 'operator' : conversion from 'type1' to 'type2', possible loss of data													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4254
// 4265 - 'class' : class has virtual functions, but destructor is not virtual														https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4265
// 4266 - 'function' : no override available for virtual member function from base 'type'; function is hidden						https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4266
// 4296 - 'operator' : expression is always false																					https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4296

// 4305 - 'identifier' : truncation from 'type1' to 'type2'																			https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4305
// 4307 - '' : integral constant overflow																							https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-2-c4307
// 4311 - 'variable' : pointer truncation from 'type' to 'type'																		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4311
// 4312 - 'operation' : conversion from 'type1' to 'type2' of greater size															https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4312
// 4339 - 'type' : use of undefined type detected in CLR meta-data - use of this type may lead to a runtime exception				https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4339
// 4342 - behavior change: 'function' called, but a member operator was called in previous versions									https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4342
// 4345 - behavior change: an object of POD type constructed with an initializer of the form () will be default-initialized			http://msdn.microsoft.com/en-us/library/wewb47ee.aspx
// 4350 - behavior change: 'member1' called instead of 'member2'																	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4350
// 4365 - 'action' : conversion from 'type_1' to 'type_2', signed/unsigned mismatch													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4365
// 4370 - layout of class has changed from a previous version of the compiler due to better packing									https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4200-through-c4399
// 4371 - layout of class may have changed from a previous version of the compiler due to better packing of member					https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/c4371
// 4373 - 'function': virtual function overrides 'base_function', previous versions of the compiler did not override when parameters only differed by const/volatile qualifiers	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4373
// 4388 - '==' : signed/unsigned mismatch																							https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4200-through-c4399
// 4412 - 'function': function signature contains type 'type'; C++ objects are unsafe to pass between pure code and mixed or native	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-2-c4412
// 4435 - 'class1' : Object layout under /vd2 will change due to virtual base 'class2'												https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4435
// 4437 - dynamic_cast from virtual base 'class1' to 'class2' could fail in some contexts											https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4437
// 4444 - 'identifier': top level '__unaligned' is not implemented in this context													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4400-through-c4599

// NOTE: Shadow variable declaration warnings. These should eventually be fixed up and reenabled.
// 4456 - declaration of 'identifier' hides previous local declaration																https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4456
// 4457 - declaration of 'identifier' hides function parameter																		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4457
// 4458 - declaration of 'identifier' hides class member																			https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4458
// 4459 - declaration of 'identifier' hides global declaration																		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4459

// 4463 - overflow; assigning value to bit-field that can only hold values from low_value to high_value								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4463
// 4471 - a forward declaration of an unscoped enumeration must have an underlying type (int assumed)								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4471
// 4472 - 'identifier' is a native enum: add an access specifier (private/public) to declare a 'WinRT|managed' enum					https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4400-through-c4599
// 4481 - nonstandard extension used: override specifier 'keyword'																	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4481

// 4510 - 'class' : default constructor could not be generated																		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4510
// 4547 - 'operator' : operator before comma has no effect; expected operator with side-effect										https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4547
// 4548 - expression before comma has no effect; expected expression with side-effect												https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4548
// 4549 - 'operator' : operator before comma has no effect; did you intend 'operator'?												https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4549
// 4555 - expression has no effect; expected expression with side-effect															https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4555
// 4571 - Informational: catch(...) semantics changed since Visual C++ 7.1; structured exceptions (SEH) are no longer caught		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4571
// 4574 - 'Identifier' is defined to be '0': did you mean to use '#if identifier'?													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4400-through-c4599
// 4577 - 'noexcept' used with no exception handling mode specified; termination on exception is not guaranteed. Specify /EHsc		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4400-through-c4599

// 4608 - 'union_member' has already been initialized by another union member in the initializer list, 'union_member'				https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4608
// 4619 - #pragma warning : there is no warning number 'number'																		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4619
// 4643 - Forward declaring 'identifier' in namespace std is not permitted by the C++ Standard.										https://docs.microsoft.com/en-us/cpp/cpp-conformance-improvements-2017
// 4647 - behavior change: __is_pod(type) has different value in previous versions													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4600-through-c4799
// 4651 - 'definition' specified for precompiled header but not for current compile													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4651
// 4668 - 'symbol' is not defined as a preprocessor macro, replacing with '0' for 'directives'										https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4668
// 4692 - 'function': signature of non-private member contains assembly private native type 'native_type'							https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4692

// 4701 - Potentially uninitialized local variable 'name' used																		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4701
// 4730 - 'main' : mixing _m64 and floating point expressions may result in incorrect code											https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4730
// 4738 - storing 32-bit float result in memory, possible loss of performance														https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4738
// 4767 - section name '%s' is longer than 8 characters and will be truncated by the linker											https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4600-through-c4799
// 4770 - partially validated enum 'name' used as index																				https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4600-through-c4799
// 4774 - 'string' : format string expected in argument number is not a string literal												https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4600-through-c4799

// 4819 - The file contains a character that cannot be represented in the current code page (number). Save the file in Unicode format to prevent data loss.	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4819
// 4820 - 'bytes' bytes padding added after construct 'member_name'																	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4820
// 4826 - Conversion from 'type1' to 'type2' is sign-extended. This may cause unexpected runtime behavior.							https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4800-through-c4999
// 4828 - The file contains a character starting at offset ... that is illegal in the current source character set (codepage ...).	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4800-through-c4999
// 4837 - trigraph detected: '??character' replaced by 'character'																	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4800-through-c4999
// 4868 - file(line_number)' compiler may not enforce left-to-right evaluation order in braced initializer list                     https://msdn.microsoft.com/en-us/library/mt656772.aspx

// 4962 - 'function' : Profile-guided optimizations disabled because optimizations caused profile data to become inconsistent		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-c4962
// 4986 - 'function': exception specification does not match previous declaration													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-c4986
// 4987 - nonstandard extension used: 'throw (...)'																					https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4800-through-c4999
// 4988 - 'variable': variable declared outside class/function scope																https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4800-through-c4999
// 4996 - The compiler encountered a deprecated declaration.																		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4996

// 5045 - Compiler will insert Spectre mitigation for memory load if /Qspectre switch specified										https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/c5045

//
// Disabled warnings
//

#pragma warning(disable: 4063) // case 'val' is not a valid value for switch of enum 'E'

// Unwanted VC++ level 4 warnings to disable.
#pragma warning(disable: 4100) // 'identifier' : unreferenced formal parameter														https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4100
#pragma warning(disable: 4121) // 'symbol' : alignment of a member was sensitive to packing											https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4121
#pragma warning(disable: 4127) // Conditional expression is constant																https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4127
#pragma warning(disable: 4180) // qualifier applied to function type has no meaning; ignored										https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4180
#pragma warning(disable: 4189) // 'identifier': local variable is initialized but not referenced									https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4000-through-c4199

#pragma warning(disable: 4200) // Zero-length array item at end of structure, a VC-specific extension								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-levels-2-and-4-c4200
#pragma warning(disable: 4201) // nonstandard extension used : nameless struct/union												https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4201
#pragma warning(disable: 4217) // 'operator' : member template functions cannot be used for copy-assignment or copy-construction	// No docs
#pragma warning(disable: 4245) // 'initializing': conversion from 'type' to 'type', signed/unsigned mismatch						https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4245
#pragma warning(disable: 4251) // 'type' needs to have dll-interface to be used by clients of 'type'								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4251
#pragma warning(disable: 4267) // 'var' : conversion from 'size_t' to 'type', possible loss of data									https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4267
#pragma warning(disable: 4275) // non - DLL-interface classkey 'identifier' used as base for DLL-interface classkey 'identifier'	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-2-c4275
#pragma warning(disable: 4291) // no matching operator delete found; memory will not be freed if initialization throws an exception https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4291

#pragma warning(disable: 4307) // '': integral constant overflow																	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-2-c4307
#pragma warning(disable: 4315) // 'classname': 'this' pointer for member 'member' may not be aligned 'alignment' as expected by the constructor	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4200-through-c4399
#pragma warning(disable: 4316) // 'identifier': object allocated on the heap may not be aligned 'alignment'							https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4200-through-c4399
#pragma warning(disable: 4324) // structure was padded due to __declspec(align())													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4324
#pragma warning(disable: 4347) // behavior change: 'function template' is called instead of 'function								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4200-through-c4399
#pragma warning(disable: 4351) // new behavior: elements of array 'array' will be default initialized								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4200-through-c4399
#pragma warning(disable: 4355) // this used in base initializer list																https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-c4355
#pragma warning(disable: 4366) // The result of the unary 'operator' operator may be unaligned										https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4366
#pragma warning(disable: 4373) // '%$S': virtual function overrides '%$pS', previous versions of the compiler did not override when parameters only differed by const/volatile qualifiers	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4373
#pragma warning(disable: 4389) // signed/unsigned mismatch																			https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-c4355

#pragma warning(disable: 4464) // relative include path contains '..'																https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/c4464
#pragma warning(disable: 4482) // nonstandard extension used: enum 'enumeration' used in qualified name								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4400-through-c4599

#pragma warning(disable: 4505) // 'function' : unreferenced local function has been removed											https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4505
#pragma warning(disable: 4511) // 'class' : copy constructor could not be generated													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4511
#pragma warning(disable: 4512) // 'class' : assignment operator could not be generated												https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4512
#pragma warning(disable: 4514) // 'function' : unreferenced inline function has been removed										https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4514

#pragma warning(disable: 4592) // 'function': 'constexpr' call evaluation failed; function will be called at run-time				https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4400-through-c4599
#pragma warning(disable: 4599) // 'flag path': command line argument number number does not match precompiled header				https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4400-through-c4599

#pragma warning(disable: 4605) // '/Dmacro' specified on current command line, but was not specified when precompiled header was built	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4600-through-c4799
#pragma warning(disable: 4623) // 'derived class' : default constructor was implicitly defined as deleted because a base class default constructor is inaccessible or deleted	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4623
#pragma warning(disable: 4625) // 'derived class' : copy constructor was implicitly defined as deleted because a base class copy constructor is inaccessible or deleted	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4625
#pragma warning(disable: 4626) // 'derived class' : assignment operator was implicitly defined as deleted because a base class assignment operator is inaccessible or deleted	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4626
#pragma warning(disable: 4640) // 'instance' : construction of local static object is not thread-safe								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4640
#pragma warning(disable: 4699) // creating precompiled header																		// No docs

#pragma warning(disable: 4710) // 'function' : function not inlined																	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4710
#pragma warning(disable: 4711) // function selected for automatic inlining															https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4711
#pragma warning(disable: 4714) // function 'function' marked as __forceinline not inlined											https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4714
#pragma warning(disable: 4748) // /GS can not protect parameters and local variables from local buffer overrun because optimizations are disabled in function	// No docs
#pragma warning(disable: 4768) // __declspec attributes before linkage specification are ignored									https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4600-through-c4799
// NOTE: _mm_cvtpu8_ps will generate this falsely if it doesn't get inlined
#pragma warning(disable: 4799) // Warning: function 'ident' has no EMMS instruction													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4799

// NOTE: https://answers.unrealengine.com/questions/701635/warning-c4828.html
#pragma warning(disable: 4828) // The file contains a character starting at offset ... that is illegal in the current source character set(codepage ...).	// No docs
#pragma warning(disable: 4868) // 'file(line_number)' compiler may not enforce left-to-right evaluation order in braced initializer list	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-c4868

// NOTE: ocid.h breaks this
#pragma warning(disable: 4917) // 'declarator' : a GUID can only be associated with a class, interface or namespace 				https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4917

//
// All of the /Wall warnings that we are able to enable
// @todo: https://docs.microsoft.com/en-us/cpp/preprocessor/compiler-warnings-that-are-off-by-default
// NOTE: This is currently just overriding the error versions above, removing these will cause them to be errors!
//

#pragma warning(default: 4191) // 'operator/operation': unsafe conversion from 'type_of_expression' to 'type_required'				https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warnings-c4000-through-c4199

#pragma warning(default: 4255) // 'function' : no function prototype given: converting '()' to '(void)'								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4255
#pragma warning(default: 4263) // 'function' : member function does not override any base class virtual member function				https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4263
#pragma warning(default: 4264) // 'virtual_function' : no override available for virtual member function from base 'class'; function is hidden	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4264
#pragma warning(3:       4265) // 'class' : class has virtual functions, but destructor is not virtual								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4265
#pragma warning(default: 4287) // 'operator' : unsigned/negative constant mismatch													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4264
#pragma warning(default: 4289) // nonstandard extension used : 'var' : loop control variable declared in the for-loop is used outside the for-loop scope	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4289

//#pragma warning(disable : 4339) // 'type' : use of undefined type detected in CLR meta-data - use of this type may lead to a runtime exception
#pragma warning(disable: 4345) // behavior change: an object of POD type constructed with an initializer of the form () will be default-initialized

#pragma warning(disable: 4514) // unreferenced inline/local function has been removed
#pragma warning(default: 4529) // 'member_name' : forming a pointer-to-member requires explicit use of the address-of operator ('&') and a qualified name	// No docs
#pragma warning(default: 4536) // 'type name' : type-name exceeds meta-data limit of 'limit' characters								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4536
#pragma warning(default: 4545) // expression before comma evaluates to a function which is missing an argument list					https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4545
#pragma warning(default: 4546) // function call before comma missing argument list													https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4546
#pragma warning(default: 4557) // '__assume' contains effect 'effect'																https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4557
#pragma warning(disable: 4577) // 'noexcept' used with no exception handling mode specified; termination on exception is not guaranteed. Specify /EHsc

#pragma warning(default: 4628) // digraphs not supported with -Ze. Character sequence 'digraph' not interpreted as alternate token for 'char'	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4289
#pragma warning(default: 4682) // 'parameter' : no directional parameter attribute specified, defaulting to [in]					https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4682
#pragma warning(default: 4686) // 'user-defined type' : possible change in behavior, change in UDT return calling convention		https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4686

#pragma warning(disable: 4710) // 'function' : function not inlined / The given function was selected for inline expansion, but the compiler did not perform the inlining.
#pragma warning(disable: 4746) // volatile access of '<expression>' is subject to /volatile											https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-c4746?view=msvc-170
#pragma warning(default: 4786) // 'identifier' : identifier was truncated to 'number' characters in the debug information			// No docs
#pragma warning(default: 4793) // 'function' : function is compiled as native code: 'reason'										https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-and-3-c4793

#pragma warning(default: 4905) // wide string literal cast to 'LPSTR'																https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4905
#pragma warning(default: 4906) // string literal cast to 'LPWSTR'																	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4906
#pragma warning(default: 4928) // illegal copy-initialization; more than one user-defined conversion has been implicitly applied	https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4928
#pragma warning(default: 4931) // we are assuming the type library was built for number-bit pointers								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4931
#pragma warning(disable: 4946) // reinterpret_cast used between related classes: 'class1' and 'class2'								https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4946
#pragma warning(default: 5038) // data member 'A::y' will be initialized after data member 'A::x'									https://docs.microsoft.com/en-us/cpp/error-messages/compiler-warnings/c5038
#pragma warning(disable: 4984) // 'if constexpr' is a C++17 language extension
#pragma warning(disable : 5105) // macro expansion producing 'defined' has undefined behavior

// #pragma warning(default: 4996) // Deprecation: controlled via command line

// interesting ones to turn on and off at times
//#pragma warning(disable : 4266) // '' : no override available for virtual member function from base ''; function is hidden
//#pragma warning(disable : 4296) // 'operator' : expression is always true / false
//#pragma warning(disable : 4820) // 'bytes' bytes padding added after member 'member'
// Mixing MMX/SSE intrinsics will cause this warning, even when it's done correctly.
//#pragma warning(disable : 4730) //mixing _m64 and floating point expressions may result in incorrect code

// Make sure characters are signed
#ifdef _CHAR_UNSIGNED
#error "Bad VC++ option: Characters must be signed" //lint !e309 suppress as lint doesn't have this defined
#endif


