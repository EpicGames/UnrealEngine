// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Microsoft/MicrosoftPlatformCrashContext.h"


struct FWindowsPlatformCrashContext : public FMicrosoftPlatformCrashContext
{
	FWindowsPlatformCrashContext(ECrashContextType InType, const TCHAR* InErrorMessage)
		: FMicrosoftPlatformCrashContext(InType, InErrorMessage)
	{
	}

	
	CORE_API virtual void AddPlatformSpecificProperties() const override;
	CORE_API virtual void CopyPlatformSpecificFiles(const TCHAR* OutputDirectory, void* Context) override;

	// Windows crash contexts support "optional attachments", which are
	// extra files copied into a subdirectory within the main crash report folder.
	// Optional attachments are specifically distinguished from other attachments
	// as they will not be uploaded by the crash report client by default.
	CORE_API static void AddOptionalAttachment(const FString& OptionalAttachmentFilepath);
	
protected:
	void CopyOptionalAttachments(const TCHAR* BaseOutputDirectory) const;
	void WriteOptionalAttachmentsXML(const FString& Filepath) const;
};

typedef FWindowsPlatformCrashContext FPlatformCrashContext;

