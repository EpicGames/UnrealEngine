// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !PLATFORM_WINDOWS
	#error this code should only be included on Windows
#endif

#include "HAL/Platform.h"

#if PLATFORM_CPU_X86_FAMILY
	#include <intrin.h>
	#include <smmintrin.h>
#endif

#include "Microsoft/MicrosoftPlatformMath.h"

typedef FMicrosoftPlatformMathBase FWindowsPlatformMath;
typedef FWindowsPlatformMath FPlatformMath;
