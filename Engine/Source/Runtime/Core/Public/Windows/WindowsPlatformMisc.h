// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformMemory.h"
#include "GenericPlatform/GenericPlatformMisc.h"

#define UE_DEBUG_BREAK_IMPL() PLATFORM_BREAK()

class GenericApplication;
struct FGuid;
class IPlatformChunkInstall;

/** Helper struct used to get the string version of the Windows version. */
struct FWindowsOSVersionHelper
{
	enum ErrorCodes
	{
		SUCCEEDED = 0,
		ERROR_UNKNOWNVERSION = 1,
		ERROR_GETPRODUCTINFO_FAILED = 2,
		ERROR_GETVERSIONEX_FAILED = 4,
		ERROR_GETWINDOWSGT62VERSIONS_FAILED = 8,
	};

	static CORE_API int32 GetOSVersions( FString& OutOSVersion, FString& OutOSSubVersion );
};

/**
  * Determines the concurrency model to be set for a thread.
  * 
  * @see FWindowsPlatformMisc::CoInitialize
  */
enum class ECOMModel : uint8
{
	Singlethreaded = 0,		///< Single-Threaded Apartment (STA)
	Multithreaded,			///< Multi-Threaded Apartment (MTA)
};

/**
 * Type of storage device
 */
enum class EStorageDeviceType : uint8
{
	/** Drive type cannot be determined */
	Unknown = 0,
	/** Drive is a hard disk, may or may not have a cache. */
	HDD = 1,
	/** Drive is a Solid State disk, typically with faster IO and constant latency. */
	SSD = 2,
	/** Drive is an NVMe . */
	NVMe = 3,
	/** Drive is a hybrid SSD/HDD */
	Hybrid = 4,

	Other = 0xff
};

CORE_API const TCHAR* LexToString(EStorageDeviceType StorageType);

/**
 * Storage drive information
 */
struct FPlatformDriveStats
{
	/** Drive name, usually C or D */
	TCHAR DriveName;
	/** Total number of used bytes on the drive, determined during PlatformInit. This information can be refreshed using FWindowsPlatformMisc::UpdateDriveFreeSpace(); */
	uint64 UsedBytes;
	/** Total number of free bytes on the drive, determined during PlatformInit. This information can be refreshed using FWindowsPlatformMisc::UpdateDriveFreeSpace(); */
	uint64 FreeBytes;
	/** Type of underlying hardware. */
	EStorageDeviceType DriveType;
};

enum class ETDRLevel
{
	/** TDR was disabled explicitly */
	Disabled = 0,

	/** Don't attempt to recover from a GPU or driver timeout and display a bugcheck screen instead */
	BugCheck,

	/** Not implemented */
	RecoverVGA,

	/** Recover adapter/driver on timeout (Default) */
	Recover,

	Invalid
};

CORE_API const TCHAR* LexToString(ETDRLevel TDRLevel);


/**
 *	Information about Windows' Timeout Detection and Recovery (TDR) mechanism for detecting display adapter hangs
 */
struct FTDRInfo
{
	static constexpr uint32 DefaultDelay = 2;
	static constexpr uint32 DefaultDdiDelay = 5;

	ETDRLevel Level;
	uint32 Delay;		// GPU timeout threshold in seconds after which TDR will kick in
	uint32 DdiDelay;	// Graphics driver timeout threshold in seconds after which TDR will kick in
};

/**
* Windows implementation of the misc OS functions
**/
struct FWindowsPlatformMisc
	: public FGenericPlatformMisc
{
	static CORE_API void PlatformPreInit();
	static CORE_API void PlatformInit();
	static CORE_API void PlatformTearDown();
	static CORE_API void SetGracefulTerminationHandler();
	static CORE_API void CallGracefulTerminationHandler();
	static CORE_API ECrashHandlingType GetCrashHandlingType();
	static CORE_API ECrashHandlingType SetCrashHandlingType(ECrashHandlingType);
	static CORE_API int32 GetMaxPathLength();
	static CORE_API int32 GetExternalAppMaxPathLength();

	static CORE_API FString GetEnvironmentVariable(const TCHAR* VariableName);
	static CORE_API void SetEnvironmentVar(const TCHAR* VariableName, const TCHAR* Value);

	static CORE_API TArray<uint8> GetMacAddress();
	static CORE_API void SubmitErrorReport( const TCHAR* InErrorHist, EErrorReportMode::Type InMode );

#if !UE_BUILD_SHIPPING
	static CORE_API bool IsDebuggerPresent();
	static CORE_API EProcessDiagnosticFlags GetProcessDiagnostics();
#endif

	inline static void MemoryBarrier() 
	{
#if PLATFORM_CPU_X86_FAMILY
		_mm_sfence();
#elif PLATFORM_CPU_ARM_FAMILY
		__dmb(_ARM64_BARRIER_SY);
#endif
	}

	static CORE_API void AsymmetricThreadFenceHeavy();

	UE_FORCEINLINE_HINT static void AsymmetricThreadFenceLight()
	{
		_ReadWriteBarrier();
	}

	static CORE_API bool IsRemoteSession();

	static CORE_API void SetUTF8Output();
	static CORE_API void LocalPrint(const TCHAR *Message);

	static bool IsLocalPrintThreadSafe()
	{ 
		//returning true when the debugger is attached is to allow
		//printing of log lines immediately to the output window.
		//return false in not attached because OutputDebuString is slow.
		return IsDebuggerPresent();
	}
	
	static CORE_API void RequestExitWithStatus(bool Force, uint8 ReturnCode, const TCHAR* CallSite = nullptr);
	static CORE_API void RequestExit(bool Force, const TCHAR* CallSite = nullptr);
	static CORE_API const TCHAR* GetSystemErrorMessage(TCHAR* OutBuffer, int32 BufferCount, int32 Error);
	static CORE_API void CreateGuid(struct FGuid& Result);
	static CORE_API EAppReturnType::Type MessageBoxExt( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption );
	static CORE_API bool CommandLineCommands();
	static CORE_API bool IsValidAbsolutePathFormat(const FString& Path);
	static CORE_API int32 NumberOfCores();
	static CORE_API const FProcessorGroupDesc& GetProcessorGroupDesc();
	static CORE_API int32 NumberOfCoresIncludingHyperthreads();
	static CORE_API int32 NumberOfWorkerThreadsToSpawn();

	static CORE_API const TCHAR* GetPlatformFeaturesModuleName();

	static CORE_API FString GetDefaultLanguage();
	static CORE_API FString GetDefaultLocale();

	static CORE_API uint32 GetLastError();
	static CORE_API void SetLastError(uint32 ErrorCode);
	static CORE_API void RaiseException( uint32 ExceptionCode );
	static CORE_API bool SetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, const FString& InValue);
	static CORE_API bool GetStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName, FString& OutValue);
	static CORE_API bool DeleteStoredValue(const FString& InStoreId, const FString& InSectionName, const FString& InKeyName);
	static CORE_API bool DeleteStoredSection(const FString& InStoreId, const FString& InSectionName);

	static CORE_API bool CoInitialize(ECOMModel Model = ECOMModel::Singlethreaded);
	static CORE_API void CoUninitialize();

	/**
	 * Has the OS execute a command and path pair (such as launch a browser)
	 *
	 * @param ComandType OS hint as to the type of command 
	 * @param Command the command to execute
	 * @param CommandLine the commands to pass to the executable
	 *
	 * @return whether the command was successful or not
	 */
	static CORE_API bool OsExecute(const TCHAR* CommandType, const TCHAR* Command, const TCHAR* CommandLine = NULL);

	/**
	 * Attempts to get the handle to a top-level window of the specified process.
	 *
	 * If the process has a single main window (root), its handle will be returned.
	 * If the process has multiple top-level windows, the first one found is returned.
	 *
	 * @param ProcessId The identifier of the process to get the window for.
	 * @return Window handle, or 0 if not found.
	 */
	static CORE_API Windows::HWND GetTopLevelWindowHandle(uint32 ProcessId);

	/** 
	 * Determines if we are running on the Windows version or newer
	 *
	 * See the 'Remarks' section of https://msdn.microsoft.com/en-us/library/windows/desktop/ms724833(v=vs.85).aspx
	 * for a list of MajorVersion/MinorVersion version combinations for Microsoft Windows.
	 *
	 * @return	Returns true if the current Windows version if equal or newer than MajorVersion
	 */
	static CORE_API bool VerifyWindowsVersion(uint32 MajorVersion, uint32 MinorVersion, uint32 BuildNumber = 0);

	/** 
	 * Determines if we are running under Wine rather than a real version of Windows
	 *
	 * @return	Returns true if the current runtime environment is Wine
	 */
	static CORE_API bool IsWine();

#if !UE_BUILD_SHIPPING
	static CORE_API void PromptForRemoteDebugging(bool bIsEnsure);
#endif	//#if !UE_BUILD_SHIPPING


	/** 
	 * Determines if the cpuid instruction is supported on this processor
	 *
	 * @return	Returns true if cpuid is supported
	 */
	static CORE_API bool HasCPUIDInstruction();

	static CORE_API bool HasTPM2Support();
	static CORE_API bool HasSecureBootSupport();
	/**
	 * Checks whether this device has the required hardware and software capabilities required to run Windows 11.
	 * @return True if the Windows 11 min spec is met.
	 */
	static CORE_API bool SupportsWindows11();

	static CORE_API FString GetCPUVendor();
	static CORE_API FString GetCPUBrand();
	static CORE_API uint32 GetCPUModelID();
	static CORE_API uint32 GetCPUMicrocodeRevision();

	static CORE_API FString GetPrimaryGPUBrand();
	static CORE_API struct FGPUDriverInfo GetGPUDriverInfo(const FString& DeviceDescription, bool bVerbose = true);
	static CORE_API struct FTDRInfo GetTDRInfo();
	static CORE_API void GetOSVersions( FString& out_OSVersionLabel, FString& out_OSSubVersionLabel );
	static CORE_API FString GetOSVersion();
	static CORE_API bool GetDiskTotalAndFreeSpace( const FString& InPath, uint64& TotalNumberOfBytes, uint64& NumberOfFreeBytes );
	static CORE_API bool GetPageFaultStats(FPageFaultStats& OutStats, EPageFaultFlags Flags=EPageFaultFlags::All);
	static CORE_API bool GetBlockingIOStats(FProcessIOStats& OutStats, EInputOutputFlags Flags=EInputOutputFlags::All);

	/**
	 * Uses cpuid instruction to get the vendor string
	 *
	 * @return	CPU info bitfield
	 *
	 *			Bits 0-3	Stepping ID
	 *			Bits 4-7	Model
	 *			Bits 8-11	Family
	 *			Bits 12-13	Processor type (Intel) / Reserved (AMD)
	 *			Bits 14-15	Reserved
	 *			Bits 16-19	Extended model
	 *			Bits 20-27	Extended family
	 *			Bits 28-31	Reserved
	 */
	static CORE_API uint32 GetCPUInfo();

	/** @return whether this cpu has timed pause instruction support or not */
	static CORE_API bool HasTimedPauseCPUFeature();

	/** @return architecture of the host machine (x64, arm64) **/
	static CORE_API FString GetHostArchitecture();

	/** 
	 * Provides a simpler interface for fetching and cleanup of registry value queries
	 *
	 * @param	InKey		The Key (folder) in the registry to search under
	 * @param	InSubKey	The Sub Key (folder) within the main Key to look for
	 * @param	InValueName	The Name of the Value (file) withing the Sub Key to look for
	 * @param	OutData		The Data entered into the Value
	 *
	 * @return	true, if it successfully found the Value
	 */
	static CORE_API bool QueryRegKey( const Windows::HKEY InKey, const TCHAR* InSubKey, const TCHAR* InValueName, FString& OutData );

	/**
	 * Gets Visual Studio common tools path.
	 *
	 * @param Version Version of VS to get (11 - 2012, 12 - 2013).
	 * @param OutData Output parameter with common tools path.
	 *
	 * @return Returns if succeeded.
	 */
	static CORE_API bool GetVSComnTools(int32 Version, FString& OutData);

	/**
	* @return Windows path separator.
	*/
	static CORE_API const TCHAR* GetDefaultPathSeparator();

	/** @return Get the name of the platform specific file manager (Explorer) */
	static CORE_API FText GetFileManagerName();

	/**
	* Returns whether WiFi connection is currently active
	*/
	static bool HasActiveWiFiConnection()
	{
		// for now return true
		return true;
	}

	/**
	 * Returns the current network connection type using NLM (Network List Manager).
	 */
	static CORE_API ENetworkConnectionType GetNetworkConnectionType();

	/**
	 * Starts event-driven network connection monitoring via INetworkListManagerEvents.
	 * Called from PlatformInit(). Updates network connection status on connectivity changes.
	 */
	static CORE_API void StartNetworkConnectionMonitoring();

	/**
	 * Stops network connection monitoring and releases NLM COM resources.
	 * Called from PlatformTearDown().
	 */
	static CORE_API void StopNetworkConnectionMonitoring();

	/**
	 * Returns whether the device is considered a gaming handheld device. It's not a dynamic property, the status is cached at startup.
	 */
	static CORE_API bool IsDeviceGamingHandheld();
	
	/**
	 * Returns whether the platform is running on battery power or not. For Windows laptops this is true regardless of charging state.
	 */
	static CORE_API bool IsRunningOnBattery();

	/**
	 * Returns the current battery level between 0 and 100, or -1 if a battery is not present or if the current battery level cannot be retrieved.
	 */
	static CORE_API int GetBatteryLevel();

	/**
	 * Returns whether the platform is currently running with battery saver enabled.
	 */
	static CORE_API bool IsUsingBatterySaver();

	inline static void ChooseHDRDeviceAndColorGamut(uint32 DeviceId, uint32 DisplayNitLevel, EDisplayOutputFormat& OutputDevice, EDisplayColorGamut& ColorGamut)
	{
		// needs to match GRHIHDRDisplayOutputFormat chosen in FD3D12DynamicRHI::Init
#if WITH_EDITOR
		// ScRGB, 1000 or 2000 nits
		OutputDevice = (DisplayNitLevel == 1000) ? EDisplayOutputFormat::HDR_ACES_1000nit_ScRGB : EDisplayOutputFormat::HDR_ACES_2000nit_ScRGB;
		// Rec709
		ColorGamut = EDisplayColorGamut::sRGB_D65;
#else
		// ST-2084, 1000 or 2000 nits
		OutputDevice = (DisplayNitLevel == 1000) ? EDisplayOutputFormat::HDR_ACES_1000nit_ST2084 : EDisplayOutputFormat::HDR_ACES_2000nit_ST2084;
		// Rec2020
		ColorGamut = EDisplayColorGamut::Rec2020_D65;
#endif
	}

	/**
	 * Gets a globally unique ID the represents a particular operating system install.
	 */
	static CORE_API FString GetOperatingSystemId();

	static CORE_API EConvertibleLaptopMode GetConvertibleLaptopMode();

	static CORE_API IPlatformChunkInstall* GetPlatformChunkInstall();

	static CORE_API void PumpMessagesOutsideMainLoop();

	static CORE_API uint64 GetFileVersion(const FString &FileName);

	static CORE_API int32 GetMaxRefreshRate();

	/** 
	 * Update statistics of free/used bytes on all drives.
	 * NOTE: This call will block (if GCollectStorageInformationAsync is enabled) until stats are available
	 */
	static CORE_API void UpdateDriveFreeSpace();

	/**
	 * Retrieve information about a drive, or nullptr if no information is available.
	 * NOTE: This call will block (if GCollectStorageInformationAsync is enabled) until stats are available
	 */
	static CORE_API const FPlatformDriveStats* GetDriveStats(WIDECHAR DriveLetter);

	static CORE_API void ApplyWindowsCompatConfigRules(void);
	static CORE_API const class TMap<FString, FString>& GetConfigRuleVars();
};


#if defined(WINDOWS_USE_FEATURE_PLATFORMMISC_CLASS) && WINDOWS_USE_FEATURE_PLATFORMMISC_CLASS
typedef FWindowsPlatformMisc FPlatformMisc;
#endif
