// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Timespan.h"

class FString;

namespace UE
{

/**
 * A mutex that supports recursive locking.
 *
 * Prefer FRecursiveMutex.
 */
class FWindowsRecursiveMutex final
{
public:
	FWindowsRecursiveMutex(const FWindowsRecursiveMutex&) = delete;
	FWindowsRecursiveMutex& operator=(const FWindowsRecursiveMutex&) = delete;

	inline FWindowsRecursiveMutex()
	{
		CA_SUPPRESS(28125);
		Windows::InitializeCriticalSectionAndSpinCount(&CriticalSection, 4000);
	}

	UE_FORCEINLINE_HINT ~FWindowsRecursiveMutex()
	{
		Windows::DeleteCriticalSection(&CriticalSection);
	}

	UE_FORCEINLINE_HINT bool TryLock()
	{
		return !!Windows::TryEnterCriticalSection(&CriticalSection);
	}

	UE_FORCEINLINE_HINT void Lock()
	{
		Windows::EnterCriticalSection(&CriticalSection);
	}

	UE_FORCEINLINE_HINT void Unlock()
	{
		Windows::LeaveCriticalSection(&CriticalSection);
	}

private:
	Windows::CRITICAL_SECTION CriticalSection;
};

/**
 * A shared (read/write) mutex that does not support recursive locking.
 *
 * Prefer FSharedMutex.
 *
 * SRWLOCK: https://learn.microsoft.com/en-us/windows/win32/sync/slim-reader-writer--srw--locks
 */
class FWindowsSharedMutex final
{
public:
	FWindowsSharedMutex(const FWindowsSharedMutex&) = delete;
	FWindowsSharedMutex& operator=(const FWindowsSharedMutex&) = delete;

	FWindowsSharedMutex() = default;

	~FWindowsSharedMutex()
	{
		checkf(!IsLocked(), TEXT("Destroying a lock that is still held!"));
	}

	UE_FORCEINLINE_HINT bool TryLock()
	{
		return !!Windows::TryAcquireSRWLockExclusive(&Mutex);
	}

	UE_FORCEINLINE_HINT void Lock()
	{
		Windows::AcquireSRWLockExclusive(&Mutex);
	}

	UE_FORCEINLINE_HINT void Unlock()
	{
		Windows::ReleaseSRWLockExclusive(&Mutex);
	}

	UE_FORCEINLINE_HINT bool TryLockShared()
	{
		return !!Windows::TryAcquireSRWLockShared(&Mutex);
	}

	UE_FORCEINLINE_HINT void LockShared()
	{
		Windows::AcquireSRWLockShared(&Mutex);
	}

	UE_FORCEINLINE_HINT void UnlockShared()
	{
		Windows::ReleaseSRWLockShared(&Mutex);
	}

private:
	bool IsLocked()
	{
		if (Windows::TryAcquireSRWLockExclusive(&Mutex))
		{
			Windows::ReleaseSRWLockExclusive(&Mutex);
			return false;
		}
		return true;
	}

	Windows::SRWLOCK Mutex = { 0 }; // SRWLOCK_INIT
};

/** A system-wide mutex for Windows. Uses a named mutex. */
class FWindowsSystemWideMutex final
{
public:
	FWindowsSystemWideMutex(const FWindowsSystemWideMutex&) = delete;
	FWindowsSystemWideMutex& operator=(const FWindowsSystemWideMutex&) = delete;

	/** Construct a named, system-wide mutex and attempt to get access/ownership of it. */
	CORE_API explicit FWindowsSystemWideMutex(const FString& InName, FTimespan InTimeout = FTimespan::Zero());

	/** Destructor releases system-wide mutex if it is currently owned. */
	CORE_API ~FWindowsSystemWideMutex();

	/**
	 * Does the calling thread have ownership of the system-wide mutex?
	 *
	 * @return True if obtained. WARNING: Returns true for an owned but previously abandoned locks so shared resources can be in undetermined states. You must handle shared data robustly.
	 */
	CORE_API bool IsValid() const;

	/** Releases system-wide mutex if it is currently owned. */
	CORE_API void Release();

private:
	Windows::HANDLE Mutex;
};

using FPlatformRecursiveMutex = FWindowsRecursiveMutex;
using FPlatformSharedMutex = FWindowsSharedMutex;
using FPlatformSystemWideMutex = FWindowsSystemWideMutex;

} // UE
