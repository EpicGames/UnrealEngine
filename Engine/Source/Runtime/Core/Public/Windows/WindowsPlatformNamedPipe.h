// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformNamedPipe.h"
#include "Windows/WindowsSystemIncludes.h"


#if PLATFORM_SUPPORTS_NAMED_PIPES

class FString;


// Windows wrapper for named pipe communications
class FWindowsPlatformNamedPipe
	: public FGenericPlatformNamedPipe
{
public:

	CORE_API FWindowsPlatformNamedPipe();
	CORE_API virtual ~FWindowsPlatformNamedPipe();

	FWindowsPlatformNamedPipe(const FWindowsPlatformNamedPipe&) = delete;
	FWindowsPlatformNamedPipe& operator=(const FWindowsPlatformNamedPipe&) = delete;

public:

	// FGenericPlatformNamedPipe overrides

	CORE_API virtual bool Create(const FString& PipeName, bool bAsServer, bool bAsync) override;
	CORE_API virtual bool Destroy() override;
	CORE_API virtual bool OpenConnection() override;
	CORE_API virtual bool BlockForAsyncIO() override;
	CORE_API virtual bool IsReadyForRW() const override;
	CORE_API virtual bool UpdateAsyncStatus() override;
	CORE_API virtual bool WriteBytes(int32 NumBytes, const void* Data) override;
	CORE_API virtual bool ReadBytes(int32 NumBytes, void* OutData) override;
	CORE_API virtual bool IsCreated() const override;
	CORE_API virtual bool HasFailed() const override;

private:

	void LogInternals(const char* Context) const;

	void*		Pipe;
	Windows::OVERLAPPED	Overlapped;
	bool		bUseOverlapped : 1;
	bool		bIsServer : 1;

	enum EState
	{
		State_Uninitialized,
		State_Created,
		State_Connecting,
		State_ReadyForRW,
		State_WaitingForRW,

		State_ErrorPipeClosedUnexpectedly,
	};

	EState State;

	CORE_API bool UpdateAsyncStatusAfterRW();
};


typedef FWindowsPlatformNamedPipe FPlatformNamedPipe;

#endif	// PLATFORM_SUPPORTS_NAMED_PIPES
