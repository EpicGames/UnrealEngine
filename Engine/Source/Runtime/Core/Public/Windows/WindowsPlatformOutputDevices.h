// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformOutputDevices.h"

class FOutputDevice;
class FOutputDeviceConsole;
class FOutputDeviceError;
class FFeedbackContext;

struct FWindowsPlatformOutputDevices
	: public FGenericPlatformOutputDevices
{
	static CORE_API FOutputDevice*			GetEventLog();
	static CORE_API FOutputDeviceError*      GetError();
};


typedef FWindowsPlatformOutputDevices FPlatformOutputDevices;
