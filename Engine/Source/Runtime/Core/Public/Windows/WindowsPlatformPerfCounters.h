// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FWindowsPlatformPerfCounters
{
public:
	static CORE_API void Init();
	static CORE_API void Shutdown();
};

typedef FWindowsPlatformPerfCounters FPlatformPerfCounters;