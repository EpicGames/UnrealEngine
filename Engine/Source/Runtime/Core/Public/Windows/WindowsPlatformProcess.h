// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Windows/WindowsSystemIncludes.h"

class FEvent;
class FRunnableThread;

DECLARE_DELEGATE_OneParam(FLaunchURLOverride, const TCHAR*)

/** Windows implementation of the process handle. */
struct FProcHandle : public TProcHandle<Windows::HANDLE, nullptr>
{
public:
	/** Default constructor. */
	UE_FORCEINLINE_HINT constexpr FProcHandle()
		: TProcHandle()
	{}

	/** Initialization constructor. */
	UE_FORCEINLINE_HINT explicit FProcHandle( HandleType Other )
		: TProcHandle( Other )
	{}
};


/**
* Windows implementation of the Process OS functions.
**/
struct FWindowsPlatformProcess
	: public FGenericPlatformProcess
{
	/**
	 * Windows representation of a interprocess semaphore
	 */
	struct FWindowsSemaphore : public FSemaphore
	{
		virtual void	Lock();
		virtual bool	TryLock(uint64 NanosecondsToWait);
		virtual void	Unlock();

		/** Returns the OS handle */
		Windows::HANDLE GetSemaphore() { return Semaphore; }

		/** Constructor */
		FWindowsSemaphore(const FString& InName, Windows::HANDLE InSemaphore);

		/** Allocation free constructor */
		FWindowsSemaphore(const TCHAR* InName, Windows::HANDLE InSemaphore);

		/** Destructor */
		virtual ~FWindowsSemaphore();

	protected:

		/** OS handle */
		Windows::HANDLE Semaphore;
	};

	struct FProcEnumInfo;

	/**
	 * Process enumerator.
	 */
	class FProcEnumerator
	{
	public:
		// Constructor
		CORE_API FProcEnumerator();
		FProcEnumerator(const FProcEnumerator&) = delete;
		FProcEnumerator& operator=(const FProcEnumerator&) = delete;

		// Destructor
		CORE_API ~FProcEnumerator();

		// Gets current process enumerator info.
		CORE_API FProcEnumInfo GetCurrent() const;

		/**
		 * Moves current to the next process.
		 *
		 * @returns True if succeeded. False otherwise.
		 */
		CORE_API bool MoveNext();
	private:
		// Process info structure for current process.
		Windows::PROCESSENTRY32* CurrentEntry;

		// Processes state snapshot handle.
		Windows::HANDLE SnapshotHandle;
	};

	/**
	 * Process enumeration info structure.
	 */
	struct FProcEnumInfo
	{
		friend FProcEnumInfo FProcEnumerator::GetCurrent() const;
	public:
		// Destructor
		CORE_API ~FProcEnumInfo();

		// Gets process PID.
		CORE_API uint32 GetPID() const;

		// Gets parent process PID.
		CORE_API uint32 GetParentPID() const;

		// Gets process name. I.e. exec name.
		CORE_API FString GetName() const;

		// Gets process full image path. I.e. full path of the exec file.
		CORE_API FString GetFullPath() const;

	private:
		// Private constructor.
		FProcEnumInfo(const Windows::PROCESSENTRY32& InInfo);

		// Process info structure.
		Windows::PROCESSENTRY32* Info;
	};

public:

	static CORE_API bool SetProcessAffinity(uint32 NumCoresForAffinity, bool bPhysicalCoresOnly);
	static CORE_API bool IsProcessAffinitySet();

	// FGenericPlatformProcess interface

	static CORE_API void* GetDllHandle( const TCHAR* Filename );
	static CORE_API void FreeDllHandle( void* DllHandle );
	static CORE_API void* GetDllExport( void* DllHandle, const TCHAR* ProcName );
	static CORE_API void AddDllDirectory(const TCHAR* Directory);
	static CORE_API void GetDllDirectories(TArray<FString>& OutDllDirectories);
	static CORE_API void PushDllDirectory(const TCHAR* Directory);
	static CORE_API void PopDllDirectory(const TCHAR* Directory);
	static CORE_API uint32 GetCurrentProcessId();
	static CORE_API uint32 GetCurrentCoreNumber();
	static CORE_API void SetThreadAffinityMask( uint64 AffinityMask );
	static CORE_API void SetThreadName( const TCHAR* ThreadName );
	static CORE_API void SetThreadPriority(EThreadPriority NewPriority);
	static CORE_API const TCHAR* BaseDir();
	static CORE_API const TCHAR* UserDir();
	static CORE_API const TCHAR* UserTempDir();
	static CORE_API const TCHAR* UserSettingsDir();
	static CORE_API const TCHAR* UserHomeDir();
	static CORE_API const TCHAR* UserSettingsDirMediumIntegrity();
	static CORE_API const TCHAR* ApplicationSettingsDir();
	static CORE_API FString GetApplicationSettingsDir(const ApplicationSettingsContext& Settings);
	static CORE_API const TCHAR* WindowsSystemDir();
	static CORE_API const TCHAR* ComputerName();
	static CORE_API const TCHAR* UserName(bool bOnlyAlphaNumeric = true);
	static CORE_API void SetCurrentWorkingDirectoryToBaseDir();
#if WITH_LOW_LEVEL_TESTS
	/** Test-only: overwrite GBaseDirectory so we can force paths that would normally fail. Not thread safe. */
	static CORE_API void SetBaseDirForTesting(const TCHAR* InBaseDir);
#endif
	static CORE_API FString GetCurrentWorkingDirectory();
	static CORE_API const FString ShaderWorkingDir();
	static CORE_API const TCHAR* ExecutablePath();
	static CORE_API const TCHAR* ExecutableName(bool bRemoveExtension = true);
	static CORE_API FString GenerateApplicationPath( const FString& AppName, EBuildConfiguration BuildConfiguration);
	static CORE_API const TCHAR* GetArchitectureSuffix();
	static CORE_API const TCHAR* GetModuleExtension();
	static CORE_API const TCHAR* GetBinariesSubdirectory();
	static CORE_API const FString GetModulesDirectory();
	static CORE_API FProcHandle CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild = nullptr);
	static CORE_API FProcHandle CreateProc(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild);
	static CORE_API TTuple<FProcHandle, UE::HAL::EProcessId> CreateProc(const UE::HAL::FProcessStartInfo& StartInfo);
	static CORE_API bool SetProcPriority(FProcHandle & InProcHandle, int32 PriorityModifier);
	static CORE_API FProcHandle OpenProcess(uint32 ProcessID);
	static CORE_API bool IsProcRunning( FProcHandle & ProcessHandle );
	static CORE_API void WaitForProc( FProcHandle & ProcessHandle );
	static CORE_API void CloseProc( FProcHandle & ProcessHandle );
	static CORE_API void TerminateProc( FProcHandle & ProcessHandle, bool KillTree = false );
	static CORE_API void TerminateProcTreeWithPredicate(
			FProcHandle& ProcessHandle,
			TFunctionRef<bool(uint32 ProcessId, const TCHAR* ApplicationName)> Predicate);
	static CORE_API bool GetProcReturnCode( FProcHandle & ProcHandle, int32* ReturnCode );
	static CORE_API bool GetApplicationMemoryUsage(uint32 ProcessId, SIZE_T* OutMemoryUsage);
	static CORE_API bool GetPerFrameProcessorUsage(uint32 ProcessId, float& ProcessUsageFraction, float& IdleUsageFraction);
	static CORE_API bool IsApplicationRunning( uint32 ProcessId );
	static CORE_API bool IsApplicationRunning( const TCHAR* ProcName );
	static CORE_API FString GetApplicationName( uint32 ProcessId );	
	static CORE_API bool ExecProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode, FString* OutStdOut, FString* OutStdErr, const TCHAR* OptionalWorkingDirectory = NULL, bool bShouldEndWithParentProcess  = false);
	static CORE_API bool ExecElevatedProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode);
	static CORE_API FProcHandle CreateElevatedProcess(const TCHAR* URL, const TCHAR* Params);
	static CORE_API bool LaunchFileInDefaultExternalApplication( const TCHAR* FileName, const TCHAR* Parms = NULL, ELaunchVerb::Type Verb = ELaunchVerb::Open, bool bPromptToOpenOnFailure = true );
	static CORE_API void ExploreFolder( const TCHAR* FilePath );
	static CORE_API bool ResolveNetworkPath( FString InUNCPath, FString& OutPath ); 
	static CORE_API void Sleep(float Seconds);
	static CORE_API void SleepNoStats(float Seconds);
	[[noreturn]] static CORE_API void SleepInfinite();
	static CORE_API void YieldThread();
	UE_DEPRECATED(5.0, "Please use GetSynchEventFromPool to create a new event, and ReturnSynchEventToPool to release the event.")
	static CORE_API class FEvent* CreateSynchEvent(bool bIsManualReset = false);
	static CORE_API class FRunnableThread* CreateRunnableThread();
	static CORE_API void ClosePipe( void* ReadPipe, void* WritePipe );
	static CORE_API bool CreatePipe(void*& ReadPipe, void*& WritePipe, bool bWritePipeLocal = false);
	static CORE_API FString ReadPipe( void* ReadPipe );
	static CORE_API bool ReadPipeToArray(void* ReadPipe, TArray<uint8> & Output);
	static CORE_API bool WritePipe(void* WritePipe, const FString& Message, FString* OutWritten = nullptr);
	static CORE_API bool WritePipe(void* WritePipe, const uint8* Data, const int32 DataLength, int32* OutDataLength = nullptr);
	static CORE_API FSemaphore* NewInterprocessSynchObject(const FString& Name, bool bCreate, uint32 MaxLocks = 1);
	static CORE_API FSemaphore* NewInterprocessSynchObject(const TCHAR* Name, bool bCreate, uint32 MaxLocks = 1);
	static CORE_API bool DeleteInterprocessSynchObject(FSemaphore * Object);
	static CORE_API bool Daemonize();
	static CORE_API void SetupGameThread();
	static CORE_API void SetupAudioThread();
	static CORE_API void TeardownAudioThread();
	static CORE_API bool IsFirstInstance();
	static CORE_API TSharedPtr<IProcessSentinel> CreateProcessSentinelObject();

	/**
	 * @brief Releases locks that we held for IsFirstInstance check
	 */
	static CORE_API void CeaseBeingFirstInstance();

	static CORE_API bool TryGetMemoryUsage(FProcHandle& ProcessHandle, FPlatformProcessMemoryStats& OutStats);
	/**
	 * Whether to expect to run at a low process integrity level or not. This affects the paths that must be used for user and temp storage.
	 * The process may launch at default (medium) integrity level and then downgrade itself to low later for security benefits, so checking
	 * the current integrity level alone is not sufficient.
	 *
	 * @return true if low integrity level should be expected, otherwise false.
	 */
	static CORE_API bool ShouldExpectLowIntegrityLevel();

	/*
	* Allows us to add an alternative route for launching URLs
	*/
	static CORE_API FLaunchURLOverride LaunchURLOverrideDelegate;
	static CORE_API bool CanLaunchURL(const TCHAR* URL);	
	static CORE_API void LaunchURL(const TCHAR* URL, const TCHAR* Parms, FString* Error);

protected:

	/**
	 * Reads from a collection of anonymous pipes.
	 *
	 * @param OutStrings Will hold the read data.
	 * @param InPipes The pipes to read from.
	 * @param PipeCount The number of pipes.
	 */
	static CORE_API void ReadFromPipes(FString* OutStrings[], Windows::HANDLE InPipes[], int32 PipeCount);

private:

	static FProcHandle CreateProcInternal(const TCHAR* URL, const TCHAR* Parms, bool bLaunchDetached, bool bLaunchHidden, bool bLaunchReallyHidden, bool bBreakawayFromJob, uint32* OutProcessID, int32 PriorityModifier, const TCHAR* OptionalWorkingDirectory, void* PipeWriteChild, void* PipeReadChild, void* PipeStdErrChild, bool bInheritHandles);
	
	static void LaunchURLInternal(const TCHAR* URL, const TCHAR* Parms, FString* Error);

	/**
	 * Since Windows can only have one directory at a time,
	 * this stack is used to reset the previous DllDirectory.
	 */
	static TArray<FString> DllDirectoryStack;

	/**
	 * All the DLL directories we want to load from. 
	 */
	static TArray<FString> DllDirectories;
	
	/**
	 * A cache of the dlls found in each directory in DllDirectories. 
	 */
	static TMap<FName, TArray<FString>> SearchPathDllCache;

	/**
	 * Replacement implementation of the Win32 LoadLibrary function which searches the given list of directories for dependent imports, and attempts
	 * to load them from the correct location first. The normal LoadLibrary function (pre-Win8) only searches a limited list of locations. 
	 *
	 * @param FileName Path to the library to load
	 * @param SearchPaths Search directories to scan for imports
	 */
	static void* LoadLibraryWithSearchPaths(const FString& FileName, TConstArrayView<FString> SearchPaths);

	/**
	 * Resolve an individual import.
	 *
	 * @param ImportName Name of the imported module
	 * @param OutFileName On success, receives the path to the imported file
	 * @return true if an import was found.
	 */
	static bool ResolveImport(const FString& Name, TConstArrayView<FString> SearchPaths, FString& OutFileName);

	/**
	 * Resolve all the imports for the given library, searching through a set of directories.
	 *
	 * @param FileName Path to the library to load
	 * @param SearchPaths Search directories to scan for imports
	 * @param ImportFileNames Array which is filled with a list of the resolved imports found in the given search directories
	 * @param VisitedImportNames Array which stores a list of imports which have been checked
	 */
	static void ResolveMissingImportsRecursive(const FString& FileName, TConstArrayView<FString> SearchPaths, TArray<FString>& ImportFileNames, TSet<FString>& VisitedImportNames);
};

typedef FWindowsPlatformProcess FPlatformProcess;

