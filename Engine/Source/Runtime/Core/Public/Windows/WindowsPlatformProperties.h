// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformProperties.h"


/**
 * Implements Windows platform properties.
 */
template<bool HAS_EDITOR_DATA, bool IS_DEDICATED_SERVER, bool IS_CLIENT_ONLY>
struct FWindowsPlatformProperties
	: public FGenericPlatformProperties
{
	static constexpr UE_FORCEINLINE_HINT bool HasEditorOnlyData()
	{
		return HAS_EDITOR_DATA;
	}

	static constexpr UE_FORCEINLINE_HINT const char* IniPlatformName()
	{
		return "Windows";
	}

	static constexpr UE_FORCEINLINE_HINT const TCHAR* GetRuntimeSettingsClassName()
	{
		return TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings");
	}

	static constexpr UE_FORCEINLINE_HINT bool IsGameOnly()
	{
		return UE_GAME;
	}

	static constexpr UE_FORCEINLINE_HINT bool IsServerOnly()
	{
		return IS_DEDICATED_SERVER;
	}

	static constexpr UE_FORCEINLINE_HINT bool IsClientOnly()
	{
		return IS_CLIENT_ONLY;
	}

	static constexpr inline const char* PlatformName()
	{
		if (IS_DEDICATED_SERVER)
		{
			return "WindowsServer";
		}
		
		if (HAS_EDITOR_DATA)
		{
			return "WindowsEditor";
		}
		
		if (IS_CLIENT_ONLY)
		{
			return "WindowsClient";
		}

		return "Windows";
	}

	static constexpr UE_FORCEINLINE_HINT bool RequiresCookedData()
	{
		return !HAS_EDITOR_DATA;
	}

	static constexpr UE_FORCEINLINE_HINT bool HasSecurePackageFormat()
	{
		return IS_DEDICATED_SERVER;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMemoryMappedFiles()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsAudioStreaming()
	{
		return !IsServerOnly();
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMeshLODStreaming()
	{
		return !IsServerOnly() && !HasEditorOnlyData();
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsRayTracing()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsGrayscaleSRGB()
	{
		return false; // Requires expand from G8 to RGBA
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsMultipleGameInstances()
	{
		return true;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsWindowedMode()
	{
		return true;
	}
	
	static constexpr UE_FORCEINLINE_HINT bool HasFixedResolution()
	{
		return false;
	}

	static constexpr UE_FORCEINLINE_HINT bool SupportsQuit()
	{
		return true;
	}

	static constexpr inline float GetVariantPriority()
	{
		if (IS_DEDICATED_SERVER)
		{
			return 0.0f;
		}

		if (HAS_EDITOR_DATA)
		{
			return 0.0f;
		}

		if (IS_CLIENT_ONLY)
		{
			return 0.0f;
		}

		return 1.0f;
	}

	static constexpr UE_FORCEINLINE_HINT int64 GetMemoryMappingAlignment()
	{
		return 4096;
	}

	static constexpr UE_FORCEINLINE_HINT int GetMaxSupportedVirtualMemoryAlignment()
	{
		return 65536;
	}
};

#ifdef PROPERTY_HEADER_SHOULD_DEFINE_TYPE
typedef FWindowsPlatformProperties<WITH_EDITORONLY_DATA, UE_SERVER, !WITH_SERVER_CODE && !WITH_EDITOR> FPlatformProperties;
#endif
