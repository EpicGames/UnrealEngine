// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Microsoft/MicrosoftPlatformStackWalk.h"

/**
 * Windows implementation of the stack walking.
 **/
struct FWindowsPlatformStackWalk
	: public FMicrosoftPlatformStackWalk
{
	static CORE_API bool InitStackWalking();
	static CORE_API bool InitStackWalkingForProcess(const FProcHandle& Process);
	
	static CORE_API TArray<FProgramCounterSymbolInfo> GetStack(int32 IgnoreCount, int32 MaxDepth = 100, void* Context = nullptr);

	static CORE_API void EnumerateSymbolInfosForProgramCounter(uint64 ProgramCounter, bool bIncludeInlineFrames, TFunctionRef<void(FProgramCounterSymbolInfo& OutSymbolInfo)> Callback);

	static CORE_API void ProgramCounterToSymbolInfo( uint64 ProgramCounter, FProgramCounterSymbolInfo& out_SymbolInfo );
	static CORE_API void ProgramCounterToSymbolInfoEx( uint64 ProgramCounter, FProgramCounterSymbolInfoEx& out_SymbolInfo );
	CORE_API FORCENOINLINE static uint32 CaptureStackBackTrace( uint64* BackTrace, uint32 MaxDepth, void* Context = nullptr );  // FORCENOINLINE so it can be counted during StackTrace
	static CORE_API uint32 CaptureThreadStackBackTrace( uint64 ThreadId, uint64* BackTrace, uint32 MaxDepth, void* Context = nullptr);
	CORE_API FORCENOINLINE static void StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, void* Context = nullptr );  // FORCENOINLINE so it can be counted during StackTrace
	static CORE_API void StackWalkAndDump( ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, void* ProgramCounter, void* Context = nullptr );
	static CORE_API void ThreadStackWalkAndDump(ANSICHAR* HumanReadableString, SIZE_T HumanReadableStringSize, int32 IgnoreCount, uint32 ThreadId);

	static CORE_API int32 GetProcessModuleCount();
	static CORE_API int32 GetProcessModuleSignatures(FStackWalkModuleInfo *ModuleSignatures, const int32 ModuleSignaturesSize);

	static CORE_API void RegisterOnModulesChanged();

	/**
	 * Upload localy built symbols to network symbol storage.
	 *
	 * Use case:
	 *   Game designers use game from source (without prebuild game .dll-files).
	 *   In this case all game .dll-files are compiled locally.
	 *   For post-mortem debug programmers need .dll and .pdb files from designers.
	 */
	static CORE_API bool UploadLocalSymbols();

	/**
	 * Get downstream storage with downloaded from remote symbol storage files.
	 */
	static CORE_API FString GetDownstreamStorage();

	static CORE_API void* MakeThreadContextWrapper(void* Context, void* ThreadHandle);
	static CORE_API void ReleaseThreadContextWrapper(void* ThreadContext);

	/**
	 * Returns the source file pathname, line and column where the specified function is defined.
	 *
	 * The implementation extracts the information from the debug engine and the debug symbols and
	 * takes care of loading the debug symbols if the debug engine was configured to load symbols
	 * on demand. This function can be expensive if the debug symbols needs to be loaded.
	 FMovieSceneEventParameters*
	 * @param FunctionSymbolName The function name to lookup.
	 * @param FunctionModuleName The module name containing the function to lookup.
	 * @param OutPathname The source file pathname.
	 * @param OutLineNumber The line at which the function is defined in the source file.
	 * @param OutColumnNumber The offset on the line at which the function is defined in the source file.
	 * @return True if the the function location is found, false otherwise.
	 */
	static CORE_API bool GetFunctionDefinitionLocation(const FString& FunctionSymbolName, const FString& FunctionModuleName, FString& OutPathname, uint32& OutLineNumber, uint32& OutColumnNumber);

protected:
	static CORE_API void CaptureStackTraceByProcess(uint64* OutBacktrace, uint32 MaxDepth, void* InContext, void* InThreadHandle, uint32* OutDepth, bool bExternalProcess);
private:
	static bool InitStackWalkingInternal(void* Process, bool bForceReinitOnProcessMismatch);
};


typedef FWindowsPlatformStackWalk FPlatformStackWalk;
