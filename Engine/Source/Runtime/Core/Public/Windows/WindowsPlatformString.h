// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Microsoft/MicrosoftPlatformString.h"


/**
 * Windows string implementation.
 */
struct FWindowsPlatformString
	: public FMicrosoftPlatformString
{
	// These should be replaced with equivalent Convert and ConvertedLength functions when we properly support variable-length encodings.
/*	static void WideCharToMultiByte(const wchar_t *Source, uint32 LengthWM1, ANSICHAR *Dest, uint32 LengthA)
	{
		::WideCharToMultiByte(CP_ACP,0,Source,LengthWM1+1,Dest,LengthA,NULL,NULL);
	}
	static void MultiByteToWideChar(const ANSICHAR *Source, TCHAR *Dest, uint32 LengthM1)
	{
		::MultiByteToWideChar(CP_ACP,0,Source,LengthM1+1,Dest,LengthM1+1);
	}*/
};


typedef FWindowsPlatformString FPlatformString;

// Format specifiers to be able to print values of these types correctly, for example when using UE_LOG.

// SIZE_T format specifier
#define SIZE_T_FMT "I64u"
// SIZE_T format specifier for lowercase hexadecimal output
#define SIZE_T_x_FMT "I64x"
// SIZE_T format specifier for uppercase hexadecimal output
#define SIZE_T_X_FMT "I64X"

// SSIZE_T format specifier
#define SSIZE_T_FMT "I64d"
// SSIZE_T format specifier for lowercase hexadecimal output
#define SSIZE_T_x_FMT "I64x"
// SSIZE_T format specifier for uppercase hexadecimal output
#define SSIZE_T_X_FMT "I64X"

// PTRINT format specifier for decimal output
#define PTRINT_FMT "lld"
// PTRINT format specifier for lowercase hexadecimal output
#define PTRINT_x_FMT "llx"
// PTRINT format specifier for uppercase hexadecimal output
#define PTRINT_X_FMT "llX"

// UPTRINT format specifier for decimal output
#define UPTRINT_FMT "llu"
// UPTRINT format specifier for lowercase hexadecimal output
#define UPTRINT_x_FMT "llx"
// UPTRINT format specifier for uppercase hexadecimal output
#define UPTRINT_X_FMT "llX"

// int64 format specifier for decimal output
#define INT64_FMT "lld"
// int64 format specifier for lowercase hexadecimal output
#define INT64_x_FMT "llx"
// int64 format specifier for uppercase hexadecimal output
#define INT64_X_FMT "llX"

// uint64 format specifier for decimal output
#define UINT64_FMT "llu"
// uint64 format specifier for lowercase hexadecimal output
#define UINT64_x_FMT "llx"
// uint64 format specifier for uppercase hexadecimal output
#define UINT64_X_FMT "llX"
