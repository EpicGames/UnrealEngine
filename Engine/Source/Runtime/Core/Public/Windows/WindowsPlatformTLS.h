// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformTLS.h"
#include "Windows/WindowsSystemIncludes.h"

#if WITH_EDITOR

#include "Misc/AssertionMacros.h"

// Custom cross-platform dynamic TLS implementation because we hit Windows TLS slot limit (1088). compile-time defined limits for the maximum
// number of TLS slots and the maximum number of threads that use TLS. No dynamic memory allocation. Lock-free.
// Getting and setting a slot value is fast but has an additional indirection compared with OS TLS.
// Memory footprint is (roughly): MaxSlots * 4B + MaxThreads * 4B + MaxSlots * MaxThreads * 8B

namespace WindowsPlatformTLS_Private
{
	inline constexpr uint32 MaxSlots = WINDOWS_MAX_NUM_TLS_SLOTS; // how many slots are available
	inline constexpr uint32 MaxThreads = WINDOWS_MAX_NUM_THREADS_WITH_TLS_SLOTS; // how many threads can use TLS

	// a single OS TLS slot that is used to store the thread storage
	CORE_API extern uint32 PrimarySlot;

	CORE_API void** AllocThreadStorage();
}

#endif

/**
 * Windows implementation of the TLS OS functions.
 */
struct FWindowsPlatformTLS
	: public FGenericPlatformTLS
{
	/**
	 * Returns the currently executing thread's identifier.
	 *
	 * @return The thread identifier.
	 */
	static UE_FORCEINLINE_HINT uint32 GetCurrentThreadId(void)
	{
		return Windows::GetCurrentThreadId();
	}

	/**
	 * Allocates a thread local store slot.
	 *
	 * @return The index of the allocated slot.
	 */
#if WITH_EDITOR
	CORE_API static uint32 AllocTlsSlot();
#else
	UE_FORCEINLINE_HINT static uint32 AllocTlsSlot()
	{
		return Windows::TlsAlloc();
}

#endif

	/**
	 * Frees a previously allocated TLS slot
	 *
	 * @param SlotIndex the TLS index to store it in
	 */
#if WITH_EDITOR
	CORE_API static void FreeTlsSlot(uint32 SlotIndex);
#else
	UE_FORCEINLINE_HINT static void FreeTlsSlot(uint32 SlotIndex)
	{
		Windows::TlsFree(SlotIndex);
	}
#endif

	/**
	 * Sets a value in the specified TLS slot.
	 *
	 * @param SlotIndex the TLS index to store it in.
	 * @param Value the value to store in the slot.
	 */
	static inline void SetTlsValue(uint32 SlotIndex, void* Value)
	{
#if WITH_EDITOR
		using namespace WindowsPlatformTLS_Private;

		checkf(SlotIndex < MaxSlots, TEXT("Invalid slot index %u"), SlotIndex);

		void** ThreadStorage = (void**)Windows::FlsGetValue(PrimarySlot);
		if (ThreadStorage == 0)
		{
			ThreadStorage = AllocThreadStorage();
		}
		ThreadStorage[SlotIndex] = Value;
#else
		Windows::TlsSetValue(SlotIndex, Value);
#endif
	}

	/**
	 * Reads the value stored at the specified TLS slot.
	 *
	 * @param SlotIndex The index of the slot to read.
	 * @return The value stored in the slot.
	 */
	static inline void* GetTlsValue(uint32 SlotIndex)
	{
#if WITH_EDITOR
		using namespace WindowsPlatformTLS_Private;

		checkf(SlotIndex < MaxSlots, TEXT("Invalid slot index %u"), SlotIndex);
	
		void** ThreadStorage = (void**)Windows::FlsGetValue(PrimarySlot);
		return ThreadStorage ? ThreadStorage[SlotIndex] : nullptr;
#else
		return Windows::TlsGetValue(SlotIndex);
#endif
	}
};


typedef FWindowsPlatformTLS FPlatformTLS;
