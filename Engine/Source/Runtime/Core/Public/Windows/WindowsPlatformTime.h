// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "Windows/WindowsSystemIncludes.h"


/**
 * Windows implementation of the Time OS functions.
 *
 * Please see following UDN post about using rdtsc on processors that support
 * result being invariant across cores.
 *
 * https://udn.epicgames.com/lists/showpost.php?id=46794&list=unprog3
 */
struct FWindowsPlatformTime
	: public FGenericPlatformTime
{
	static CORE_API double InitTiming();

	static inline double Seconds()
	{
		Windows::LARGE_INTEGER Cycles;
		Windows::QueryPerformanceCounter(&Cycles);

		// add big number to make bugs apparent where return value is being passed to float
		return (double)Cycles.QuadPart * GetSecondsPerCycle() + GetSecondsTimeOffset();
	}

	static inline uint32 Cycles()
	{
		Windows::LARGE_INTEGER Cycles;
		Windows::QueryPerformanceCounter(&Cycles);
		return (uint32)Cycles.QuadPart;
	}

	static inline uint64 Cycles64()
	{
		Windows::LARGE_INTEGER Cycles;
		QueryPerformanceCounter(&Cycles);
		return Cycles.QuadPart;
	}

	static inline double GetSecondsTimeOffset()
	{
		return 16777216.0;
	}

	static CORE_API void SystemTime( int32& Year, int32& Month, int32& DayOfWeek, int32& Day, int32& Hour, int32& Min, int32& Sec, int32& MSec );
	static CORE_API void UtcTime( int32& Year, int32& Month, int32& DayOfWeek, int32& Day, int32& Hour, int32& Min, int32& Sec, int32& MSec );

	static CORE_API bool UpdateCPUTime( float DeltaTime );
	static CORE_API bool UpdateThreadCPUTime(float = 0.0);
	static CORE_API void AutoUpdateGameThreadCPUTime(double UpdateInterval);
	static CORE_API FCPUTime GetCPUTime();
	static CORE_API FCPUTime GetThreadCPUTime();
	static CORE_API double GetLastIntervalThreadCPUTimeInSeconds();

protected:

	/** Percentage CPU utilization for the last interval relative to one core. */
	static CORE_API float CPUTimePctRelative;
};


typedef FWindowsPlatformTime FPlatformTime;
