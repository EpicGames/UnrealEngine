// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include "Misc/AssertionMacros.h"
#include "Windows/AllowWindowsPlatformTypes.h"

class FWindowsSemaphore
{
public:
	UE_NONCOPYABLE(FWindowsSemaphore);

	FWindowsSemaphore(int32 InitialCount, int32 MaxCount)
		: Semaphore(CreateSemaphore(nullptr, InitialCount, MaxCount, nullptr))
	{
		checkfSlow(Semaphore, TEXT("CreateSemaphore failed: %u"), GetLastError());
	}

	~FWindowsSemaphore()
	{
		CloseHandle(Semaphore);
	}

	void Acquire()
	{
		DWORD Res = WaitForSingleObject(Semaphore, INFINITE);
		checkfSlow(Res == WAIT_OBJECT_0, TEXT("Acquiring semaphore failed: %d (%u)"), Res, GetLastError());
	}

	bool TryAcquire(FTimespan Timeout = FTimespan::Zero())
	{
		DWORD Res = WaitForSingleObject(Semaphore, (DWORD)Timeout.GetTotalMilliseconds());
		checkfSlow(Res == WAIT_OBJECT_0 || Res == WAIT_TIMEOUT, TEXT("Acquiring semaphore failed: %d (%u)"), Res, GetLastError());
		return Res == WAIT_OBJECT_0;
	}

	void Release(int32 Count = 1)
	{
		checkfSlow(Count > 0, TEXT("Releasing semaphore with Count = %d, that should be greater than 0"), Count);
		bool bRes = ReleaseSemaphore(Semaphore, Count, nullptr);
		checkfSlow(bRes, TEXT("Releasing semaphore for %d failed: %u"), Count, GetLastError());
	}

private:
	HANDLE Semaphore;
};

using FSemaphore = FWindowsSemaphore;

#include "Windows/HideWindowsPlatformTypes.h"
