// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Windows/WindowsPlatformCompilerSetup.h"
#include "Windows/MinimalWindowsApi.h"

// Macro for releasing COM objects
#define SAFE_RELEASE(p) { if(p) { (p)->Release(); (p)=NULL; } }

// Current instance
extern "C" CORE_API Windows::HINSTANCE hInstance;

// SIMD intrinsics
THIRD_PARTY_INCLUDES_START
#include <intrin.h>

#include <stdint.h>
#include "HAL/HideTCHAR.h"
#include <tchar.h>
#include "HAL/AllowTCHAR.h"

// When compiling under Windows, these headers cause us particular problems.  We need to make sure they're included before we pull in our 
// 'DoNotUseOldUE4Type' namespace.  This is because these headers will redeclare various numeric typedefs, but under the Clang and Visual
// Studio compilers it is not allowed to define a typedef with a global scope operator in it (such as ::INT). So we'll get these headers
// included early on to avoid compiler errors with that.
#if defined(__clang__) || (defined(_MSC_VER) && (_MSC_VER >= 1900))
#include <intsafe.h>

// strsafe declares some of the functions as "static inline" which breaks when compiling with header units/modules.. 
// "static inline" means that function is private to module/header unit.. which causes compile error when used in other modules.
// The beautiful solution is to redefine static to inline when including strsafe.h to fix this issue.
// A bug has been filed to fix this but it is likely it will take a while to get a fix (problem exists in latest version which is 10.0.22621.0 today)
#if defined(_MSC_VER) && (_MSC_VER >= 1940)
#define static inline
#endif

#include <strsafe.h>

#undef static
#endif

#if USING_CODE_ANALYSIS
// Source annotation support
#include <CodeAnalysis/SourceAnnotations.h>

// Allows for disabling code analysis warnings by defining ALL_CODE_ANALYSIS_WARNINGS
#include <CodeAnalysis/Warnings.h>

// We import the vc_attributes namespace so we can use annotations more easily.  It only defines
// MSVC-specific attributes so there should never be collisions.
using namespace vc_attributes;
#endif
THIRD_PARTY_INCLUDES_END
