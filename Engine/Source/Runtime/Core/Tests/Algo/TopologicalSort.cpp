// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Algo/TopologicalSort.h"

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FSystemCoreAlgoTopologicalSortTest, "System::Core::Algo::TopologicalSort", "[ApplicationContextMask][EngineFilter]")
{
	using namespace Algo;

	{
		TArray<int32> Array{ 1, 2, 3 };

		// Test the sort when each node depends on the previous one
		bool bHasSucceeded = TopologicalSort(Array, [](int32 Element) { return Element > 1 ? TArray<int32>{ Element - 1 } : TArray<int32>{}; });
		CHECK_MESSAGE(TEXT("TopologicalSort returned true when each node depends on the previous one"), bHasSucceeded == true);
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly when each node depends on the previous one"), (Array == TArray<int32>{1, 2, 3}));
	}
	{
		TArray<int32> Array{ 1, 2, 3 };
		
		// Test the sort when each node depends on the next one
		bool bHasSucceeded = TopologicalSort(Array, [](int32 Element) { return Element < 3 ? TArray<int32>{ Element + 1 } : TArray<int32>{}; });
		CHECK_MESSAGE(TEXT("TopologicalSort returned true when each node depends on the next one"), bHasSucceeded == true);
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly when each node depends on the next one"), (Array == TArray<int32>{3, 2, 1}));
	}
	{
		TArray<int32> Array{ 1, 2 };
		
		// Test the sort with a cycle between 1 and 2
		bool bHasSucceeded = TopologicalSort(Array, [](int32 Element) { return TArray<int32>{ 1 + Element % 2 }; }, ETopologicalSort::None);
		CHECK_MESSAGE(TEXT("TopologicalSort returned false when a cycle is detected and AllowCycles is not specified"), bHasSucceeded == false);
		CHECK_MESSAGE(TEXT("TopologicalSort does not modify the array when a cycle is detected and AllowCycles is not specified"), (Array == TArray<int32>{1, 2}));

		bHasSucceeded = TopologicalSort(Array, [](int32 Element) { return TArray<int32>{ 1 + Element % 2 }; }, ETopologicalSort::AllowCycles);
		CHECK_MESSAGE(TEXT("TopologicalSort returned true when a cycle is detected but AllowCycles is specified"), bHasSucceeded == true);
	}
	{
		TArray<int32> Array;
		for (int32 Index = 0; Index < 1000; ++Index)
		{
			Array.Add(Index);
		}

		// Make sure node 500 makes it on top if every other node depends on it
		bool bHasSucceeded = TopologicalSort(Array, [](int32 Element) { return Element == 500 ? TArray<int32>{} : TArray<int32>{ 500 }; });
		CHECK_MESSAGE(TEXT("TopologicalSort returned true when every node has a dependency on a single node"), bHasSucceeded == true);
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly when every node has a dependency on a single node"), Array[0] == 500);
	}
	{
		TArray<int32> Array{ 1, 2, 3, 4, 5, 6, 7, 8, 9, 10 };
		//              7
		//             / \
		//            6   8
		//           / \   \
		//          1   2   9
 		//           \ / \   \
		//            4   5   |
		//             \   \ /
		//              \   10
		//               \ /
		//                3       
		TMultiMap<int32, int32> Links;
		Links.Add(6, 7);
		Links.Add(1, 6);
		Links.Add(4, 1);
		Links.Add(3, 4);
		Links.Add(3, 10);
		Links.Add(10, 5);
		Links.Add(10, 9);
		Links.Add(9, 8);
		Links.Add(8, 7);
		Links.Add(2, 6);
		Links.Add(5, 2);

		bool bHasSucceeded = TopologicalSort(Array, [&Links](int32 Element) { TArray<int32> Dependencies; Links.MultiFind(Element, Dependencies); return Dependencies; });
		CHECK_MESSAGE(TEXT("TopologicalSort returned true on sketched-out-example-1"), bHasSucceeded == true);

		// There might be multiple valid answers, so test each condition separately to make sure they are all met.
		for (const auto& Pair : Links)
		{
			int32 ChildIndex = Array.IndexOfByKey(Pair.Key);
			int32 ParentIndex = Array.IndexOfByKey(Pair.Value);
			CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly on sketched-out-example-1"), ParentIndex < ChildIndex);
		}
	}

	{
		// Test the sort with a cycle in the root and with the root cycle depending on a chain of non-cycle verts
		TArray<int32> Array{ 1, 2, 3, 4 };
		TMultiMap<int32, int32> Links;
		Links.Add(1, 2);
		Links.Add(2, 1);
		Links.Add(2, 3);
		Links.Add(3, 4);

		bool bHasSucceeded = TopologicalSort(Array,
			[&Links](int32 Element) { TArray<int32> Dependencies; Links.MultiFind(Element, Dependencies); return Dependencies; },
			ETopologicalSort::AllowCycles);
		CHECK_MESSAGE(TEXT("TopologicalSort returned true with a cycle in the root and with the root cycle depending on a chain of non-cycle verts"),
			bHasSucceeded == true);
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly with a cycle in the root and with the root cycle depending on a chain of non-cycle verts"),
			(Array == TArray<int32>{4, 3, 2, 1}) || (Array == TArray<int32>{4, 3, 1, 2}));
	}
	{
		// Test the sort with a cycle in the root and with the root cycle depending on a chain of non-cycle verts, submitted in revese
		TArray<int32> Array{ 1, 2, 3, 4 };
		TMultiMap<int32, int32> Links;
		Links.Add(4, 3);
		Links.Add(3, 4);
		Links.Add(3, 2);
		Links.Add(2, 1);

		bool bHasSucceeded = TopologicalSort(Array,
			[&Links](int32 Element) { TArray<int32> Dependencies; Links.MultiFind(Element, Dependencies); return Dependencies; },
			ETopologicalSort::AllowCycles);

		CHECK_MESSAGE(TEXT("TopologicalSort returned true with a cycle in the root and with the root cycle depending on a chain of non-cycle verts, submitted in revese"),
			bHasSucceeded == true);
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly with a cycle in the root and with the root cycle depending on a chain of non-cycle verts, submitted in revese"),
			(Array == TArray<int32>{1, 2, 3, 4}) || (Array == TArray<int32>{1, 2, 4, 3}));
	}
	{
		// Test the sort with a cycle at a leaf and a chain from the root depending on that cycle
		TArray<int32> Array{ 1, 2, 3, 4 };
		TMultiMap<int32, int32> Links;
		Links.Add(1, 2);
		Links.Add(2, 3);
		Links.Add(3, 4);
		Links.Add(4, 3);

		bool bHasSucceeded = TopologicalSort(Array,
			[&Links](int32 Element) { TArray<int32> Dependencies; Links.MultiFind(Element, Dependencies); return Dependencies; },
			ETopologicalSort::AllowCycles);

		CHECK_MESSAGE(TEXT("TopologicalSort returned true with a cycle at a leaf and a chain from the root depending on that cycle"),
			bHasSucceeded == true);
		// There might be multiple valid answers, so test each condition separately to make sure they are all met.
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly with a cycle at a leaf and a chain from the root depending on that cycle"),
			Array.IndexOfByKey(2) < Array.IndexOfByKey(1));
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly with a cycle at a leaf and a chain from the root depending on that cycle"),
			(Array[0] == 3) || (Array[0] == 4));
	}
	{
		// Test the sort with a cycle in the root and with the root cycle depending on a chain of non-cycle verts, submitted in reverse
		TArray<int32> Array{ 1, 2, 3, 4 };
		TMultiMap<int32, int32> Links;
		Links.Add(4, 3);
		Links.Add(3, 2);
		Links.Add(2, 1);
		Links.Add(1, 2);

		bool bHasSucceeded = TopologicalSort(Array,
			[&Links](int32 Element) { TArray<int32> Dependencies; Links.MultiFind(Element, Dependencies); return Dependencies; },
			ETopologicalSort::AllowCycles);

		CHECK_MESSAGE(TEXT("TopologicalSort returned true with a cycle in the root and with the root cycle depending on a chain of non-cycle verts, submitted in reverse"),
			bHasSucceeded == true);
		// There might be multiple valid answers, so test each condition separately to make sure they are all met.
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly with a cycle in the root and with the root cycle depending on a chain of non-cycle verts, submitted in reverse"),
			Array.IndexOfByKey(3) < Array.IndexOfByKey(4));
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly with a cycle in the root and with the root cycle depending on a chain of non-cycle verts, submitted in reverse"),
			(Array[0] == 1) || (Array[0] == 2));
	}
	{
		// Verify that when breaking a cycle a member of the cycle is selected rather than an element that depends on but is not in the cycle
		TArray<int32> Array{ 1, 2, 3, 4 };
		TMultiMap<int32, int32> Links;
		Links.Add(1, 2);
		// Each of the cycle verts is given two dependencies, to verify that the algorithm is not just picking the vertex in the stack with minimum count
		Links.Add(2, 3);
		Links.Add(2, 4);
		Links.Add(3, 2);
		Links.Add(3, 4);
		Links.Add(4, 2);
		Links.Add(4, 3);

		bool bHasSucceeded = TopologicalSort(Array,
			[&Links](int32 Element) { TArray<int32> Dependencies; Links.MultiFind(Element, Dependencies); return Dependencies; },
			ETopologicalSort::AllowCycles);

		CHECK_MESSAGE(TEXT("TopologicalSort returned true with a vertex dependent upon a cycle"),bHasSucceeded == true);
		// There might be multiple valid answers, so test each condition separately to make sure they are all met.
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly with a vertex dependent upon a cycle"), Array.IndexOfByKey(2) < Array.IndexOfByKey(1));
	}
	{
		// Verify that when two cycles are present and one cycle depends on the other, the independent cycle is taken
		TMultiMap<int32, int32> Links;
		// 1 -> 2 -> 3 -> 4 -> 2
		//           |
		//           v
		//      5 -> 6 -> 7 -> 5
		Links.Add(1, 2);
		Links.Add(2, 3);
		Links.Add(3, 4);
		Links.Add(3, 6);
		Links.Add(4, 2);
		Links.Add(5, 6);
		Links.Add(6, 7);
		Links.Add(7, 5);

		int32 TrialIndex = 0;
		while (TrialIndex >= 0)
		{
			TArray<int32> Array;
			switch (TrialIndex++)
			{
			case 0:
				Array = { 1, 2, 3, 4, 5, 6, 7 };
				break;
			case 1:
				Array = { 7, 6, 5, 4, 3, 2, 1 };
				break;
			default:
				TrialIndex = -1;
				break;
			};
			if (TrialIndex < 0)
			{
				break;
			}

			bool bHasSucceeded = TopologicalSort(Array,
				[&Links](int32 Element) { TArray<int32> Dependencies; Links.MultiFind(Element, Dependencies); return Dependencies; },
				ETopologicalSort::AllowCycles);

			CHECK_MESSAGE(TEXT("TopologicalSort returned true for one cycle dependent upon another"), bHasSucceeded == true);
			// Vertex 1 comes last because it depends on the top cycle which depends on the bottom cycle
			CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly for one cycle dependent upon another"), Array.Last() == 1);
			// Test that all elements of the bottom cycle come before all elements of the top cycle
			int32 LastIndexOfBottom = FMath::Max3(Array.IndexOfByKey(5), Array.IndexOfByKey(6), Array.IndexOfByKey(7));
			int32 FirstIndexOfTop = FMath::Min3(Array.IndexOfByKey(2), Array.IndexOfByKey(3), Array.IndexOfByKey(4));
			CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly for one cycle dependent upon another"), LastIndexOfBottom < FirstIndexOfTop);
		}
	}
	{
		// Verify that when a short cycle is embedded in a larger mutually reachable set, and that set has a
		// dependency, from a vertex outside of the short cycle, to a separate mutually reachable set, that the more
		// independent mutually reachable set is reported as more leaf-ward
		TArray<int32> Array = { 0,1,2,3,4,5 };
		TMultiMap<int32, int32> Links;
		// 5 -> 0 -> (1 -> 2 -> 1)
		//      |          |
		//      |          v
		//      |          5
		//      v
		//     (3 -> 4 -> 3)
		Links.Add(0, 1);
		Links.Add(1, 2);
		Links.Add(2, 1);
		// This edge creates the longer cycle and is later in the edge list from 2 and goes to a higher vertex than
		// the edge that creates the shorter cycle 
		Links.Add(2, 5);
		// This edge that leads to the more independency cycle is later in the edge list from 0 than the edge list
		// to the short cycle
		Links.Add(0, 3);
		Links.Add(3, 4);
		Links.Add(4, 3);
		Links.Add(5, 0);

		bool bHasSucceeded = TopologicalSort(Array,
			[&Links](int32 Element) { TArray<int32> Dependencies; Links.MultiFind(Element, Dependencies); return Dependencies; },
			ETopologicalSort::AllowCycles);

		CHECK_MESSAGE(TEXT("TopologicalSort returned true for MutuallyReachableSet Problem1"), bHasSucceeded == true);
		// 3,4 comes first
		CHECK_MESSAGE(TEXT("TopologicalSort sorted correctly for MutuallyReachableSet Problem1"), (Array[0] == 3 && Array[1] == 4) || (Array[0] == 4 && Array[1] == 3));
		// Any order of 0,1,2,5 is valid since they are mutually reachable, so there's nothing else to test
	}
}

#endif // WITH_TESTS
