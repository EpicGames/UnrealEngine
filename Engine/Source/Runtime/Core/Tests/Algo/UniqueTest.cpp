// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Algo/Unique.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Templates/Tuple.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FUniqueTest, "System::Core::Algo::Unique", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace Algo;

	{
		TArray<int32> Array;
		int32 RemoveFrom = Unique(Array);
		CHECK_MESSAGE(TEXT("`Unique` must handle an empty container"), RemoveFrom == 0);
	}
	{
		TArray<int32> Array{ 1, 2, 3 };
		Array.SetNum(Unique(Array));
		CHECK_MESSAGE(TEXT("Uniqued container with no duplicates must remain unchanged"), (Array == TArray<int32>{1, 2, 3}));
	}
	{
		TArray<int32> Array{ 1, 1, 2, 2, 2, 3, 3, 3, 3 };
		Array.SetNum(Unique(Array));
		CHECK_MESSAGE(TEXT("`Unique` with multiple duplicates must return correct result"), (Array == TArray<int32>{1, 2, 3}));
	}
	{
		TArray<int32> Array{ 1, 1, 2, 3, 3, 3 };
		Array.SetNum(Unique(Array));
		CHECK_MESSAGE(TEXT("`Unique` with duplicates and unique items must return correct result"), (Array == TArray<int32>{1, 2, 3}));
	}
	{
		FString Str = "aa";
		Str = Str.Mid(0, Unique(Str));
		CHECK_MESSAGE(TEXT("`Unique` on `FString` as an example of arbitrary random-access container must compile and return correct result"),
			Str == FString("a"));
	}
	{
		int32 Array[] = { 1 };
		int32 NewSize = (int32)Unique(Array);
		CHECK_MESSAGE(TEXT("`Unique` must support C arrays"), NewSize == 1);
	}
	{
		TArray<int32> Array = { 1, 1 };
		int32 NewSize = Unique(MakeArrayView(Array.GetData() + 1, 1));
		CHECK_MESSAGE(TEXT("`Unique` must support ranges"), NewSize == 1);
	}
}

TEST_CASE_NAMED(FUniqueByTest, "System::Core::Algo::UniqueBy", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace Algo;

	using FIntPair = TPair<int32, int32>;

	{
		TArray<FIntPair> Array;
		int32 RemoveFrom = UniqueBy(Array, &FIntPair::Key);
		CHECK_MESSAGE(TEXT("`UniqueBy` must handle an empty container"), RemoveFrom == 0);
	}
	{
		TArray<FIntPair> Array{ { 1, 0 }, { 2, 0 }, { 3, 0} };
		Array.SetNum(UniqueBy(Array, &FIntPair::Key));
		CHECK_MESSAGE(TEXT("`UniqueBy` with no duplicates must remain unchanged"), (Array == TArray<FIntPair>{ { 1, 0 }, { 2, 0 }, { 3, 0 } }));
	}
	{
		TArray<FIntPair> Array{ { 1, 0 }, { 1, 0 }, { 2, 0}, { 2, 0 }, { 2, 0 }, { 3, 0 }, { 3, 0 }, { 3, 0 }, { 3, 0 }};
		Array.SetNum(UniqueBy(Array, &FIntPair::Key));
		CHECK_MESSAGE(TEXT("`UniqueBy` with multiple duplicates must return correct result"), (Array == TArray<FIntPair>{ { 1, 0 }, { 2, 0 }, { 3, 0 } }));
	}
	{
		TArray<FIntPair> Array{ { 1, 0 }, { 1, 0 }, { 2, 0}, { 3, 0 }, { 3, 0 }, { 3, 0 } };
		Array.SetNum(UniqueBy(Array, &FIntPair::Key));
		CHECK_MESSAGE(TEXT("`UniqueBy` with duplicates and unique items must return correct result"), (Array == TArray<FIntPair>{ { 1, 0 }, { 2, 0 }, { 3, 0 } }));
	}
	{
		FString Str = "aa";
		Str = Str.Mid(0, UniqueBy(Str, FIdentityFunctor()));
		CHECK_MESSAGE(TEXT("`UniqueBy` on `FString` as an example of arbitrary random-access container must compile and return correct result"),
			Str == FString("a"));
	}
	{
		FIntPair Array[] = { { 1, 0 } };
		int32 NewSize = (int32)UniqueBy(Array, &FIntPair::Key);
		CHECK_MESSAGE(TEXT("`UniqueBy` must support C arrays"), NewSize == 1);
	}
	{
		TArray<FIntPair> Array = { { 1, 0 }, { 1, 0 } };
		int32 NewSize = UniqueBy(MakeArrayView(Array.GetData() + 1, 1), &FIntPair::Key);
		CHECK_MESSAGE(TEXT("`UniqueBy` must support ranges"), NewSize == 1);
	}
}

#endif // WITH_TESTS
