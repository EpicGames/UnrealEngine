// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Templates/Function.h"
#include "Tests/TestHarnessAdapter.h"
#include "Async/Async.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

#if WITH_TESTS

/** Helper methods used in the test cases. */
namespace AsyncTestUtils
{
	TFunction<int()> Task = [] {
		return 123;
	};

	bool bHasVoidTaskFinished = false;

	TFunction<void()> VoidTask = [] {
		bHasVoidTaskFinished = true;
	};
}


/** Test that task graph tasks return correctly. */
TEST_CASE_NAMED(FAsyncGraphTest, "System::Core::Async::Async (Task Graph)", "[EditorContext][EngineFilter]")
{
	auto Future = Async(EAsyncExecution::TaskGraph, AsyncTestUtils::Task);
	int Result = Future.Get();

	CHECK_EQUALS(TEXT("Task graph task must return expected value"), Result, 123);
}


/** Test that threaded tasks return correctly. */
TEST_CASE_NAMED(FAsyncThreadedTaskTest, "System::Core::Async::Async (Thread)", "[EditorContext][EngineFilter]")
{
	auto Future = Async(EAsyncExecution::Thread, AsyncTestUtils::Task);
	int Result = Future.Get();

	CHECK_EQUALS(TEXT("Threaded task must return expected value"), Result, 123);
}


/** Test that threaded pool tasks return correctly. */
TEST_CASE_NAMED(FAsyncThreadedPoolTest, "System::Core::Async::Async (Thread Pool)", "[EditorContext][EngineFilter]")
{
	auto Future = Async(EAsyncExecution::ThreadPool, AsyncTestUtils::Task);
	int Result = Future.Get();

	CHECK_EQUALS(TEXT("Thread pool task must return expected value"), Result, 123);
}


/** Test that void tasks run without errors or warnings. */
TEST_CASE_NAMED(FAsyncVoidTaskTest, "System::Core::Async::Async (Void)", "[EditorContext][EngineFilter]")
{
	// Reset test variable before running
	AsyncTestUtils::bHasVoidTaskFinished = false;
	auto Future = Async(EAsyncExecution::TaskGraph, AsyncTestUtils::VoidTask);
	Future.Get();

	// Check that the variable state was updated by task
	CHECK_MESSAGE(TEXT("Void tasks should run"), AsyncTestUtils::bHasVoidTaskFinished);
}


/** Test that asynchronous tasks have their completion callback called. */
TEST_CASE_NAMED(FAsyncCompletionCallbackTest, "System::Core::Async::Async (Completion Callback)", "[EditorContext][EngineFilter]")
{
	bool Completed = false;
	FEvent* CompletedEvent = FPlatformProcess::GetSynchEventFromPool(true);

	auto Future = Async(EAsyncExecution::TaskGraph, AsyncTestUtils::Task, [&] {
		Completed = true;
		CompletedEvent->Trigger();
	});

	int Result = Future.Get();

	// We need an additional synchronization point here since the future get above will return after
	// the task is done but before the completion callback has returned!

	bool CompletedEventTriggered = CompletedEvent->Wait(FTimespan(0 /* hours */, 0 /* minutes */, 5 /* seconds */));
	FPlatformProcess::ReturnSynchEventToPool(CompletedEvent);

	CHECK_EQUALS(TEXT("Async Result"), Result, 123);
	CHECK_MESSAGE(TEXT("Completion callback to be called"), CompletedEventTriggered && Completed);
}

#endif //WITH_TESTS
