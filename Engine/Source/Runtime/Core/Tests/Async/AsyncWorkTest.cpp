// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "Async/AsyncWork.h"
#include "Misc/QueuedThreadPool.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_TESTS

namespace AsyncWorkTestImpl
{
	thread_local bool bIsOuterTaskRunning = false;

	struct FInnerTask : public FNonAbandonableTask
	{
		void DoWork() 
		{ 
			TRACE_CPUPROFILER_EVENT_SCOPE(FInnerTask::DoWork);
			FPlatformProcess::Sleep(0.01f); 
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FInnerTask, STATGROUP_ThreadPoolAsyncTasks);
		}
	};

	struct FOuterTask : public FNonAbandonableTask
	{
		FQueuedThreadPool* ThreadPool = nullptr;
		int32 InnerTaskCount = 0;
		void DoWork() 
		{ 
			TRACE_CPUPROFILER_EVENT_SCOPE(FOuterTask::DoWork);

			bIsOuterTaskRunning = true;
			TArray<FAsyncTask<FInnerTask>> InnerTasks;
			InnerTasks.SetNum(InnerTaskCount);

			for (int32 Index = 0; Index < InnerTasks.Num(); ++Index)
			{
				InnerTasks[Index].StartBackgroundTask(ThreadPool);
			}

			for (int32 Index = 0; Index < InnerTasks.Num(); ++Index)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FOuterTask::EnsureCompletion);
				InnerTasks[Index].EnsureCompletion();
			}
			bIsOuterTaskRunning = false;
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FRootTask, STATGROUP_ThreadPoolAsyncTasks);
		}
	};

	template <typename OtherTaskType>
	struct FCompletionTask : public FNonAbandonableTask
	{
		OtherTaskType* OtherTask;
		void DoWork() 
		{ 
			checkf(!bIsOuterTaskRunning, TEXT("We got picked by the busywait of the outer task we are going to wait on, we are now deadlocked"));

			TRACE_CPUPROFILER_EVENT_SCOPE(FCompletionTask::DoWork);

			// We need to wait a little bit before entering EnsureCompletion otherwise
			// EnsureCompletion will busy wait into every completion task and there 
			// will not be any completion tasks left for the outer task to pick up.
			FPlatformProcess::Sleep(0.01f);

			OtherTask->EnsureCompletion(); 
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FCompletionTask, STATGROUP_ThreadPoolAsyncTasks);
		}
	};
}

TEST_CASE_NAMED(FAsyncWorkEnsureCompletionBusyWaitDeadLockTest, "System::Core::AsyncWork::EnsureCompletionBusyWaitDeadlockTest", "[.][EditorContext][EngineFilter][Disabled]")
{
	using namespace AsyncWorkTestImpl;
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncWorkEnsureCompletionBusyWaitDeadLockTest::RunTest);
	
	FQueuedThreadPool* ThreadPool = GThreadPool;
#if WITH_EDITOR
	ThreadPool = GLargeThreadPool;
#endif

	FAsyncTask<FOuterTask> OuterTask;
	OuterTask.GetTask().ThreadPool = ThreadPool;
	OuterTask.GetTask().InnerTaskCount = ThreadPool->GetNumThreads() * 2 * 100;

	TArray<FAsyncTask<FCompletionTask<FAsyncTask<FOuterTask>>>> CompletionTasks;
	CompletionTasks.SetNum(ThreadPool->GetNumThreads());
	for (int32 Index = 0; Index < CompletionTasks.Num(); ++Index)
	{
		CompletionTasks[Index].GetTask().OtherTask = &OuterTask;
	}

	// Start the task that all the other tasks will try to complete as fast as possible
	OuterTask.StartBackgroundTask(ThreadPool, EQueuedWorkPriority::Normal, EQueuedWorkFlags::None);

	// Wait until the outer task had time to schedule it's own tasks
	FPlatformProcess::Sleep(0.1f);

	// Now fill the thread pool with tasks that simply want to complete the first one
	for (int32 Index = 0; Index < CompletionTasks.Num(); ++Index)
	{
		EQueuedWorkFlags QueuedWorkFlags = EQueuedWorkFlags::None;

		// Busy waiting on our task could cause a circular dependency between tasks on the same thread, which cannot be resolved and results into a deadlock.
		// We need to tell the scheduler we want to opt-out of being picked up by busy wait.
		// Comment this line to repro the deadlock scenario on the low-level task graph (will not reproduce on the old task graph as busywait doesn't exists).
		QueuedWorkFlags |= EQueuedWorkFlags::DoNotRunInsideBusyWait;

		// Use Highest priority so we have a chance to run before all the inner tasks are finished
		CompletionTasks[Index].StartBackgroundTask(ThreadPool, EQueuedWorkPriority::Highest, QueuedWorkFlags);
	}

	// Do not contribute to any work here, we normally tick slate for the game-thread to stay responsive while we're waiting on tasks to complete
	for (int32 Index = 0; Index < CompletionTasks.Num(); ++Index)
	{
		CompletionTasks[Index].EnsureCompletion(false);
	}
	OuterTask.EnsureCompletion(false);

}

#endif //WITH_TESTS
