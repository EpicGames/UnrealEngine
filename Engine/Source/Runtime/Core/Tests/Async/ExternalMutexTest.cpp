// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Async/ExternalMutex.h"

#include "Async/UniqueLock.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "TestHarness.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

namespace UE
{

struct FExternalMutexTestParams
{
	constexpr static uint8 IsLockedFlag = 1;
	constexpr static uint8 MayHaveWaitingLockFlag = 2;
};

TEST_CASE("Core::Async::ExternalMutex", "[Core][Async][Slow][LinuxSkip]")
{
	constexpr static int32 TaskCount = 5;

	SECTION("TExternalMutex IsLocked() and TryLock()")
	{
		FThread Threads[TaskCount];
		std::atomic<uint32> TasksComplete = 0;
		const uint8 ThirdBit = 1 << 2;
		// Only the 2 LSBs of this should change.
		std::atomic<uint8> ExternalState = ThirdBit;

		TExternalMutex<FExternalMutexTestParams> MainMutex(ExternalState);
		MainMutex.Lock();

		// Launch tasks that wait on the locked Mutex.
		for (int32 Index = 0; Index < TaskCount; ++Index)
		{
			// Using FThread for now because UE::Tasks::Launch does not always wake a worker thread.
			Threads[Index] = FThread(TEXT("ExternalMutexTest"), [&TasksComplete, &ExternalState, Index]
			{
				TExternalMutex<FExternalMutexTestParams> Mutex(ExternalState);
				while (!Mutex.TryLock()) // spin on attempting to acquire the lock
				{
					FPlatformProcess::YieldThread();
				}
				CHECK(Mutex.IsLocked());
				CHECK(!Mutex.TryLock());
				CHECK(ExternalState.load() & ThirdBit);
				TasksComplete++;
				Mutex.Unlock();
			});
		}

		MainMutex.Unlock();

		// spin while waiting for tasks to complete
		while (TasksComplete.load() != TaskCount)
		{
		}

		// Wait for the threads to exit.
		for (FThread& Thread : Threads)
		{
			Thread.Join();
		}

		CHECK(ExternalState.load() == ThirdBit);
	}

	SECTION("TExternalMutex with TUniqueLock which uses SlowLock() and SlowUnlock()")
	{
		FThread Threads[TaskCount];
		std::atomic<uint32> TasksComplete = 0;
		const uint8 ThirdBit = 1 << 2;
		// Only the 2 LSBs of this should change.
		std::atomic<uint8> ExternalState = ThirdBit;

		TExternalMutex<FExternalMutexTestParams> MainMutex(ExternalState);
		MainMutex.Lock();

		// Launch tasks that wait on the locked Mutex.
		for (int32 Index = 0; Index < TaskCount; ++Index)
		{
			// Using FThread for now because UE::Tasks::Launch does not always wake a worker thread.
			Threads[Index] = FThread(TEXT("ExternalMutexTest"), [&TasksComplete, &ExternalState, Index]
			{
				TExternalMutex<FExternalMutexTestParams> Mutex(ExternalState);
				TUniqueLock Lock(Mutex);
				CHECK(Mutex.IsLocked());
				CHECK(ExternalState.load() & ThirdBit);
				TasksComplete++;
			});
		}

		MainMutex.Unlock();

		// spin while waiting for tasks to complete
		while (TasksComplete.load() != TaskCount)
		{
		}

		// Wait for the threads to exit.
		for (FThread& Thread : Threads)
		{
			Thread.Join();
		}

		CHECK(ExternalState.load() == ThirdBit);
	}
}

} // UE

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // WITH_LOW_LEVEL_TESTS
