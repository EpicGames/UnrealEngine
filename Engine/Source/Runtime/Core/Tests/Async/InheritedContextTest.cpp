// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Async/ParallelFor.h"
#include "Async/InheritedContext.h"
#include "Async/InheritedGuardValue.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Tasks/Task.h"
#include "Tests/TestHarnessAdapter.h"

#include <atomic>

#if WITH_TESTS

namespace InheritedContextTests
{
	// Getter/setter pair for accessor-based tests
	static thread_local bool GAccessorFlag = false;
	static bool GetAccessorFlag() { return GAccessorFlag; }
	static void SetAccessorFlag(bool Value) { GAccessorFlag = Value; }

	// Helpers for stateful extension tests
	struct FTaskTime
	{
		std::atomic<uint64> GameThreadTime = 0;
		std::atomic<uint64> WorkerThreadTime = 0;
	};

	static uint64 GetTime() { return FPlatformTime::Cycles64(); }

	static void SumTime(const uint64& Start, FTaskTime& Time)
	{
		if (IsInGameThread())
		{
			Time.GameThreadTime.fetch_add(FPlatformTime::Cycles64() - Start, std::memory_order_relaxed);
		}
		else
		{
			Time.WorkerThreadTime.fetch_add(FPlatformTime::Cycles64() - Start, std::memory_order_relaxed);
		}
	}

	// Test simple flag propagation through inherited context extension
	TEST_CASE_NAMED(FInheritedContextSimpleFlag, "System::Core::Async::InheritedContext::SimpleFlag", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		static thread_local bool GTestFlag = false;

		UE::FInheritedContextExtension GTestFlagExt = UE::MakeInheritedContextExtension([]() -> bool& { return GTestFlag; });

		TGuardValue<bool> Guard(GTestFlag, true);

		// Activate extension - tasks launched here will capture GTestFlag
		UE::FInheritedContextExtensionScope ActivateFlag(GTestFlagExt);

		// Using an event prevents retraction since we want those tasks to run on worker threads
		// to validate propagation.
		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&EventA]()
		{
			CHECK(GTestFlag);

			FTaskEvent EventB(TEXT("EventB"));
			auto TaskB = Launch(TEXT("TaskB"), [&EventB]()
			{
				CHECK(GTestFlag);

				EventB.Trigger();
			});

			EventB.Wait();

			// Wait until the task is complete even after the event is triggered
			// otherwise the inherited scope might revert any value after we're out of scope
			TaskB.Wait();
			EventA.Trigger();
		});

		EventA.Wait();

		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();
	}

	// Test TInheritedGuardValue propagation (combines guard + extension scope)
	TEST_CASE_NAMED(FInheritedContextGuardValue, "System::Core::Async::InheritedContext::InheritedGuardValue", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		static thread_local bool GTestFlag = false;

		// Simpler way to propagate guard value on the whole task chain
		TInheritedGuardValue<bool> Guard([]() -> bool& { return GTestFlag; }, true);

		// Using an event prevents retraction since we want those tasks to run on worker threads
		// to validate propagation.
		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&EventA]()
		{
			CHECK(GTestFlag);

			FTaskEvent EventB(TEXT("EventB"));
			auto TaskB = Launch(TEXT("TaskB"), [&EventB]()
			{
				CHECK(GTestFlag);

				EventB.Trigger();
			});

			EventB.Wait();

			// Wait until the task is complete even after the event is triggered
			// otherwise the inherited scope might revert any value after we're out of scope
			TaskB.Wait();
			EventA.Trigger();
		});

		EventA.Wait();

		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();
	}

	// Test TInheritedGuardValue with pre-built extension (avoids per-guard allocation)
	TEST_CASE_NAMED(FInheritedContextPreBuiltGuardValue, "System::Core::Async::InheritedContext::PreBuiltGuardValue", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		static thread_local bool GTestFlag = false;
		static UE::FInheritedContextExtension GTestFlagExt = UE::MakeInheritedContextExtension([]() -> bool& { return GTestFlag; });

		TInheritedGuardValue<bool> Guard(GTestFlagExt, GTestFlag, true);

		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&EventA]()
		{
			CHECK(GTestFlag);

			EventA.Trigger();
		});

		EventA.Wait();

		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();
	}

	// Test getter/setter variant of MakeInheritedContextExtension
	TEST_CASE_NAMED(FInheritedContextGetterSetter, "System::Core::Async::InheritedContext::GetterSetter", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		UE::FInheritedContextExtension GAccessorFlagExt = UE::MakeInheritedContextExtension<&GetAccessorFlag, &SetAccessorFlag>();

		SetAccessorFlag(true);
		UE::FInheritedContextExtensionScope ActivateFlag(GAccessorFlagExt);

		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&EventA]()
		{
			CHECK(GetAccessorFlag());

			FTaskEvent EventB(TEXT("EventB"));
			auto TaskB = Launch(TEXT("TaskB"), [&EventB]()
			{
				CHECK(GetAccessorFlag());
				EventB.Trigger();
			});

			EventB.Wait();

			// Wait until the task is complete even after the event is triggered
			// otherwise the inherited scope might revert any value after we're out of scope
			TaskB.Wait();
			EventA.Trigger();
		});

		EventA.Wait();

		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();

		SetAccessorFlag(false);
	}

	// Test TInheritedGuardValueAccessors
	TEST_CASE_NAMED(FInheritedContextGuardValueAccessors, "System::Core::Async::InheritedContext::GuardValueAccessors", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		TInheritedGuardValueAccessors<&GetAccessorFlag, &SetAccessorFlag> Guard(true);

		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&EventA]()
		{
			CHECK(GetAccessorFlag());

			EventA.Trigger();
		});

		EventA.Wait();

		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();
	}

	// Test propagation through ParallelFor
	TEST_CASE_NAMED(FInheritedContextParallelFor, "System::Core::Async::InheritedContext::ParallelFor", "[.][ApplicationContextMask][EngineFilter]")
	{
		static thread_local bool GTestFlag = false;

		UE::FInheritedContextExtension GTestFlagExt = UE::MakeInheritedContextExtension([]() -> bool& { return GTestFlag; });

		TGuardValue<bool> Guard(GTestFlag, true);
		UE::FInheritedContextExtensionScope ActivateFlag(GTestFlagExt);

		std::atomic<int32> SuccessCount{0};
		constexpr int32 NumIterations = 100;

		ParallelFor(NumIterations, [&SuccessCount](int32)
		{
			if (GTestFlag)
			{
				SuccessCount.fetch_add(1, std::memory_order_relaxed);
			}
		}, EParallelForFlags::Unbalanced);

		CHECK(SuccessCount.load() == NumIterations);
	}

	// Test stateful propagation (timing accumulation across a task chain)
	TEST_CASE_NAMED(FInheritedContextStatefulContext, "System::Core::Async::InheritedContext::StatefulContext", "[.][ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		// Stateful propagation allows automatic accumulation of metrics (e.g. time) across a task chain.
		UE::FInheritedContextExtension TaskTimingExtension = UE::MakeStatefulInheritedContextExtension<&GetTime, &SumTime, FTaskTime>();

		// Activate the extension now
		UE::FInheritedContextExtensionScope ActivateTiming(TaskTimingExtension);

		// Using an event prevents retraction since we want those tasks to run on worker threads
		// to validate propagation.
		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&EventA]()
		{
			FPlatformProcess::Sleep(0.1f);

			FTaskEvent EventB(TEXT("EventB"));
			auto TaskB = Launch(TEXT("TaskB"), [&EventB]()
			{
				FPlatformProcess::Sleep(0.1f);
				EventB.Trigger();
			});

			EventB.Wait(); // Additional 0.1s for waiting on the other task

			// Wait until the task is complete even after the event is triggered
			// otherwise the inherited scope might revert any value after we're out of scope
			TaskB.Wait();

			EventA.Trigger();
		});

		EventA.Wait();

		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();

		FTaskTime& TaskTime = TaskTimingExtension.GetState<FTaskTime>();

		uint64 TotalCycles = TaskTime.GameThreadTime.load() + TaskTime.WorkerThreadTime.load();
		CHECK(FPlatformTime::ToSeconds64(TotalCycles) >= 0.3);
	}

	// Helpers for deduplication test — uses a global atomic counter so Apply/Restore
	// side-effects are visible regardless of which thread the task runs on.
	struct FDedupTracker {};
	static std::atomic<int32> GApplyCount{0};

	static int32 OnDedupApply()
	{
		return GApplyCount.fetch_add(1, std::memory_order_relaxed);
	}

	static void OnDedupRestore(const int32& /*Saved*/, FDedupTracker& /*State*/)
	{
		GApplyCount.fetch_add(-1, std::memory_order_relaxed);
	}

	// Verify that duplicate extensions are captured only once when the same extension
	// appears in both the active scope chain and propagated data. This can happen during
	// task retraction (inline execution on the launching thread) or when a task explicitly
	// re-activates a scope for an already-propagated extension.
	TEST_CASE_NAMED(FInheritedContextDeduplication, "System::Core::Async::InheritedContext::Deduplication", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		GApplyCount.store(0, std::memory_order_relaxed);

		UE::FInheritedContextExtension Ext = UE::MakeStatefulInheritedContextExtension<&OnDedupApply, &OnDedupRestore, FDedupTracker>();
		UE::FInheritedContextExtensionScope Scope(Ext);

		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&Ext, &EventA]() 
		{
			// TaskA's scope applied once
			CHECK(GApplyCount.load(std::memory_order_relaxed) == 1);

			// Re-activate scope for the same extension within the task.
			// Ext is now in both the scope chain (from this scope) and propagated data
			// (from TaskA's restored context), creating the dedup scenario.
			UE::FInheritedContextExtensionScope InnerScope(Ext);

			FTaskEvent EventB(TEXT("EventB"));
			auto TaskB = Launch(TEXT("TaskB"), [&EventB]() 
			{
				// With correct dedup: TaskA(1) + TaskB(1) = 2
				// Without dedup: TaskA(1) + TaskB(2) = 3
				CHECK(GApplyCount.load(std::memory_order_relaxed) == 2);
				EventB.Trigger();
			});

			EventB.Wait();

			// Wait until the task is complete even after the event is triggered
			// otherwise the inherited scope might revert any value after we're out of scope
			TaskB.Wait();
			EventA.Trigger();
		});

		EventA.Wait();
		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();

		// All scopes restored
		CHECK(GApplyCount.load(std::memory_order_relaxed) == 0);
	}

	// Verify that flags are NOT propagated when no extension scope is active
	TEST_CASE_NAMED(FInheritedContextNoPropagationWithoutScope, "System::Core::Async::InheritedContext::NoPropagationWithoutScope", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		static thread_local bool GTestFlag = false;

		// Create extension but do NOT activate a scope
		UE::MakeInheritedContextExtension([]() -> bool& { return GTestFlag; });

		TGuardValue<bool> Guard(GTestFlag, true);

		// No FInheritedContextExtensionScope - flag should not propagate to worker threads
		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&EventA]() 
		{
			CHECK(!GTestFlag);

			FTaskEvent EventB(TEXT("EventB"));
			auto TaskB = Launch(TEXT("TaskB"), [&EventB]() 
			{
				CHECK(!GTestFlag);

				EventB.Trigger();
			});

			EventB.Wait();
			// Wait until the task is complete even after the event is triggered
			// otherwise the inherited scope might revert any value after we're out of scope
			TaskB.Wait();

			EventA.Trigger();
		});

		EventA.Wait();
		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();
	}

	TEST_CASE_NAMED(FInheritedContextMultipleExtensions, "System::Core::Async::InheritedContext::MultipleExtensions", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		static thread_local bool GTestFlags[5] = { false };

		// Simpler way to propagate guard value on the whole task chain
		TInheritedGuardValue<bool> Guard0([]() -> bool& { return GTestFlags[0]; }, true);
		TInheritedGuardValue<bool> Guard1([]() -> bool& { return GTestFlags[1]; }, true);
		// Omit 2 on purpose
		TInheritedGuardValue<bool> Guard3([]() -> bool& { return GTestFlags[3]; }, true);
		TInheritedGuardValue<bool> Guard4([]() -> bool& { return GTestFlags[4]; }, true);

		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&EventA]()
		{
			CHECK(GTestFlags[0]);
			CHECK(GTestFlags[1]);
			CHECK(!GTestFlags[2]);
			CHECK(GTestFlags[3]);
			CHECK(GTestFlags[4]);

			FTaskEvent EventB(TEXT("EventB"));
			auto TaskB = Launch(TEXT("TaskB"), [&EventB]()
			{
				CHECK(GTestFlags[0]);
				CHECK(GTestFlags[1]);
				CHECK(!GTestFlags[2]);
				CHECK(GTestFlags[3]);
				CHECK(GTestFlags[4]);

				EventB.Trigger();
			});

			EventB.Wait();

			// Wait until the task is complete even after the event is triggered
			// otherwise the inherited scope might revert any value after we're out of scope
			TaskB.Wait();
			EventA.Trigger();
		});

		EventA.Wait();

		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();
	}

	// Helpers for ordering test — tracks the sequence in which Apply and Restore are
	// called across multiple extensions. Uses compile-time IDs baked into template
	// instantiations so each extension records a unique identifier.
	struct FOrderState {};
	static std::atomic<int32> GOrderApplyCounter{0};
	static std::atomic<int32> GOrderRestoreCounter{0};
	static int32 GApplySequence[3];
	static int32 GRestoreSequence[3];

	template<int32 Id>
	static int32 OnApplyOrdered()
	{
		int32 Pos = GOrderApplyCounter.fetch_add(1, std::memory_order_seq_cst);
		GApplySequence[Pos] = Id;
		return 0;
	}

	template<int32 Id>
	static void OnRestoreOrdered(const int32& /*Saved*/, FOrderState& /*State*/)
	{
		int32 Pos = GOrderRestoreCounter.fetch_add(1, std::memory_order_seq_cst);
		GRestoreSequence[Pos] = Id;
	}

	// Verify that when multiple extensions are captured and applied on a worker thread,
	// Apply is called in definition order (matching the order scopes were entered on
	// the parent thread) and Restore is called in reverse definition order (standard
	// RAII unwinding). This ensures the innermost scope's value wins on the worker
	// thread, consistent with what the parent thread sees.
	TEST_CASE_NAMED(FInheritedContextMultipleExtensionsApplyOrder, "System::Core::Async::InheritedContext::MultipleExtensionsApplyOrder", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		// Reset global tracking state
		GOrderApplyCounter.store(0, std::memory_order_seq_cst);
		GOrderRestoreCounter.store(0, std::memory_order_seq_cst);
		for (int32 i = 0; i < 3; ++i)
		{
			GApplySequence[i] = -1;
			GRestoreSequence[i] = -1;
		}

		// Create 3 stateful extensions that track their Apply/Restore order.
		// Each has a unique compile-time ID (0, 1, 2).
		UE::FInheritedContextExtension Ext0 = UE::MakeStatefulInheritedContextExtension<&OnApplyOrdered<0>, &OnRestoreOrdered<0>, FOrderState>();
		UE::FInheritedContextExtension Ext1 = UE::MakeStatefulInheritedContextExtension<&OnApplyOrdered<1>, &OnRestoreOrdered<1>, FOrderState>();
		UE::FInheritedContextExtension Ext2 = UE::MakeStatefulInheritedContextExtension<&OnApplyOrdered<2>, &OnRestoreOrdered<2>, FOrderState>();

		// Activate scopes in definition order 0, 1, 2.
		// The scope chain linked list will be 2 -> 1 -> 0 (LIFO), and entries are
		// captured in that order. Apply iterates backward to replay in definition order.
		UE::FInheritedContextExtensionScope Scope0(Ext0);
		UE::FInheritedContextExtensionScope Scope1(Ext1);
		UE::FInheritedContextExtensionScope Scope2(Ext2);

		// Using an event prevents retraction since we want the task to run on a worker
		// thread to validate cross-thread ordering.
		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&EventA]()
		{
			// By this point, RestoreInheritedContext has been called by the task system,
			// which applied all 3 extensions via the FInheritedContextScope constructor.
			// Verify that Apply was called in definition order: 0, 1, 2.
			// This ensures the innermost scope (2) is applied last and its value wins.
			CHECK(GOrderApplyCounter.load(std::memory_order_seq_cst) == 3);
			CHECK(GApplySequence[0] == 0);
			CHECK(GApplySequence[1] == 1);
			CHECK(GApplySequence[2] == 2);

			EventA.Trigger();
		});

		EventA.Wait();

		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();

		// After the task completes and its FInheritedContextScope destructor runs,
		// Restore should have been called in reverse definition order: 2, 1, 0
		// (standard RAII unwinding — last applied is first restored)
		CHECK(GOrderRestoreCounter.load(std::memory_order_seq_cst) == 3);
		CHECK(GRestoreSequence[0] == 2);
		CHECK(GRestoreSequence[1] == 1);
		CHECK(GRestoreSequence[2] == 0);
	}

	// Verify that when multiple nested scopes modify the same variable via the same
	// pre-built extension, the worker thread sees the innermost (last set) value.
	// All scopes share the same Impl pointer, so CaptureInheritedContext captures
	// the variable's current value (set by the innermost guard) for each entry.
	TEST_CASE_NAMED(FInheritedContextNestedScopesSameVariable, "System::Core::Async::InheritedContext::NestedScopesSameVariable", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		static thread_local int32 GValue = 0;
		static UE::FInheritedContextExtension GValueExt = UE::MakeInheritedContextExtension([]() -> int32& { return GValue; });

		// Three nested guard+scope pairs on the same variable and extension.
		// Each guard sets a different value; the innermost (30) should win.
		TGuardValue<int32> Guard1(GValue, 10);
		UE::FInheritedContextExtensionScope Scope1(GValueExt);

		TGuardValue<int32> Guard2(GValue, 20);
		UE::FInheritedContextExtensionScope Scope2(GValueExt);

		TGuardValue<int32> Guard3(GValue, 30);
		UE::FInheritedContextExtensionScope Scope3(GValueExt);

		// Using an event prevents retraction since we want the task to run on a worker
		// thread to validate propagation.
		FTaskEvent EventA(TEXT("EventA"));
		auto TaskA = Launch(TEXT("TaskA"), [&EventA]()
		{
			// The innermost scope's value (30) should be visible on the worker thread,
			// consistent with what the parent thread sees.
			CHECK(GValue == 30);

			EventA.Trigger();
		});

		EventA.Wait();

		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();
	}

	TEST_CASE_NAMED(FInheritedContextExtensionGoesOutOfScopeBeforeExecution, "System::Core::Async::InheritedContext::ExtensionGoesOutOfScopeBeforeExecution", "[ApplicationContextMask][EngineFilter]")
	{
		using namespace UE::Tasks;

		static thread_local bool GTestFlag = false;

		FTaskEvent EventA(TEXT("EventA"));
		FTaskEvent StartEvent(TEXT("StartEvent"));

		FTask TaskA;
		{
			UE::FInheritedContextExtension GTestFlagExt = UE::MakeInheritedContextExtension([]() -> bool& { return GTestFlag; });
		
			TGuardValue<bool> Guard(GTestFlag, true);
			UE::FInheritedContextExtensionScope ActivateFlag(GTestFlagExt);

			TaskA = Launch(TEXT("TaskA"), [&EventA]()
			{
				CHECK(GTestFlag);
				// test toggling the flag for the next task while we're at it
				TGuardValue<bool> Guard(GTestFlag, false);

				FTaskEvent EventB(TEXT("EventB"));
				Launch(TEXT("TaskB"), [&EventB]()
				{
					CHECK(!GTestFlag);
					TGuardValue<bool> Guard(GTestFlag, true);

					EventB.Trigger();
				});

				EventB.Wait();

				EventA.Trigger();
			}, StartEvent);
		}

		CHECK(!GTestFlag);
		StartEvent.Trigger();
		EventA.Wait();

		// Wait until the task is complete even after the event is triggered
		// otherwise the inherited scope might revert any value after we're out of scope
		TaskA.Wait();
	}
}

#endif //WITH_TESTS
