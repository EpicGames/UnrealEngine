// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Async/ManualResetEvent.h"

#include "Tasks/Task.h"
#include "TestHarness.h"

namespace UE
{

TEST_CASE("Core::Async::ManualResetEvent", "[Core][Async]")
{
	FManualResetEvent Event;

	CHECK_FALSE(Event.IsNotified());
	CHECK_FALSE(Event.WaitFor(FMonotonicTimeSpan::Zero()));
	CHECK_FALSE(Event.WaitFor(FMonotonicTimeSpan::FromMilliseconds(1.0)));
	CHECK_FALSE(Event.WaitUntil(FMonotonicTimePoint::Now() + FMonotonicTimeSpan::FromMilliseconds(1.0)));
	Event.Notify();
	Event.Wait();
	CHECK(Event.IsNotified());
	CHECK(Event.WaitFor(FMonotonicTimeSpan::Zero()));
	CHECK(Event.WaitFor(FMonotonicTimeSpan::FromSeconds(60.0)));
	CHECK(Event.WaitUntil(FMonotonicTimePoint::Now() + FMonotonicTimeSpan::FromSeconds(60.0)));
	Event.Reset();
	CHECK_FALSE(Event.IsNotified());
	CHECK_FALSE(Event.WaitFor(FMonotonicTimeSpan::Zero()));
	CHECK_FALSE(Event.WaitFor(FMonotonicTimeSpan::FromMilliseconds(1.0)));
	CHECK_FALSE(Event.WaitUntil(FMonotonicTimePoint::Now() + FMonotonicTimeSpan::FromMilliseconds(1.0)));

	Tasks::Launch(UE_SOURCE_LOCATION, [&Event] { Event.Notify(); });
	Event.Wait();

	Event.Notify();
	CHECK(Event.IsNotified());
	Event.Reset();
	CHECK_FALSE(Event.IsNotified());
}

} // UE

#endif // WITH_LOW_LEVEL_TESTS
