// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Algo/Accumulate.h"
#include "Algo/Sort.h"
#include "Async/ExternalMutex.h"
#include "Async/Fundamental/Scheduler.h"
#include "Async/IntrusiveMutex.h"
#include "Async/Mutex.h"
#include "Async/ParallelFor.h"
#include "Async/RecursiveMutex.h"
#include "Async/SharedMutex.h"
#include "Async/SharedRecursiveMutex.h"
#include "Async/UniqueLock.h"
#include "Async/WordMutex.h"
#include "HAL/PlatformMutex.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeExit.h"
#include "Tasks/Task.h"
#include "TestHarness.h"

namespace UE
{

template <typename BodyType>
static double TestConcurrency(const int32 TaskCount, BodyType&& Body)
{
	constexpr int32 MaxTaskCount = 256;
	check(TaskCount <= MaxTaskCount);

	LowLevelTasks::FTask Tasks[MaxTaskCount];
	std::atomic<int32> StartCount = TaskCount;
	std::atomic<int32> EndCount = TaskCount;

	const auto TaskBody = [&Body, &StartCount, &EndCount](int32 TaskIndex)
	{
		StartCount.fetch_sub(1, std::memory_order_relaxed);
		while (StartCount.load(std::memory_order_relaxed) > 0)
		{
		}
		Invoke(Body, TaskIndex);
		EndCount.fetch_sub(1, std::memory_order_relaxed);
	};

	for (int32 TaskIndex = 1; TaskIndex < TaskCount; ++TaskIndex)
	{
		Tasks[TaskIndex].Init(UE_SOURCE_LOCATION, LowLevelTasks::ETaskPriority::Normal, [&TaskBody, TaskIndex]
		{
			Invoke(TaskBody, TaskIndex);
		});
		verify(LowLevelTasks::TryLaunch(Tasks[TaskIndex]));
	}

	while (StartCount.load(std::memory_order_relaxed) > 1)
	{
	}

	const double StartTime = FPlatformTime::Seconds();
	StartCount.fetch_sub(1, std::memory_order_relaxed);
	Invoke(Body, 0);
	while (EndCount.load(std::memory_order_relaxed) > 1)
	{
	}
	const double EndTime = FPlatformTime::Seconds();

	for (LowLevelTasks::FTask& Task : MakeArrayView(Tasks, TaskCount))
	{
		while (!Task.IsCompleted())
		{
		}
	}

	return EndTime - StartTime;
}

template <typename LockType>
static void TestLockRate(LockType& Mutex, int32 LockTarget, int32 IterationCount)
{
	using namespace UE::Core::Private;

	UE_LOGF(LogCore, Display, "%-8ls %12ls %12ls %12ls", TEXT("Threads"), TEXT("LockRate"), TEXT("Mean"), TEXT("StdDev"));

	static const int32 MinThreadCount = []
	{
		int32 Value = 1;
		FParse::Value(FCommandLine::Get(), TEXT("-MinThreadCount="), Value);
		return Value;
	}();
	static const int32 MaxThreadCount = []
	{
		int32 Value = 1024;
		FParse::Value(FCommandLine::Get(), TEXT("-MaxThreadCount="), Value);
		return Value;
	}();

	struct FIteration
	{
		int64 LockRate = 0;
		int64 LockCount = 0;
		TArray<int64> LockCountByThread;
	};
	TArray<FIteration> Iterations;

	const int32 ThreadLimit = FMath::Min((int32)LowLevelTasks::FScheduler::Get().GetNumWorkers(), FPlatformMisc::NumberOfCores());
	for (int32 ThreadCount = FPlatformMath::Min(MinThreadCount, ThreadLimit); ThreadCount <= FPlatformMath::Min(MaxThreadCount, ThreadLimit); ++ThreadCount)
	{
		Iterations.Reset(IterationCount);
		for (int32 Iteration = 0; Iteration < IterationCount; ++Iteration)
		{
			TArray<int64> LockCountByThread;
			LockCountByThread.AddZeroed(ThreadCount);
			std::atomic<bool> bStop = false;
			std::atomic<int64> LockCount = 0;
			const double Duration = TestConcurrency(ThreadCount, [&Mutex, &LockCount, &LockCountByThread, LockTarget, &bStop](int32 ThreadIndex)
			{
				int64 ThreadLockCount = 0;
				while (!bStop.load(std::memory_order_relaxed))
				{
					Mutex.Lock();
					Mutex.Unlock();
					if (++ThreadLockCount >= LockTarget && ThreadIndex == 0)
					{
						bStop.store(true, std::memory_order_relaxed);
					}
				}
				LockCount.fetch_add(ThreadLockCount, std::memory_order_relaxed);
				LockCountByThread[ThreadIndex] = ThreadLockCount;
			});
			Iterations.Add({int64(LockCount / Duration), LockCount, MoveTemp(LockCountByThread)});
		}

		Algo::SortBy(Iterations, &FIteration::LockRate);
		const FIteration& Iteration = Iterations.Last();
		const int64 LockRate = Iteration.LockRate;
		const int64 LockCountMean = Iteration.LockCount / ThreadCount;
		const int64 LockCountStdDev =
			int64(FMath::Sqrt(double(Algo::TransformAccumulate(Iteration.LockCountByThread, [LockCountMean](int64 ThreadLockRate)
			{
				return FMath::Square(FMath::Abs(ThreadLockRate - LockCountMean));
			}, int64(0))) / ThreadCount));

		UE_LOG(LogCore, Display, TEXT("%-8d %12" INT64_FMT " %12" INT64_FMT " %12" INT64_FMT), ThreadCount, LockRate, LockCountMean, LockCountStdDev);
	}
}

struct FExternalMutexRateTestParams
{
	constexpr static uint8 IsLockedFlag = 1 << 7;
	constexpr static uint8 MayHaveWaitingLockFlag = 1 << 6;
};

struct FIntrusiveMutexRateTestParams
{
	constexpr static int64 IsLockedFlag = int64(1) << 40;
	constexpr static int64 MayHaveWaitingLockFlag = int64(1) << 50;
};

TEST_CASE("Core::Async::MutexLockRate", "[.][Core][Async][Slow]")
{
	using namespace UE;

	LowLevelTasks::FScheduler::Get().RestartWorkers(0, FPlatformMisc::NumberOfWorkerThreadsToSpawn());
	ON_SCOPE_EXIT { LowLevelTasks::FScheduler::Get().RestartWorkers(); };

	GLog->Flush();
	TGuardValue NoTime(GPrintLogTimes, ELogTimes::None);
	TGuardValue NoCategory(GPrintLogCategory, false);
	TGuardValue NoVerbosity(GPrintLogVerbosity, false);

	int32 LockTarget = 24576;
	FParse::Value(FCommandLine::Get(), TEXT("-LockTarget="), LockTarget);
	int32 IterationCount = 16;
	FParse::Value(FCommandLine::Get(), TEXT("-Iterations="), IterationCount);

	SECTION("WordMutex")
	{
		UE_LOGF(LogCore, Display, "FWordMutex");
		TestLockRate(*MakeUnique<FWordMutex>(), LockTarget, IterationCount);
	}

	SECTION("Mutex")
	{
		UE_LOGF(LogCore, Display, "FMutex");
		TestLockRate(*MakeUnique<FMutex>(), LockTarget, IterationCount);
	}

	SECTION("RecursiveMutex")
	{
		UE_LOGF(LogCore, Display, "FRecursiveMutex");
		TestLockRate(*MakeUnique<FRecursiveMutex>(), LockTarget, IterationCount);
	}

	SECTION("SharedMutex")
	{
		UE_LOGF(LogCore, Display, "FSharedMutex");
		TestLockRate(*MakeUnique<FSharedMutex>(), LockTarget, IterationCount);
	}

	SECTION("SharedRecursiveMutex")
	{
		UE_LOGF(LogCore, Display, "FSharedRecursiveMutex");
		TestLockRate(*MakeUnique<FSharedRecursiveMutex>(), LockTarget, IterationCount);
	}

	SECTION("ExternalMutex")
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_LOGF(LogCore, Display, "TExternalMutex(uint8)");
		struct FExternalMutexType
		{
			std::atomic<uint8> State;
			TExternalMutex<FExternalMutexRateTestParams> Mutex{State};
			void Lock() { Mutex.Lock(); }
			void Unlock() { Mutex.Unlock(); }
		};
		TestLockRate(*MakeUnique<FExternalMutexType>(), LockTarget, IterationCount);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	SECTION("IntrusiveMutex")
	{
		UE_LOGF(LogCore, Display, "TIntrusiveMutex(int64)");
		struct FIntrusiveMutexType
		{
			std::atomic<int64> State;
			void Lock() { TIntrusiveMutex<FIntrusiveMutexRateTestParams>::Lock(State); }
			void Unlock() { TIntrusiveMutex<FIntrusiveMutexRateTestParams>::Unlock(State); }
		};
		TestLockRate(*MakeUnique<FIntrusiveMutexType>(), LockTarget, IterationCount);
	}

	SECTION("PlatformRecursiveMutex")
	{
		UE_LOGF(LogCore, Display, "FPlatformRecursiveMutex");
		TestLockRate(*MakeUnique<FPlatformRecursiveMutex>(), LockTarget, IterationCount);
	}

	SECTION("PlatformSharedMutex")
	{
		UE_LOGF(LogCore, Display, "FPlatformSharedMutex");
		TestLockRate(*MakeUnique<FPlatformSharedMutex>(), LockTarget, IterationCount);
	}

	GLog->Flush();
}

} // UE

#endif // WITH_LOW_LEVEL_TESTS
