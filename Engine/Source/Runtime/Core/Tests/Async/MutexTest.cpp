// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Async/Mutex.h"

#include "Async/UniqueLock.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "TestHarness.h"

namespace UE
{

TEST_CASE("Core::Async::Mutex", "[Core][Async][Slow][LinuxSkip]")
{
	constexpr static int32 TaskCount = 5;

	SECTION("FMutex IsLocked() and TryLock()")
	{
		FThread Threads[TaskCount];
		std::atomic<uint32> TasksComplete = 0;
		
		FMutex Mutex;
		CHECK(Mutex.TryLock());

		// Launch tasks that wait on the locked Mutex.
		for (int32 Index = 0; Index < TaskCount; ++Index)
		{
			// Using FThread for now because UE::Tasks::Launch does not always wake a worker thread.
			Threads[Index] = FThread(TEXT("MutexTest"), [&Mutex, &TasksComplete]
			{
				while (!Mutex.TryLock()) // spin on attempting to acquire the lock
				{
					FPlatformProcess::YieldThread();
				}
				CHECK(Mutex.IsLocked());
				CHECK(!Mutex.TryLock());
				TasksComplete++;
				Mutex.Unlock();
			});
		}

		Mutex.Unlock();

		// spin while waiting for tasks to complete
		while (TasksComplete != TaskCount)
		{
		}

		// Wait for the threads to exit.
		for (FThread& Thread : Threads)
		{
			Thread.Join();
		}
	}

	SECTION("FMutex with TUniqueLock which uses SlowLock() and SlowUnlock()")
	{
		FThread Threads[TaskCount];
		std::atomic<uint32> TasksComplete = 0;
		
		FMutex Mutex;
		Mutex.Lock();
		CHECK(Mutex.IsLocked());

		// Launch tasks that wait on the locked Mutex.
		for (int32 Index = 0; Index < TaskCount; ++Index)
		{
			// Using FThread for now because UE::Tasks::Launch does not always wake a worker thread.
			Threads[Index] = FThread(TEXT("MutexTest"), [&Mutex, &TasksComplete]
			{
				TUniqueLock Lock(Mutex);
				CHECK(Mutex.IsLocked());
				TasksComplete++;
			});
		}

		Mutex.Unlock();

		// spin while waiting for tasks to complete
		while (TasksComplete != TaskCount)
		{
		}

		// Wait for the threads to exit.
		for (FThread& Thread : Threads)
		{
			Thread.Join();
		}
	}
}

} // UE

#endif // WITH_LOW_LEVEL_TESTS
