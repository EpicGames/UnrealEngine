// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Async/ParkingLot.h"

#include "Async/ParallelFor.h"
#include "HAL/Thread.h"
#include "TestHarness.h"

namespace UE
{

TEST_CASE("Core::Async::ParkingLot", "[Core][Async]")
{
	SECTION("FIFO, WakeToken")
	{
		constexpr static int32 TaskCount = 5;
		FThread Threads[TaskCount];
		std::atomic<int32> WaitCount = 0;
		ParkingLot::FWaitState WaitStates[TaskCount];

		// Launch tasks that wait on the address of WaitCount.
		for (int32 Index = 0; Index < TaskCount; ++Index)
		{
			// Using FThread for now because UE::Tasks::Launch does not always wake a worker thread.
			Threads[Index] = FThread(TEXT("ParkingLotTest"), [&WaitCount, OutState = &WaitStates[Index]]
			{
				int32 CanWaitCount = 0;
				int32 BeforeWaitCount = 0;
				*OutState = ParkingLot::Wait(&WaitCount,
					[&CanWaitCount] { ++CanWaitCount; return true; },
					[&BeforeWaitCount, &WaitCount] { ++BeforeWaitCount; WaitCount.fetch_add(1); });
				WaitCount.fetch_sub(1);
				CHECK(CanWaitCount == 1);
				CHECK(BeforeWaitCount == 1);
			});

			// Spin until the task is waiting.
			while (WaitCount != Index + 1)
			{
			}
		}

		// Wake each task with a sequence number, with an extra wake call that has no thread to wake.
		uint64 Sequence = 0;
		for (int32 Index = 0; Index <= TaskCount; ++Index)
		{
			int32 WakeCount = 0;
			bool bDidWake = false;
			bool bHasWaitingThreads = false;
			ParkingLot::WakeOne(&WaitCount, [&](ParkingLot::FWakeState WakeState) -> uint64
			{
				++WakeCount;
				bDidWake = WakeState.bDidWake;
				bHasWaitingThreads = WakeState.bHasWaitingThreads;
				return ++Sequence;
			});
			// The callback must be invoked exactly once.
			CHECK(WakeCount == 1);
			CHECK(bDidWake == (Index < TaskCount));
			CHECK(bHasWaitingThreads >= (Index + 1 < TaskCount));
		}

		// Spin until the tasks are complete.
		while (WaitCount != 0)
		{
		}

		// Verify that tasks woke in FIFO order.
		for (int32 Index = 0; Index < TaskCount; ++Index)
		{
			const ParkingLot::FWaitState& WaitState = WaitStates[Index];
			CHECK(WaitState.bDidWait);
			CHECK(WaitState.bDidWake);
			CHECK(WaitState.WakeToken == uint64(Index + 1));
		}

		// Wait for the threads to exit.
		for (FThread& Thread : Threads)
		{
			Thread.Join();
		}
	}

	SECTION("CanWait")
	{
		int32 Value = 0;
		int32 CanWaitCount = 0;
		int32 BeforeWaitCount = 0;
		ParkingLot::FWaitState State = ParkingLot::Wait(&Value,
			[&CanWaitCount] { ++CanWaitCount; return false; },
			[&BeforeWaitCount] { ++BeforeWaitCount; });
		CHECK(CanWaitCount == 1);
		CHECK(BeforeWaitCount == 0);
		CHECK_FALSE(State.bDidWait);
		CHECK_FALSE(State.bDidWake);
		CHECK(State.WakeToken == 0);
	}

	SECTION("WaitFor with Timeout")
	{
		int32 Value = 0;
		int32 CanWaitCount = 0;
		int32 BeforeWaitCount = 0;
		ParkingLot::FWaitState State = ParkingLot::WaitFor(&Value,
			[&CanWaitCount] { ++CanWaitCount; return true; },
			[&BeforeWaitCount] { ++BeforeWaitCount; },
			FMonotonicTimeSpan::FromMilliseconds(1.0));
		CHECK(CanWaitCount == 1);
		CHECK(BeforeWaitCount == 1);
		CHECK(State.bDidWait);
		CHECK_FALSE(State.bDidWake);
		CHECK(State.WakeToken == 0);
	}

	SECTION("WaitUntil with Timeout")
	{
		int32 Value = 0;
		int32 CanWaitCount = 0;
		int32 BeforeWaitCount = 0;
		ParkingLot::FWaitState State = ParkingLot::WaitUntil(&Value,
			[&CanWaitCount] { ++CanWaitCount; return true; },
			[&BeforeWaitCount] { ++BeforeWaitCount; },
			FMonotonicTimePoint::Now() + FMonotonicTimeSpan::FromMilliseconds(1.0));
		CHECK(CanWaitCount == 1);
		CHECK(BeforeWaitCount == 1);
		CHECK(State.bDidWait);
		CHECK_FALSE(State.bDidWake);
		CHECK(State.WakeToken == 0);
	}

	SECTION("WaitFor in ParallelFor")
	{
		// This exercises the code that grows the parking lot hash table.
		UE_CALL_ONCE([]
		{
			constexpr int32 Count = 512;
			std::atomic<int32> Remaining = Count;
			ParallelFor(TEXT("WaitFor"), Count, 1, [&Remaining](int32)
			{
				int32 Value = 0;
				ParkingLot::WaitFor(&Value, [] { return true; },
					[&Remaining]
					{
						Remaining.fetch_sub(1, std::memory_order_relaxed);
					}, FMonotonicTimeSpan::FromMilliseconds(1.0));
			});
			CHECK(Remaining == 0);
		});
	}
}

} // UE

#endif // WITH_LOW_LEVEL_TESTS
