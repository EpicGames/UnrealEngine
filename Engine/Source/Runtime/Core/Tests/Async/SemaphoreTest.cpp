// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "Tests/Benchmark.h"
#include "HAL/Thread.h"
#include "HAL/UESemaphore.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Containers/Array.h"
#include "Templates/UniquePtr.h"
#include "Tests/TestHarnessAdapter.h"

#include <atomic>

#if WITH_TESTS

namespace UE::SemaphoreTests
{
	using namespace Tasks;

	template<uint32 NumThreads, uint32 NumLoops, bool ReleaseAllAtOnce>
	void TestSemaphoreWakeUpPerf()
	{
		TArray<FThread> Threads;
		Threads.Reserve(NumThreads);
		std::atomic<bool> bQuit{ false };

		FSemaphore Semaphore{ 0, NumThreads };
		std::atomic<uint32> Sync1{ 0 };
		std::atomic<uint32> Sync2{ 0 };

		for (uint32 i = 0; i < NumThreads; ++i)
		{
			Threads.Emplace(TEXT("SemaphoreWakeUpPerfTest"),
				[&Semaphore, &Sync1, &Sync2, &bQuit]
				{
					while (!bQuit)
					{
						Semaphore.Acquire();
						--Sync1;
						while (Sync1 != 0)
						{
						}
						--Sync2;
					}
				}
				);
		}

		auto Iteration = [&Semaphore, &Sync1, &Sync2]
		{
			Sync2 = NumThreads;
			Sync1 = NumThreads;

			if constexpr (ReleaseAllAtOnce)
			{
				Semaphore.Release(NumThreads);
			}
			else
			{
				for (uint32 i = 0; i < NumThreads; ++i)
				{
					Semaphore.Release();
				}
			}

			while (Sync2 != 0)
			{
			}
			FPlatformProcess::Sleep(0.0);
		};

		for (uint32 i = 0; i < NumLoops; ++i)
		{
			Iteration();
		}

		Sync1 = NumThreads;
		bQuit = true;

		Semaphore.Release(NumThreads);
		while (Sync1 != 0)
		{
		}

		for (FThread& Thread : Threads)
		{
			Thread.Join();
		}
	}

	template<uint32 NumThreads, uint32 NumLoops>
	void TestMultiEventWakeUpPerf()
	{
		TArray<FThread> Threads;
		Threads.Reserve(NumThreads);
		std::atomic<bool> bQuit{ false };

		FEventRef Events[NumThreads];

		std::atomic<uint32> Sync1{ 0 };
		std::atomic<uint32> Sync2{ 0 };

		for (uint32 i = 0; i < NumThreads; ++i)
		{
			Threads.Emplace(TEXT("MultiEventWakeUpPerfTest"),
				[Event = &Events[i], &Sync1, &Sync2, &bQuit]()
				{
					while (!bQuit)
					{
						Event->Get()->Wait();
						--Sync1;
						while (Sync1 != 0)
						{
						}
						--Sync2;
					}
				}
			);
		}

		auto Iteration = [&Events, &Sync1, &Sync2]
		{
			Sync2 = NumThreads;
			Sync1 = NumThreads;

			for (uint32 i = 0; i < NumThreads; ++i)
			{
				Events[i]->Trigger();
			}

			while (Sync2 != 0)
			{
			}
			FPlatformProcess::Sleep(0.0);
		};

		for (uint32 i = 0; i != NumLoops; ++i)
		{
			Iteration();
		}

		Sync1 = NumThreads;
		bQuit = true;

		for (int i = 0; i != NumThreads; ++i)
		{
			Events[i]->Trigger();
		}
		while (Sync1 != 0)
		{
		}
	
		for (FThread& Thread : Threads)
		{
			Thread.Join();
		}
	}

	// benchmark waking up multiple threads by releasing a semaphore and compare with triggering multiple auto-reset events
	TEST_CASE_NAMED(FSemaphoreWakeUpPerfTest, "System::Core::Async::Semaphore::WakeUpPerf", "[.][ApplicationContextMask][EngineFilter]")
	{
		UE_BENCHMARK(5, TestSemaphoreWakeUpPerf<5, 100000, true>);
		UE_BENCHMARK(5, TestSemaphoreWakeUpPerf<5, 100000, false>);
		UE_BENCHMARK(5, TestMultiEventWakeUpPerf<5, 100000>);
	}
}

#endif //WITH_TEST
