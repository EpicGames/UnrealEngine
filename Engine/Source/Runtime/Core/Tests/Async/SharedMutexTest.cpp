// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Async/SharedMutex.h"

#include "Async/ManualResetEvent.h"
#include "Async/SharedLock.h"
#include "CoreGlobals.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE
{

TEST_CASE_NAMED(FSharedMutexTest, "Core::Async::SharedMutex", "[Core][Async]")
{
	SECTION("SingleThread")
	{
		FSharedMutex Mutex;

		Mutex.Lock();
		CHECK_FALSE(TDynamicSharedLock(Mutex, DeferLock).TryLock());
		Mutex.Unlock();

		CHECK(Mutex.TryLock());
		Mutex.Unlock();

		// This test performs recursive locking because it exercises the current implementation
		// but it is not technically supported by the mutex and can deadlock if used generally.
		{
			TSharedLock Lock1(Mutex);
			TSharedLock Lock2(Mutex);
			TSharedLock Lock3(Mutex);
			CHECK_FALSE(Mutex.TryLock());
		}

		{
			TDynamicSharedLock Lock1(Mutex, DeferLock);
			TDynamicSharedLock Lock2(Mutex, DeferLock);
			TDynamicSharedLock Lock3(Mutex, DeferLock);
			CHECK(Lock1.TryLock());
			CHECK(Lock2.TryLock());
			CHECK(Lock3.TryLock());
			CHECK_FALSE(Mutex.TryLock());
		}

		CHECK(Mutex.TryLock());
		Mutex.Unlock();
	}

	SECTION("MultipleThreads")
	{
		FSharedMutex Mutex;
		uint32 Counter = 0;
		FManualResetEvent Events[4];
		const auto MakeWait = [&Events](int32 Index)
		{
			return [&Events, Index]
			{
				Events[Index].Wait();
				Events[Index].Reset();
			};
		};
		const auto Wake = [&Events](int32 Index)
		{
			Events[Index].Notify();
			FPlatformProcess::YieldThread();
		};

		class FCountdownEvent
		{
		public:
			void Reset(int32 Count)
			{
				Counter.store(Count, std::memory_order_relaxed);
				Event.Reset();
			}

			void Notify()
			{
				if (Counter.fetch_sub(1, std::memory_order_release) == 1)
				{
					Event.Notify();
				}
			}

			void Wait()
			{
				Event.Wait();
			}

		private:
			std::atomic<int32> Counter = 0;
			FManualResetEvent Event;
		};
		FCountdownEvent CountdownEvent;

		FThread Thread0(TEXT("SharedMutexTest0"), [&Mutex, &Counter, &CountdownEvent, &Wake, Wait = MakeWait(0)]
		{
			// Test 1: Exclusive w/ one waiting exclusive lock.
			Mutex.Lock();
			Wake(1);
			Wait();
			Counter = 1;
			Mutex.Unlock();

			// Test 2: Exclusive w/ one waiting shared lock.
			Wait();
			Wake(2);
			Mutex.LockShared();
			CHECK(Counter == 2);

			// Test 3: Shared w/ one waiting exclusive lock.
			Wake(1);
			Wait();
			Counter = 3;
			Mutex.UnlockShared();

			// Test 4: Exclusive w/ three waiting shared locks.
			Wait();
			Wake(1);
			Mutex.LockShared();
			CHECK(Counter == 4);
			Wait();
			Mutex.UnlockShared();

			// Test 5: Shared w/ no exclusive contention.
			CountdownEvent.Reset(3);
			Wake(1);
			Wake(2);
			Wake(3);
			for (int32 I = 0; I < 16384; ++I)
			{
				Mutex.LockShared();
				FPlatformProcess::YieldThread();
				Mutex.UnlockShared();
			}
			CountdownEvent.Wait();

			if (GIsBuildMachine)
			{
				return;
			}

			// Test 6: Shared w/ one waiting exclusive lock and one waiting shared lock.
			Mutex.LockShared();
			Counter = 5;
			Wake(1);
			Wait();
			Mutex.UnlockShared();
		});

		FThread Thread1(TEXT("SharedMutexTest1"), [&Mutex, &Counter, &CountdownEvent, &Wake, Wait = MakeWait(1)]
		{
			// Test 1: Exclusive w/ one waiting exclusive lock.
			Wait();
			Wake(2);
			Mutex.Lock();
			CHECK(Counter == 1);

			// Test 2: Exclusive w/ one waiting shared lock.
			Wake(0);
			Wait();
			Counter = 2;
			Mutex.Unlock();

			// Test 3: Shared w/ one waiting exclusive lock.
			Wait();
			Wake(2);
			Mutex.Lock();
			CHECK(Counter == 3);

			// Test 4: Exclusive w/ three waiting shared locks.
			Wake(2);
			Wait();
			Counter = 4;
			Mutex.Unlock();

			// Test 5: Shared w/ no exclusive contention.
			Wait();
			for (int32 I = 0; I < 16384; ++I)
			{
				Mutex.LockShared();
				FPlatformProcess::YieldThread();
				Mutex.UnlockShared();
			}
			CountdownEvent.Notify();

			if (GIsBuildMachine)
			{
				return;
			}

			// Test 6: Shared w/ one waiting exclusive lock and one waiting shared lock.
			Wait();
			Wake(2);
			Mutex.Lock();
			CHECK(Counter == 5);
			Counter = 6;
			Mutex.Unlock();
		});

		FThread Thread2(TEXT("SharedMutexTest2"), [&Mutex, &Counter, &CountdownEvent, &Wake, Wait = MakeWait(2)]
		{
			// Test 1: Exclusive w/ one waiting exclusive lock.
			Wait();
			Wake(0);

			// Test 2: Exclusive w/ one waiting shared lock.
			Wait();
			Wake(1);

			// Test 3: Shared w/ one waiting exclusive lock.
			Wait();
			Wake(0);

			// Test 4: Exclusive w/ three waiting shared locks.
			Wait();
			Wake(3);
			Mutex.LockShared();
			CHECK(Counter == 4);
			Wake(3);
			Mutex.UnlockShared();

			// Test 5: Shared w/ no exclusive contention.
			Wait();
			for (int32 I = 0; I < 16384; ++I)
			{
				Mutex.LockShared();
				FPlatformProcess::YieldThread();
				Mutex.UnlockShared();
			}
			CountdownEvent.Notify();

			if (GIsBuildMachine)
			{
				return;
			}

			// Test 6: Shared w/ one waiting exclusive lock and one waiting shared lock.
			Wait();
			FPlatformProcess::Sleep(0.001f); // Hopefully reliable enough to ensure the exclusive lock is waiting.
			Wake(0);
			Mutex.LockShared();
			CHECK(Counter == 6);
			Mutex.UnlockShared();
		});

		FThread Thread3(TEXT("SharedMutexTest3"), [&Mutex, &Counter, &CountdownEvent, &Wake, Wait = MakeWait(3)]
		{
			// Test 4: Exclusive w/ three waiting shared locks.
			Wait();
			Wake(0);
			Mutex.LockShared();
			CHECK(Counter == 4);
			Wait();
			Wake(0);
			Mutex.UnlockShared();

			// Test 5: Shared w/ no exclusive contention.
			Wait();
			for (int32 I = 0; I < 16384; ++I)
			{
				Mutex.LockShared();
				FPlatformProcess::YieldThread();
				Mutex.UnlockShared();
			}
			CountdownEvent.Notify();
		});

		Thread0.Join();
		Thread1.Join();
		Thread2.Join();
		Thread3.Join();
	}
}

} // UE

#endif // WITH_TESTS
