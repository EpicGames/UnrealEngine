// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include "Misc/DateTime.h"
#include "HAL/PlatformProcess.h"
#include "Stats/Stats.h"
#include "Misc/AutomationTest.h"
#include "Async/TaskGraphInterfaces.h"
#include "Async/ParallelFor.h"
#include "HAL/ThreadHeartBeat.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Math/RandomStream.h"
#include "Containers/CircularQueue.h"
#include "Containers/Queue.h"
#include "Tests/Benchmark.h"
#include "HAL/Thread.h"
#include "Async/Fundamental/Scheduler.h"
#include "Tests/TestHarnessAdapter.h"

#include <atomic>

#if WITH_TESTS

namespace OldTaskGraphTests
{
	static FORCEINLINE void DoWork(const void* Hash, FThreadSafeCounter& Counter, FThreadSafeCounter& Cycles, int32 Work)
	{
		if (Work > 0)
		{
			uint32 CyclesStart = FPlatformTime::Cycles();
			Counter.Increment();
			int32 Sum = 0;
			for (int32 Index = 0; Index < Work; Index++)
			{
				Sum += PointerHash(((const uint64*)Hash) + Index);
			}
			Cycles.Add(FPlatformTime::Cycles() - CyclesStart + (Sum & 1));
		}
		else if (Work == 0)
		{
			Counter.Increment();
		}
	}

	void PrintResult(double& StartTime, double& QueueTime, double& EndTime, FThreadSafeCounter& Counter, FThreadSafeCounter& Cycles, const TCHAR* Message)
	{
		UE_LOGF(LogConsoleResponse, Display, "Total %6.3fms   %6.3fms queue   %6.3fms wait   %6.3fms work   : %ls"
			, float(1000.0 * (EndTime - StartTime)), float(1000.0 * (QueueTime - StartTime)), float(1000.0 * (EndTime - QueueTime)), float(FPlatformTime::GetSecondsPerCycle() * double(Cycles.GetValue()) * 1000.0)
			, Message
		);

		Counter.Reset();
		Cycles.Reset();
		StartTime = 0.0;
		QueueTime = 0.0;
		EndTime = 0.0;
	}

	static void TaskGraphBenchmark(const TArray<FString>& Args)
	{
		FSlowHeartBeatScope SuspendHeartBeat;

		double StartTime, QueueTime, EndTime;
		FThreadSafeCounter Counter;
		FThreadSafeCounter Cycles;

		if (!FTaskGraphInterface::IsMultithread())
		{
			UE_LOGF(LogConsoleResponse, Display, "WARNING: TaskGraphBenchmark disabled for non multi-threading platforms");
			return;
		}

		if (Args.Num() == 1 && Args[0] == TEXT("infinite"))
		{
			while (true)
			{
				{
					StartTime = FPlatformTime::Seconds();

					ParallelFor(1000,
						[&Counter, &Cycles](int32 Index)
						{
							FFunctionGraphTask::CreateAndDispatchWhenReady(
								[&Counter, &Cycles](ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionEvent)
								{
									DoWork(&CompletionEvent, Counter, Cycles, -1);
								},
								TStatId{}, nullptr, ENamedThreads::GameThread_Local
							);
						}
					);
					QueueTime = FPlatformTime::Seconds();
					FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread_Local);
					EndTime = FPlatformTime::Seconds();
				}
			}
		}
		{
			StartTime = FPlatformTime::Seconds();
			FGraphEventArray Tasks;
			Tasks.Reserve(1000);
			for (int32 Index = 0; Index < 1000; Index++)
			{
				Tasks.Add(FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, nullptr, ENamedThreads::GameThread_Local));
			}
			QueueTime = FPlatformTime::Seconds();
			FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(Tasks), ENamedThreads::GameThread_Local);
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 tasks, ordinary local GT start"));
		{
			StartTime = FPlatformTime::Seconds();
			FGraphEventArray Tasks;
			Tasks.Reserve(1000);
			for (int32 Index = 0; Index < 1000; Index++)
			{
				Tasks.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(
					[&Counter, &Cycles](ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionEvent)
					{
						DoWork(&CompletionEvent, Counter, Cycles, 100);
					},
					TStatId{}, nullptr, ENamedThreads::GameThread_Local
				));
			}
			QueueTime = FPlatformTime::Seconds();
			FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(Tasks), ENamedThreads::GameThread_Local);
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 tasks, ordinary local GT start, with work"));
		{
			StartTime = FPlatformTime::Seconds();
			FGraphEventArray Tasks;
			Tasks.AddZeroed(1000);

			ParallelFor(1000,
				[&Tasks](int32 Index)
				{
					Tasks[Index] = FFunctionGraphTask::CreateAndDispatchWhenReady([] {});
				}
			);
			QueueTime = FPlatformTime::Seconds();
			FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(Tasks), ENamedThreads::GameThread);
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 tasks, ParallelFor start"));
		{
			StartTime = FPlatformTime::Seconds();
			FGraphEventArray Tasks;
			Tasks.AddZeroed(10);

			ParallelFor(10,
				[&Tasks](int32 Index)
				{
					FGraphEventArray InnerTasks;
					InnerTasks.AddZeroed(100);
					for (int32 InnerIndex = 0; InnerIndex < 100; InnerIndex++)
					{
						InnerTasks[InnerIndex] = FFunctionGraphTask::CreateAndDispatchWhenReady([] {});
					}
					// join the above tasks
					Tasks[Index] = FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, &InnerTasks, ENamedThreads::AnyThread);
				}
			);
			QueueTime = FPlatformTime::Seconds();
			FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(Tasks), ENamedThreads::GameThread);
			EndTime = FPlatformTime::Seconds();
			PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 tasks, ParallelFor start, batched completion 10x100"));
		}

		{
			StartTime = FPlatformTime::Seconds();
			FGraphEventArray Tasks;
			Tasks.AddZeroed(100);

			ParallelFor(100,
				[&Tasks](int32 Index)
				{
					FGraphEventArray InnerTasks;
					InnerTasks.AddZeroed(10);
					for (int32 InnerIndex = 0; InnerIndex < 10; InnerIndex++)
					{
						InnerTasks[InnerIndex] = FFunctionGraphTask::CreateAndDispatchWhenReady([] {});
					}
					// join the above tasks
					Tasks[Index] = FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, &InnerTasks, ENamedThreads::AnyThread);
				}
			);
			QueueTime = FPlatformTime::Seconds();
			FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(Tasks), ENamedThreads::GameThread);
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 tasks, ParallelFor start, batched completion 100x10"));

		{
			StartTime = FPlatformTime::Seconds();

			ParallelFor(1000,
				[&Counter, &Cycles](int32 Index)
				{
					FFunctionGraphTask::CreateAndDispatchWhenReady([&Counter, &Cycles](ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionEvent) { DoWork(&CompletionEvent, Counter, Cycles, 0); });
				}
			);
			QueueTime = FPlatformTime::Seconds();
			while (Counter.GetValue() < 1000)
			{
				FPlatformMisc::MemoryBarrier();
			}
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 tasks, ParallelFor, counter tracking"));

		{
			StartTime = FPlatformTime::Seconds();

			static std::atomic<bool> Output[1000];
			FPlatformMemory::Memzero(Output, 1000);

			ParallelFor(1000,
				[](int32 Index)
				{
					std::atomic<bool>& Out = Output[Index];
					FFunctionGraphTask::CreateAndDispatchWhenReady([&Out] { Out = true; });
				}
			);
			QueueTime = FPlatformTime::Seconds();
			for (int32 Index = 0; Index < 1000; Index++)
			{
				while (!Output[Index])
				{
					FPlatformProcess::Yield();
				}
			}
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 tasks, ParallelFor, bool* tracking"));

		{
			StartTime = FPlatformTime::Seconds();

			ParallelFor(1000,
				[&Counter, &Cycles](int32 Index)
				{
					FFunctionGraphTask::CreateAndDispatchWhenReady([&Counter, &Cycles](ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionEvent) { DoWork(&CompletionEvent, Counter, Cycles, 1000); });
				}
			);
			QueueTime = FPlatformTime::Seconds();
			while (Counter.GetValue() < 1000)
			{
				FPlatformProcess::Yield();
			}
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 tasks, ParallelFor, counter tracking, with work"));
		{
			StartTime = FPlatformTime::Seconds();
			for (int32 Index = 0; Index < 1000; Index++)
			{
				FFunctionGraphTask::CreateAndDispatchWhenReady([&Counter, &Cycles](ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionEvent) { DoWork(&CompletionEvent, Counter, Cycles, 1000); });
			}
			QueueTime = FPlatformTime::Seconds();
			while (Counter.GetValue() < 1000)
			{
				FPlatformProcess::Yield();
			}
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 tasks, GT submit, counter tracking, with work"));
		{
			StartTime = FPlatformTime::Seconds();

			ParallelFor(1000,
				[&Counter, &Cycles](int32 Index)
				{
					FFunctionGraphTask::CreateAndDispatchWhenReady(
						[&Counter, &Cycles](ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionEvent)
						{
							DoWork(&CompletionEvent, Counter, Cycles, -1);
						},
						TStatId{}, nullptr, ENamedThreads::GameThread_Local
					);
				}
			);
			QueueTime = FPlatformTime::Seconds();
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread_Local);
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 local GT tasks, ParallelFor, no tracking (none needed)"));

		{
			StartTime = FPlatformTime::Seconds();
			QueueTime = StartTime;
			ParallelFor(1000,
				[&Counter, &Cycles](int32 Index)
				{
					DoWork(&Counter, Counter, Cycles, -1);
				}
			);
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 element do-nothing ParallelFor"));
		{
			StartTime = FPlatformTime::Seconds();
			QueueTime = StartTime;
			ParallelFor(1000,
				[&Counter, &Cycles](int32 Index)
				{
					DoWork(&Counter, Counter, Cycles, 1000);
				}
			);
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 element ParallelFor, with work"));

		{
			StartTime = FPlatformTime::Seconds();
			QueueTime = StartTime;
			ParallelFor(1000,
				[&Counter, &Cycles](int32 Index)
				{
					DoWork(&Counter, Counter, Cycles, 1000);
				},
				true
			);
			EndTime = FPlatformTime::Seconds();
		}
		PrintResult(StartTime, QueueTime, EndTime, Counter, Cycles, TEXT("1000 element ParallelFor, single threaded, with work"));
	}

	static FAutoConsoleCommand TaskGraphBenchmarkCmd(
		TEXT("TaskGraph.Benchmark"),
		TEXT("Prints the time to run 1000 no-op tasks."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&TaskGraphBenchmark)
	);

	struct FTestStruct
	{
		int32 Index;
		int32 Constant;
		FTestStruct(int32 InIndex)
			: Index(InIndex)
			, Constant(0xfe05abcd)
		{
		}
	};

	struct FTestRigFIFO
	{
		FLockFreePointerFIFOBase<FTestStruct, PLATFORM_CACHE_LINE_SIZE> Test1;
		FLockFreePointerFIFOBase<FTestStruct, 8> Test2;
		FLockFreePointerFIFOBase<FTestStruct, 8, 1 << 4> Test3;
	};

	struct FTestRigLIFO
	{
		FLockFreePointerListLIFOBase<FTestStruct, PLATFORM_CACHE_LINE_SIZE> Test1;
		FLockFreePointerListLIFOBase<FTestStruct, 8> Test2;
		FLockFreePointerListLIFOBase<FTestStruct, 8, 1 << 4> Test3;
	};

	static void TestLockFree(int32 OuterIters = 3)
	{
		FSlowHeartBeatScope SuspendHeartBeat;


		if (!FTaskGraphInterface::IsMultithread())
		{
			UE_LOGF(LogConsoleResponse, Display, "WARNING: TestLockFree disabled for non multi-threading platforms");
			return;
		}

		const int32 NumWorkers = FTaskGraphInterface::Get().GetNumWorkerThreads();
		// If we have too many threads active at once, they become too slow due to contention.  Set a reasonable maximum for how many are required to guarantee correctness of our LockFreePointers.
		const int32 MaxWorkersForTest = 5;
		const int32 MinWorkersForTest = 2; // With less than two threads we're not testing threading at all, so the test is pointless.
		if (NumWorkers < MinWorkersForTest)
		{
			UE_LOGF(LogConsoleResponse, Display, "WARNING: TestLockFree disabled for current machine because of not enough worker threads.  Need %d, have %d.", MinWorkersForTest, NumWorkers);
			return;
		}

		FScopedDurationTimeLogger DurationLogger(TEXT("TestLockFree Runtime"));
		const uint32 NumWorkersForTest = static_cast<uint32>(FMath::Clamp(NumWorkers, MinWorkersForTest, MaxWorkersForTest));
		auto RunWorkersSynchronous = [NumWorkersForTest](const TFunction<void(uint32)>& WorkerTask)
		{
			FGraphEventArray Tasks;
			for (uint32 Index = 0; Index < NumWorkersForTest; Index++)
			{
				TUniqueFunction<void()> WorkerTaskWithIndex{ [Index, &WorkerTask] { WorkerTask(Index); } };
				Tasks.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(WorkerTaskWithIndex), TStatId{}, nullptr, ENamedThreads::AnyNormalThreadHiPriTask));
			}
			FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(Tasks));
		};

		for (int32 Iter = 0; Iter < OuterIters; Iter++)
		{
			{
				UE_LOGF(LogTemp, Display, "******************************* Iter FIFO %d", Iter);
				FTestRigFIFO Rig;
				for (int32 Index = 0; Index < 1000; Index++)
				{
					Rig.Test1.Push(new FTestStruct(Index));
				}
				TFunction<void(uint32)> Broadcast =
					[&Rig](uint32 WorkerIndex)
				{
					FRandomStream Stream(((int32)WorkerIndex) * 7 + 13);
					for (int32 Index = 0; Index < 1000000; Index++)
					{
						if (Index % 200000 == 1)
						{
							//UE_LOGF(LogTemp, Log, "%8d iters thread=%d", Index, int32(WorkerIndex));
						}
						if (Stream.FRand() < .03f)
						{
							TArray<FTestStruct*> Items;
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.PopAll(Items);
								}
								else if (r < .66f)
								{
									Rig.Test2.PopAll(Items);
								}
								else
								{
									Rig.Test3.PopAll(Items);
								}
							}
							for (FTestStruct* Item : Items)
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.Push(Item);
								}
								else if (r < .66f)
								{
									Rig.Test2.Push(Item);
								}
								else
								{
									Rig.Test3.Push(Item);
								}
							}
						}
						else
						{
							FTestStruct* Item;
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Item = Rig.Test1.Pop();
								}
								else if (r < .66f)
								{
									Item = Rig.Test2.Pop();
								}
								else
								{
									Item = Rig.Test3.Pop();
								}
							}
							if (Item)
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.Push(Item);
								}
								else if (r < .66f)
								{
									Rig.Test2.Push(Item);
								}
								else
								{
									Rig.Test3.Push(Item);
								}
							}
						}
					}
				};
				RunWorkersSynchronous(Broadcast);

				TArray<FTestStruct*> Items;
				Rig.Test1.PopAll(Items);
				Rig.Test2.PopAll(Items);
				Rig.Test3.PopAll(Items);

				checkf(Items.Num() == 1000, TEXT("Items %d"), Items.Num());

				for (int32 LookFor = 0; LookFor < 1000; LookFor++)
				{
					bool bFound = false;
					for (int32 Index = 0; Index < 1000; Index++)
					{
						if (Items[Index]->Index == LookFor && Items[Index]->Constant == 0xfe05abcd)
						{
							check(!bFound);
							bFound = true;
						}
					}
					check(bFound);
				}
				for (FTestStruct* Item : Items)
				{
					delete Item;
				}

				UE_LOGF(LogTemp, Display, "******************************* Pass FTestRigFIFO");

			}
			{
				UE_LOGF(LogTemp, Display, "******************************* Iter LIFO %d", Iter);
				FTestRigLIFO Rig;
				for (int32 Index = 0; Index < 1000; Index++)
				{
					Rig.Test1.Push(new FTestStruct(Index));
				}
				TFunction<void(uint32)> Broadcast =
					[&Rig](uint32 WorkerIndex)
				{
					FRandomStream Stream(((int32)WorkerIndex) * 7 + 13);
					for (int32 Index = 0; Index < 1000000; Index++)
					{
						if (Index % 200000 == 1)
						{
							//UE_LOGF(LogTemp, Log, "%8d iters thread=%d", Index, int32(WorkerIndex));
						}
						if (Stream.FRand() < .03f)
						{
							TArray<FTestStruct*> Items;
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.PopAll(Items);
								}
								else if (r < .66f)
								{
									Rig.Test2.PopAll(Items);
								}
								else
								{
									Rig.Test3.PopAll(Items);
								}
							}
							for (FTestStruct* Item : Items)
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.Push(Item);
								}
								else if (r < .66f)
								{
									Rig.Test2.Push(Item);
								}
								else
								{
									Rig.Test3.Push(Item);
								}
							}
						}
						else
						{
							FTestStruct* Item;
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Item = Rig.Test1.Pop();
								}
								else if (r < .66f)
								{
									Item = Rig.Test2.Pop();
								}
								else
								{
									Item = Rig.Test3.Pop();
								}
							}
							if (Item)
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.Push(Item);
								}
								else if (r < .66f)
								{
									Rig.Test2.Push(Item);
								}
								else
								{
									Rig.Test3.Push(Item);
								}
							}
						}
					}
				};
				RunWorkersSynchronous(Broadcast);

				TArray<FTestStruct*> Items;
				Rig.Test1.PopAll(Items);
				Rig.Test2.PopAll(Items);
				Rig.Test3.PopAll(Items);

				checkf(Items.Num() == 1000, TEXT("Items %d"), Items.Num());

				for (int32 LookFor = 0; LookFor < 1000; LookFor++)
				{
					bool bFound = false;
					for (int32 Index = 0; Index < 1000; Index++)
					{
						if (Items[Index]->Index == LookFor && Items[Index]->Constant == 0xfe05abcd)
						{
							check(!bFound);
							bFound = true;
						}
					}
					check(bFound);
				}
				for (FTestStruct* Item : Items)
				{
					delete Item;
				}

				UE_LOGF(LogTemp, Display, "******************************* Pass FTestRigLIFO");

			}
		}
	}

	static void TestLockFree(const TArray<FString>& Args)
	{
		TestLockFree(10);
	}

	static FAutoConsoleCommand TestLockFreeCmd(
		TEXT("TaskGraph.TestLockFree"),
		TEXT("Test lock free lists"),
		FConsoleCommandWithArgsDelegate::CreateStatic(&TestLockFree)
	);

	static void TestLowToHighPri(const TArray<FString>& Args)
	{
		UE_LOGF(LogTemp, Display, "Starting latency test....");

		auto ForegroundTask = [](uint64 StartCycles)
		{
			float Latency = float(double(FPlatformTime::Cycles64() - StartCycles) * FPlatformTime::GetSecondsPerCycle64() * 1000.0 * 1000.0);
			//UE_LOGF(LogTemp, Display, "Latency %6.2fus", Latency);
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Latency %6.2fus\r\n"), Latency);
		};

		auto BackgroundTask = [&ForegroundTask](ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionEvent)
		{
			while (true)
			{
				uint32 RunningCrc = 0;
				for (int32 Index = 0; Index < 1000000; Index++)
				{
					RunningCrc = FCrc::MemCrc32(CompletionEvent.GetReference(), sizeof(FGraphEvent), RunningCrc);
				}
				uint64 StartTime = FPlatformTime::Cycles64();
				FFunctionGraphTask::CreateAndDispatchWhenReady([StartTime, &ForegroundTask] { ForegroundTask(StartTime); }, TStatId{}, nullptr, ENamedThreads::AnyHiPriThreadHiPriTask);
			}
		};

#if 0
		const int NumBackgroundTasks = 32;
		const int NumNormalTasks = 32;
		for (int32 Index = 0; Index < NumBackgroundTasks; Index++)
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady(BackgroundTask, TStatId{}, nullptr, ENamedThreads::AnyNormalThreadNormalTask);
		}
		for (int32 Index = 0; Index < NumNormalTasks; Index++)
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady(BackgroundTask, TStatId{}, nullptr, ENamedThreads::AnyBackgroundThreadNormalTask);
		}
		while (true)
		{
			FPlatformProcess::Sleep(25.0f);
		}
#else
		FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(BackgroundTask), TStatId{}, nullptr, ENamedThreads::AnyBackgroundThreadNormalTask);
#endif
	}

	static FAutoConsoleCommand TestLowToHighPriCmd(
		TEXT("TaskGraph.TestLowToHighPri"),
		TEXT("Test latency of high priority tasks when low priority tasks are saturating the CPU"),
		FConsoleCommandWithArgsDelegate::CreateStatic(&TestLowToHighPri)
	);


	TEST_CASE_NAMED(FTaskGraphOldBenchmark, "System::Core::Async::TaskGraph::OldBenchmark", "[EditorContext][ClientContext][ServerContext][EngineFilter]")
	{
		TArray<FString> Args;
		TaskGraphBenchmark(Args);
	}

	TEST_CASE_NAMED(FLockFreeTest, "System::Core::Async::TaskGraph::LockFree", "[.][ApplicationContextMask][EngineFilter]")
	{
		TestLockFree(3);
	}
}

extern int32 GNumForegroundWorkers;

namespace TaskGraphTests
{
	TEST_CASE_NAMED(FTaskGraphGraphEventTest, "System::Core::Async::TaskGraph::GraphEventTest", "[.][ApplicationContextMask][EngineFilter]")
	{
		{	// task completes before it's waited for
			FGraphEventRef Event = FFunctionGraphTask::CreateAndDispatchWhenReady(
				[]
				{
					//UE_LOGF(LogTemp, Log, "Main task");
				}
			);
			while (!Event->IsComplete() && FTaskGraphInterface::Get().GetNumWorkerThreads() != 0) // in single-threaded mode tasks are executed only when waited for
			{}
			Event->Wait(ENamedThreads::GameThread);
		}

		{	// task completes after it's waited for
			FGraphEventRef Event = FFunctionGraphTask::CreateAndDispatchWhenReady([]()
				{
					//UE_LOGF(LogTemp, Log, "Main task");
					FPlatformProcess::Sleep(0.1f); // pause for a bit to let waiting start
				}
			);
			check(!Event->IsComplete());
			Event->Wait(ENamedThreads::GameThread);
		}

		{	// event w/o a task, signaled by explicit call to DispatchSubsequents before it's waited for
			FGraphEventRef Event = FGraphEvent::CreateGraphEvent();
			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[&Event]
				{
					Event->DispatchSubsequents();
				}
			);
			while (!Event->IsComplete() && FTaskGraphInterface::Get().GetNumWorkerThreads() != 0) // in single-threaded mode tasks are executed only when waited for
			{}
			Event->Wait(ENamedThreads::GameThread);
		}

		{	// event w/o a task, signaled by explicit call to DispatchSubsequents after it's waited for
			FGraphEventRef Event = FGraphEvent::CreateGraphEvent();
			auto Lambda = [&Event]
				{
					FPlatformProcess::Sleep(0.1f); // pause for a bit to let waiting start
					Event->DispatchSubsequents();
			};
			FGraphEventRef Task = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(Lambda));
			check(!Event->IsComplete());
			Event->Wait();
			Task->Wait();
		}

		{	// wait for prereq by DontCompleteUntil
			FGraphEventRef Blocker = FGraphEvent::CreateGraphEvent();
			auto Lambda = [&Blocker](ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
			{
				MyCompletionGraphEvent->DontCompleteUntil(Blocker);
			};

			FGraphEventRef Task = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(Lambda));
			FPlatformProcess::Sleep(0.01f);
			check(!Task->IsComplete());
			Blocker->DispatchSubsequents();
			Task->Wait(ENamedThreads::GameThread);
		}

		{	// prereq is completed before DontCompleteUntil is called
			FGraphEventRef Prereq = FFunctionGraphTask::CreateAndDispatchWhenReady(
				[]
				{
					//UE_LOGF(LogTemp, Log, "Prereq");
				}
			);
			Prereq->SetDebugName(TEXT("Prereq"));
			Prereq->Wait(ENamedThreads::GameThread);

			FGraphEventRef Event = FFunctionGraphTask::CreateAndDispatchWhenReady(
				[&Prereq](ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
				{
					MyCompletionGraphEvent->DontCompleteUntil(Prereq);
					//UE_LOGF(LogTemp, Log, "Main task");
				}
			);
			Event->SetDebugName(TEXT("MainEvent"));
			while (!Event->IsComplete() && FTaskGraphInterface::Get().GetNumWorkerThreads() != 0) // in single-threaded mode tasks are executed only when waited for
			{}
			Event->Wait(ENamedThreads::GameThread);
		}
	}

	TEST_CASE_NAMED(FTaskGraphBasicTest, "System::Core::Async::TaskGraph::BasicTest", "[.][ApplicationContextMask][EngineFilter]")
	{
		// thread and task priorities

		{	// AnyNormalThreadNormalTask
			bool bExecuted = false;
			FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, nullptr, ENamedThreads::AnyNormalThreadNormalTask)->Wait();
			check(bExecuted);
		}

		{	// AnyNormalThreadHiPriTask
			bool bExecuted = false;
			FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, nullptr, ENamedThreads::AnyNormalThreadHiPriTask)->Wait();
			check(bExecuted);
		}

		{	// AnyBackgroundThreadNormalTask
			bool bExecuted = false;
			FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, nullptr, ENamedThreads::AnyBackgroundThreadNormalTask)->Wait();
			check(bExecuted);
		}

		{	// AnyHiPriThreadNormalTask
			bool bExecuted = false;
			FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, nullptr, ENamedThreads::AnyHiPriThreadNormalTask)->Wait();
			check(bExecuted);
		}

		{	// AnyHiPriThreadHiPriTask
			bool bExecuted = false;
			FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, nullptr, ENamedThreads::AnyHiPriThreadHiPriTask)->Wait();
			check(bExecuted);
		}

		// named threads and local queues

		if (IsRHIThreadRunning())
		{	// RHIThread
			bool bExecuted = false;
			FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, nullptr, ENamedThreads::RHIThread)->Wait();
			check(bExecuted);
		}

		{	// GameThread
			bool bExecuted = false;
			FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, nullptr, ENamedThreads::GameThread);
			// GT must be executed explicitly
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			check(bExecuted);
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (GRenderThreadId != 0)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{	// ActualRenderingThread
			bool bExecuted = false;
			FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, nullptr, ENamedThreads::ActualRenderingThread)->Wait();
			check(bExecuted);
		}

		{	// GameThread_Local
			bool bExecuted = false;
			FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, nullptr, ENamedThreads::GameThread_Local)->Wait(ENamedThreads::GameThread_Local);
			check(bExecuted);
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (GRenderThreadId != 0)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{	// ActualRenderingThread_Local
			bool bExecuted = false;
			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[&bExecuted] 
				{ 
					FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, nullptr, ENamedThreads::ActualRenderingThread_Local)->Wait(ENamedThreads::ActualRenderingThread_Local);
				}, 
				TStatId{}, nullptr, ENamedThreads::ActualRenderingThread)->Wait();
			check(bExecuted);
		}

		// dependencies

		{	// a task is not executed until its prerequisite is completed
			bool bExecuted = false;
			FGraphEventRef Prereq = FGraphEvent::CreateGraphEvent();
			FGraphEventRef MainTask = FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, Prereq);
			// dummy task that is executed while the main task is waiting for its prereq
			FFunctionGraphTask::CreateAndDispatchWhenReady([] {})->Wait();
			check(!bExecuted);
			Prereq->DispatchSubsequents();
			MainTask->Wait();
			check(bExecuted);
		}

		{	// a task is not executed until all its prerequisites are completed
			bool bExecuted = false;
			FGraphEventArray Prereqs{ FGraphEvent::CreateGraphEvent(), FGraphEvent::CreateGraphEvent() };
			FGraphEventRef MainTask = FFunctionGraphTask::CreateAndDispatchWhenReady([&bExecuted] { bExecuted = true; }, TStatId{}, &Prereqs);
			// dummy task that is executed while the main task is waiting for its prereqs
			FFunctionGraphTask::CreateAndDispatchWhenReady([] {})->Wait();
			check(!bExecuted);

			Prereqs[0]->DispatchSubsequents();
			FFunctionGraphTask::CreateAndDispatchWhenReady([] {})->Wait();
			check(!bExecuted);
			
			Prereqs[1]->DispatchSubsequents();
			MainTask->Wait();
			check(bExecuted);
		}

		{	// ParallelFor
			std::atomic<int32> Total{ 0 };
			int32 Num = 1000;
			ParallelFor(Num, [&Total](int32 i) { Total += i; });
			check(Total == (Num - 1) * (Num / 2));
		}

		{	// holding a task
			struct FTask
			{
				TStatId GetStatId() const
				{
					return TStatId{};
				}

				ENamedThreads::Type GetDesiredThread()
				{
					return ENamedThreads::AnyThread;
				}

				static ESubsequentsMode::Type GetSubsequentsMode()
				{
					return ESubsequentsMode::TrackSubsequents;
				}

				void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
				{}
			};

			TGraphTask<FTask>* Task = TGraphTask<FTask>::CreateTask().ConstructAndHold();
			FGraphEventRef Event = Task->GetCompletionEvent();
			check(!Event->IsComplete());
			Task->Unlock();
			Event->Wait();
			check(Event->IsComplete());
		}

		{	// check ref count for named thread tasks
			FGraphEventRef LocalQueueTask = FFunctionGraphTask::CreateAndDispatchWhenReady(
				[] 
				{
					check(IsInGameThread());  
				}, TStatId{}, nullptr, ENamedThreads::GameThread_Local);

			LocalQueueTask->Wait(ENamedThreads::GameThread_Local);
			check(LocalQueueTask->IsComplete());
			check(LocalQueueTask.GetRefCount() >= 1);
			// Now that the wait above can perform retraction, it will not necessarily pump the game thread queue and will just leave an empty task with a refcount on it
			// so pump it ourself to verify that once the game thread queue is empty, the refcount should have been cleaned up as well.
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread_Local);
			check(LocalQueueTask.GetRefCount() == 1);
		}

		//for (int i = 0; i != 100000; ++i)
		{	// a particular real-life case that doesn't work in the old TaskGraph if run in single-threaded mode.
			// the culprit is that when a task is waited for, in single-threaded mode the queue it was pushed to is executed. 
			// Here the (local queue) task  depends on a task that is not in the same queue and so it doesn't get executed
			std::atomic<bool> bWorkerDone = false;
			bool bHasRun = false;
			FGraphEventRef AnyTask = FFunctionGraphTask::CreateAndDispatchWhenReady([&bWorkerDone] { bWorkerDone = true; });
			FGraphEventRef LocalQueueTask = FFunctionGraphTask::CreateAndDispatchWhenReady(
				[&bHasRun]
				{
					check(bHasRun == false);
					bHasRun = true;
					check(IsInGameThread());
				}, TStatId{}, AnyTask, ENamedThreads::GameThread_Local);

			check(bHasRun == false);
			LocalQueueTask->Wait(ENamedThreads::GameThread_Local);
			check(bWorkerDone);
			check(bHasRun);
			check(LocalQueueTask->IsComplete());
			check(LocalQueueTask.GetRefCount() >= 1);
			// Now that the wait above can perform retraction, it will not necessarily pump the game thread queue and will just leave an empty task with a refcount on it
			// so pump it ourself to verify that once the game thread queue is empty, the refcount should have been cleaned up as well.
			// The reason we do it in a loop, is that the task on the game thread could be queued after the wait returns from a successful retraction.
			while (LocalQueueTask.GetRefCount() > 1)
			{
				FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread_Local);
			}
			check(LocalQueueTask.GetRefCount() == 1);
		}

		{	// launch a GT task, then an any-thread task that depends on it. wait for the any-thread task. this was a deadlock on the new frontend
			FGraphEventRef GTTask = FFunctionGraphTask::CreateAndDispatchWhenReady([] { check(IsInGameThread()); }, TStatId{}, nullptr, ENamedThreads::GameThread);
			FGraphEventRef AnyThreadTask = FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, GTTask);
			AnyThreadTask->Wait();
		}
	}
	
	// it's fast because tasks are too lightweight and so are executed almost as fast
	template<int NumTasks>
	void TestPerfBasic()
	{
		FGraphEventArray Tasks;
		Tasks.Reserve(NumTasks);

		for (int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			Tasks.Emplace(FFunctionGraphTask::CreateAndDispatchWhenReady([] {}));
		}

		FTaskGraphInterface::Get().WaitUntilTasksComplete(Tasks);
	}

	template<int NumTasks>
	void TestPerfChaining()
	{
		FGraphEventRef Ref;
		for (int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			if (Ref.IsValid())
			{
				Ref = FFunctionGraphTask::CreateAndDispatchWhenReady([&]() {}, TStatId{}, Ref);
			}
			else
			{
				Ref = FFunctionGraphTask::CreateAndDispatchWhenReady([&]() {});
			}
		}

		FTaskGraphInterface::Get().WaitUntilTaskCompletes(Ref);
	}

	template<int32 NumTasks, int32 BatchSize>
	void TestPerfBatch()
	{
		static_assert(NumTasks % BatchSize == 0, "`NumTasks` must be divisible by `BatchSize`");
		constexpr int32 NumBatches = NumTasks / BatchSize;

		FGraphEventArray Batches;
		Batches.Reserve(NumBatches);
		FGraphEventArray Tasks;
		Tasks.AddDefaulted(NumTasks);

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			Batches.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(
				[&Tasks, BatchIndex]
				{
					for (int32 TaskIndex = 0; TaskIndex < BatchSize; ++TaskIndex)
					{
						Tasks[BatchIndex * BatchSize + TaskIndex] = FFunctionGraphTask::CreateAndDispatchWhenReady([] {});
					}
				}
			));
		}

		FTaskGraphInterface::Get().WaitUntilTasksComplete(Batches);
		FTaskGraphInterface::Get().WaitUntilTasksComplete(Tasks);
	}

	template<int32 NumTasks, int32 BatchSize>
	void TestPerfBatchOptimised()
	{
		static_assert(NumTasks % BatchSize == 0, "`NumTasks` must be divisible by `BatchSize`");
		constexpr int32 NumBatches = NumTasks / BatchSize;

		FGraphEventRef SpawnSignal = FGraphEvent::CreateGraphEvent();
		FGraphEventArray AllDone;

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			AllDone.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(
				[](ENamedThreads::Type CurrentThread, const FGraphEventRef& CompletionEvent)
				{
					FGraphEventRef RunSignal = FGraphEvent::CreateGraphEvent();
					for (int32 TaskIndex = 0; TaskIndex < BatchSize; ++TaskIndex)
					{
						CompletionEvent->DontCompleteUntil(FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, RunSignal, ENamedThreads::AnyThread));
					}
					RunSignal->DispatchSubsequents();
				},
				TStatId{}, SpawnSignal
			));
		}

		SpawnSignal->DispatchSubsequents();
		FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(AllDone));
	}

	template<int NumTasks>
	void TestLatency()
	{
		for (uint32 TaskIndex = 0; TaskIndex != NumTasks; ++TaskIndex)
		{
			FGraphEventRef GraphEvent = FFunctionGraphTask::CreateAndDispatchWhenReady([] {});
			GraphEvent->Wait(ENamedThreads::GameThread);
		}
	}

	int64 Fibonacci(int64 N)
	{
		check(N > 0);
		if (N <= 2)
		{
			return 1;
		}
		else
		{
			std::atomic<int64> F1{ -1 };
			std::atomic<int64> F2{ -1 };
			FGraphEventArray GraphEvents;
			GraphEvents.Add(FFunctionGraphTask::CreateAndDispatchWhenReady([&F1, N] { F1 = Fibonacci(N - 1); }));
			GraphEvents.Add(FFunctionGraphTask::CreateAndDispatchWhenReady([&F2, N] { F2 = Fibonacci(N - 2); }));

			FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(GraphEvents));
			check(F1 > 0 && F2 > 0);

			return F1 + F2;
		}
	}

	FGraphEventRef Fib(int64 N, int64* Res)
	{
		if (N <= 2)
		{
			*Res = 1;
			FGraphEventRef ResEvent = FGraphEvent::CreateGraphEvent();
			ResEvent->DispatchSubsequents();
			return ResEvent;
		}
		else
		{
			TUniquePtr<int64> F1 = MakeUnique<int64>();
			TUniquePtr<int64> F2 = MakeUnique<int64>();

			FGraphEventArray SubTasks;

			auto FibTask = [](int64 N, int64* Res)
			{
				return FFunctionGraphTask::CreateAndDispatchWhenReady
				(
					[N, Res]
					(ENamedThreads::Type, const FGraphEventRef& CompletionEvent)
					{
						FGraphEventRef ResEvent = Fib(N, Res);
						CompletionEvent->DontCompleteUntil(ResEvent);
					}
				);
			};

			SubTasks.Add(FibTask(N - 1, F1.Get()));
			SubTasks.Add(FibTask(N - 2, F2.Get()));

			FGraphEventRef ResEvent = FFunctionGraphTask::CreateAndDispatchWhenReady
			(
				[F1 = MoveTemp(F1), F2 = MoveTemp(F2), Res]
				{
					*Res = *F1 + *F2;
				}, 
				TStatId{}, &SubTasks
			);

			return ResEvent;
		}
	}

	template<int64 N>
	void Fib()
	{
		TUniquePtr<int64> Res = MakeUnique<int64>();
		FGraphEventRef ResEvent = Fib(N, Res.Get());
		ResEvent->Wait();
		UE_LOGF(LogTemp, Display, "Fibonacci(%d) = %d", N, *Res);
	}

	template<uint32 NumTasks>
	void TestFGraphEventPerf()
	{
		FGraphEventRef Prereq = FGraphEvent::CreateGraphEvent();
		std::atomic<uint32> CompletedTasks{ 0 };

		FGraphEventArray Tasks;
		for (int i = 0; i != NumTasks; ++i)
		{
			Tasks.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(
				[&Prereq, &CompletedTasks](ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
				{
					MyCompletionGraphEvent->DontCompleteUntil(Prereq);
					++CompletedTasks;
				}
			));
		}

		Prereq->DispatchSubsequents(ENamedThreads::GameThread);

		FTaskGraphInterface::Get().WaitUntilTasksComplete(Tasks, ENamedThreads::GameThread);

		check(CompletedTasks == NumTasks);
	}

	template<int NumTasks>
	void TestSpawning()
	{
		{
			FGraphEventArray Tasks;
			Tasks.Reserve(NumTasks);
			//double StartTime = FPlatformTime::Seconds();
			for (uint32 TaskNo = 0; TaskNo != NumTasks; ++TaskNo)
			{
				Tasks.Add(FFunctionGraphTask::CreateAndDispatchWhenReady([] {}));
			}

			//double Duration = FPlatformTime::Seconds() - StartTime;
			//UE_LOGF(LogTemp, Display, "Spawning %d empty trackable tasks took %f secs", NumTasks, Duration);

			FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(Tasks));
		}
		{
			double StartTime = FPlatformTime::Seconds();
			for (uint32 TaskNo = 0; TaskNo != NumTasks; ++TaskNo)
			{
				FFunctionGraphTask::CreateAndDispatchWhenReady([] {});
			}

			//double Duration = FPlatformTime::Seconds() - StartTime;
			//UE_LOGF(LogTemp, Display, "Spawning %d empty non-trackable tasks took %f secs", NumTasks, Duration);
		}
	}

	template<int NumTasks>
	void TestBatchSpawning()
	{
		//double StartTime = FPlatformTime::Seconds();
		FGraphEventRef Trigger = FGraphEvent::CreateGraphEvent();
		for (uint32 TaskNo = 0; TaskNo != NumTasks; ++TaskNo)
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, Trigger);
		}

		//double SpawnedTime = FPlatformTime::Seconds();
		Trigger->DispatchSubsequents();

		//double EndTime = FPlatformTime::Seconds();
		//UE_LOGF(LogTemp, Display, "Spawning %d empty non-trackable tasks took %f secs total, %f secs spawning and %f secs dispatching", NumTasks, EndTime - StartTime, SpawnedTime - StartTime, EndTime - SpawnedTime);

		//FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(Tasks));
	}

	template<int64 NumBatches, int64 NumTasksPerBatch>
	void TestWorkStealing()
	{
		FGraphEventArray Batches;
		Batches.Reserve(NumBatches);

		FGraphEventArray Tasks[NumBatches];
		for (int32 BatchIndex = 0; BatchIndex != NumBatches; ++BatchIndex)
		{
			Tasks[BatchIndex].Reserve(NumBatches * NumTasksPerBatch);
			Batches.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(
				[&Tasks, BatchIndex]()
				{
					for (int32 TaskIndex = 0; TaskIndex < NumTasksPerBatch; ++TaskIndex)
					{
						Tasks[BatchIndex].Add(FFunctionGraphTask::CreateAndDispatchWhenReady([] {}));
					}
				}
			));
		}

		FTaskGraphInterface::Get().WaitUntilTasksComplete(Batches);
		for (int32 BatchIndex = 0; BatchIndex != NumBatches; ++BatchIndex)
		{
			FTaskGraphInterface::Get().WaitUntilTasksComplete(Tasks[BatchIndex]);
		}
	}

	TEST_CASE_NAMED(FTaskGraphPerfTest, "System::Core::Async::TaskGraph::PerfTest", "[.][ApplicationContextMask][EngineFilter]")
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TaskGraphTests_PerfTest);

		//UE_BENCHMARK(1, TestSpawning<100000>);
		//return true;

		//UE_BENCHMARK(5, Fib<18>);
		//UE_BENCHMARK(5, [] { Fibonacci(15); });

		UE_BENCHMARK(5, TestPerfBasic<100000>);
		UE_BENCHMARK(5, TestPerfBatch<100000, 100>);
		UE_BENCHMARK(5, TestPerfBatchOptimised<100000, 100>);
		UE_BENCHMARK(5, TestLatency<10000>);
		UE_BENCHMARK(5, TestPerfChaining<10000>);
		UE_BENCHMARK(5, TestFGraphEventPerf<100000>);
		UE_BENCHMARK(5, TestWorkStealing<100, 1000>);
		UE_BENCHMARK(5, TestSpawning<100000>);
		UE_BENCHMARK(5, TestBatchSpawning<100000>);
	}

	template<uint32 Num>
	void OversubscriptionStressTest()
	{
		// launch a number of tasks >= num of cores, each of them launches a task and waits for its completion. do this inside a task so 
		// we can detect deadlock.
		// outer tasks occupy all worker threads. when they launch inner tasks and wait for them, the inner tasks can't be 
		// executed because all workers are blocked -> deadlock.
		// can be solved by busy waiting

		for (int i = 0; i != Num; ++i)
		{
			FSharedEventRef Event;
			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[Event]
				{
					ParallelFor(200,
						[](int32)
						{
							FPlatformProcess::Sleep(0.01f); // simulate some work and let all workers to pick up ParallelFor tasks
							FFunctionGraphTask::CreateAndDispatchWhenReady([] {})->Wait();
						}
					);
					Event->Trigger();
				}
			);
			verify(Event->Wait(FTimespan::FromSeconds(5.f)));
		}
	}

	TEST_CASE_NAMED(FTaskGraphOversubscriptionTest, "System::Core::Async::TaskGraph::Oversubscription", "[.][ApplicationContextMask][EngineFilter]")
	{
		UE_BENCHMARK(5, OversubscriptionStressTest<10>);
	}

	TEST_CASE_NAMED(FTaskGraphTaskDestructionTest, "System::Core::Async::TaskGraph::TaskDestruction", "[.][ApplicationContextMask][EngineFilter]")
	{
		struct FDestructionTest
		{
			explicit FDestructionTest(bool* bDestroyedIn)
				: bDestroyed(bDestroyedIn)
			{}

			~FDestructionTest()
			{
				*bDestroyed = true;
			}

			bool* bDestroyed;
		};

		bool bDestroyed = false;

		FFunctionGraphTask::CreateAndDispatchWhenReady([DestructionTest = FDestructionTest{ &bDestroyed }]{})->Wait();

		check(bDestroyed);
	}

	TEST_CASE_NAMED(FTaskGraphWaitForAnyTask, "System::Core::Async::TaskGraph::WaitForAnyTask", "[.][ApplicationContextMask][EngineFilter]")
	{
		{	// blocks if none of tasks is completed
			FGraphEventRef Blocker = FGraphEvent::CreateGraphEvent(); // blocks all tasks

			FGraphEventArray Tasks
			{
				FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, Blocker),
				FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, Blocker)
			};

			verify(WaitForAnyTaskCompleted(Tasks, FTimespan::FromMilliseconds(1.0)) == INDEX_NONE);

			Blocker->DispatchSubsequents();

			verify(WaitForAnyTaskCompleted(Tasks) != INDEX_NONE);
		}

		{	// doesn't wait for all tasks
			FGraphEventRef Blocker = FGraphEvent::CreateGraphEvent();

			FGraphEventArray Tasks
			{
				FFunctionGraphTask::CreateAndDispatchWhenReady([] {}),
				FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, Blocker) // is blocked
			};

			verify(WaitForAnyTaskCompleted(Tasks) == 0);

			Blocker->DispatchSubsequents();
		}
	}

	// This test proves that retraction alone is not sufficient to avoid deadlocks and oversubscription is also required as a last resort.
	// This test is a repro for a virtual texture system scheduled on foreground thread, sending and waiting on an IO thread request that itself needs to 
	// do some processing on the foreground thread before releasing the IO event, causing a deadlock when only a single worker is present.
	TEST_CASE_NAMED(FTaskGraphConstrainedRetraction, "System::Core::Async::TaskGraph::ConstrainedRetraction", "[.][ApplicationContextMask][EngineFilter]")
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTaskGraphConstrainedRetraction);

		// A lot easier to prove when we have a minimal amount of workers
		LowLevelTasks::FScheduler::Get().RestartWorkers(1, 0);

		TFunction<void(FGraphEventRef Task)> WaitMethods[2] =
		{
			[](FGraphEventRef Task) { TRACE_CPUPROFILER_EVENT_SCOPE(Task_Wait); TRACE_CPUPROFILER_EVENT_FLUSH(); Task->Wait(); },
			[](FGraphEventRef Task) { TRACE_CPUPROFILER_EVENT_SCOPE(Task_WaitUntilTaskCompletes); TRACE_CPUPROFILER_EVENT_FLUSH(); FTaskGraphInterface::Get().WaitUntilTaskCompletes(Task); }
		};

		for (int32 WaitMethodIndex = 0; WaitMethodIndex < UE_ARRAY_COUNT(WaitMethods); ++WaitMethodIndex)
		{
			// This is going to be a dependency on something outside the taskgraph system
			// Could be another system entirely like the IO thread pool, or even a lowlevel task without prerequisites/subsequents knowledge.
			FGraphEventRef ExternalEvent = FGraphEvent::CreateGraphEvent();

			// We need the worker thread to be busy with our root task while we schedule the external one
			FGraphEventRef RootTask = 
				FFunctionGraphTask::CreateAndDispatchWhenReady(
					[&]
					{
						FFunctionGraphTask::CreateAndDispatchWhenReady(
							[&]
							{
								TRACE_CPUPROFILER_EVENT_SCOPE(DispatchSubsequents);
								TRACE_CPUPROFILER_EVENT_FLUSH();
								ExternalEvent->DispatchSubsequents();

							}, TStatId{}
						);
						WaitMethods[WaitMethodIndex](ExternalEvent);

					}, TStatId{}
				);

			// This wait should try to perform retraction, but it can't see past the external task prerequisites
			// Since we launch a task on a single worker system and then wait, retraction alone without oversubscription
			// will hold the worker forever and will cause a deadlock.
			WaitMethods[WaitMethodIndex](RootTask);
		}

		LowLevelTasks::FScheduler::Get().RestartWorkers();
	}

	TEST_CASE_NAMED(FTaskGraphAnyTask, "System::Core::Async::TaskGraph::AnyTask", "[.][ApplicationContextMask][EngineFilter]")
	{
		{	// blocks if none of tasks is completed
			FGraphEventRef Blocker = FGraphEvent::CreateGraphEvent(); // blocks all tasks

			FGraphEventArray Tasks
			{
				FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, Blocker),
				FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, Blocker)
			};

			FGraphEventRef CompletedEvent = AnyTaskCompleted(Tasks);
			FPlatformProcess::Sleep(0.1f);
			verify(!CompletedEvent->IsComplete());

			Blocker->DispatchSubsequents();

			CompletedEvent->Wait();
		}

		{	// doesn't wait for all tasks
			FGraphEventRef Blocker = FGraphEvent::CreateGraphEvent();

			FGraphEventArray Tasks
			{
				FFunctionGraphTask::CreateAndDispatchWhenReady([] {}),
				FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId{}, Blocker) // is blocked
			};

			AnyTaskCompleted(Tasks)->Wait();

			Blocker->DispatchSubsequents();
		}
	}
}

#endif //WITH_TESTS