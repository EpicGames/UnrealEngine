// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Compression/CompressedBuffer.h"

#include "Algo/Compare.h"
#include "Compression/OodleDataCompression.h"
#include "IO/IoHash.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

TEST_CASE("Core::Compression::CompressedBuffer::Compress", "[Core][Compression][Smoke]")
{
	const uint8 ZeroBuffer[1024]{};
	const FIoHash ZeroBufferHash = FIoHash::HashBuffer(MakeMemoryView(ZeroBuffer));

	const auto SerializeBuffer = [](const FCompressedBuffer& Buffer) -> FCompressedBuffer
	{
		FCompressedBuffer SerializedBuffer = Buffer;
		TArray<uint8> Bytes;
		{
			FMemoryWriter Ar(Bytes, /*bIsPersistent*/ true);
			Ar << SerializedBuffer;
			CHECK_FALSE(Ar.IsError());
		}
		SerializedBuffer.Reset();
		{
			FMemoryReader Ar(Bytes, /*bIsPersistent*/ true);
			Ar << SerializedBuffer;
			CHECK_FALSE(Ar.IsError());
		}
		return SerializedBuffer;
	};

	SECTION("Null")
	{
		const FCompressedBuffer Buffer;

		CHECK_FALSE(Buffer);
		CHECK(Buffer.IsNull());
		CHECK(Buffer.IsOwned());
		CHECK(Buffer.GetCompressedSize() == 0);
		CHECK(Buffer.GetRawSize() == 0);
		CHECK(Buffer.GetRawHash() == FIoHash::Zero);
		CHECK(Buffer.Decompress().IsNull());
		CHECK(Buffer.DecompressToComposite().IsNull());
	}

	SECTION("Empty")
	{
		const FSharedBuffer EmptyBuffer = FUniqueBuffer::Alloc(0).MoveToShared();
		const FIoHash ExpectedRawHash = FIoHash::HashBuffer(EmptyBuffer);
		const FCompressedBuffer OriginalBuffer = FCompressedBuffer::Compress(EmptyBuffer);
		const FCompressedBuffer SerializedBuffer = SerializeBuffer(OriginalBuffer);
		const FCompressedBuffer Buffer = GENERATE_REF(OriginalBuffer, SerializedBuffer);

		CHECK(Buffer);
		CHECK_FALSE(Buffer.IsNull());
		CHECK(Buffer.IsOwned());
		CHECK(Buffer.GetRawSize() == 0);
		CHECK(Buffer.GetRawHash() == ExpectedRawHash);
		CHECK_FALSE(Buffer.Decompress().IsNull());
		CHECK_FALSE(Buffer.DecompressToComposite().IsNull());
	}

	SECTION("Method None")
	{
		const FCompressedBuffer OriginalBuffer = FCompressedBuffer::Compress(FSharedBuffer::MakeView(MakeMemoryView(ZeroBuffer)),
			ECompressedBufferCompressor::NotSet, ECompressedBufferCompressionLevel::None);
		const FCompressedBuffer CopiedBuffer = FCompressedBuffer::FromCompressed(OriginalBuffer.GetCompressed());
		const FCompressedBuffer SerializedBuffer = SerializeBuffer(OriginalBuffer);
		const FCompressedBuffer Buffer = GENERATE_REF(OriginalBuffer, CopiedBuffer, SerializedBuffer);

		CHECK(Buffer);
		CHECK_FALSE(Buffer.IsNull());
		CHECK(Buffer.IsOwned());
		CHECK(Buffer.GetCompressedSize() == sizeof(ZeroBuffer) + 64);
		CHECK(Buffer.GetRawSize() == sizeof(ZeroBuffer));
		CHECK(Buffer.GetRawHash() == ZeroBufferHash);
		CHECK(FIoHash::HashBuffer(Buffer.Decompress()) == ZeroBufferHash);
		CHECK(FIoHash::HashBuffer(Buffer.DecompressToComposite()) == Buffer.GetRawHash());

		ECompressedBufferCompressor Compressor = ECompressedBufferCompressor::Kraken;
		ECompressedBufferCompressionLevel CompressionLevel = ECompressedBufferCompressionLevel::Normal;
		uint64 BlockSize = MAX_uint64;
		CHECK(Buffer.TryGetCompressParameters(Compressor, CompressionLevel, BlockSize));
		CHECK(Compressor == ECompressedBufferCompressor::NotSet);
		CHECK(CompressionLevel == ECompressedBufferCompressionLevel::None);
		CHECK(BlockSize == 0);
	}

	SECTION("Method Oodle")
	{
		const FCompressedBuffer OriginalBuffer = FCompressedBuffer::Compress(FSharedBuffer::MakeView(MakeMemoryView(ZeroBuffer)),
			ECompressedBufferCompressor::Mermaid, ECompressedBufferCompressionLevel::VeryFast);
		const FCompressedBuffer CopiedBuffer = FCompressedBuffer::FromCompressed(OriginalBuffer.GetCompressed());
		const FCompressedBuffer SerializedBuffer = SerializeBuffer(OriginalBuffer);
		const FCompressedBuffer Buffer = GENERATE_REF(OriginalBuffer, CopiedBuffer, SerializedBuffer);

		CHECK(Buffer);
		CHECK_FALSE(Buffer.IsNull());
		CHECK(Buffer.IsOwned());
		CHECK(Buffer.GetCompressedSize() < sizeof(ZeroBuffer));
		CHECK(Buffer.GetRawSize() == sizeof(ZeroBuffer));
		CHECK(Buffer.GetRawHash() == ZeroBufferHash);
		CHECK(FIoHash::HashBuffer(Buffer.Decompress()) == ZeroBufferHash);
		CHECK(FIoHash::HashBuffer(Buffer.DecompressToComposite()) == Buffer.GetRawHash());

		ECompressedBufferCompressor Compressor = ECompressedBufferCompressor::Kraken;
		ECompressedBufferCompressionLevel CompressionLevel = ECompressedBufferCompressionLevel::Normal;
		uint64 BlockSize = MAX_uint64;
		CHECK(Buffer.TryGetCompressParameters(Compressor, CompressionLevel, BlockSize));
		CHECK(Compressor == ECompressedBufferCompressor::Mermaid);
		CHECK(CompressionLevel == ECompressedBufferCompressionLevel::VeryFast);
		CHECK(FMath::IsPowerOfTwo(BlockSize));
	}
}

TEST_CASE("Core::Compression::CompressedBuffer::Decompress", "[Core][Compression][Smoke]")
{
	const auto GenerateData = [](int32 N) -> TArray<uint64>
	{
		TArray<uint64> Data;
		Data.SetNum(N);
		for (int32 Idx = 0; Idx < Data.Num(); ++Idx)
		{
			Data[Idx] = uint64(Idx);
		}
		return Data;
	};

	const auto CastToArrayView = [](FMemoryView View) -> TConstArrayView<uint64>
	{
		return MakeArrayView(static_cast<const uint64*>(View.GetData()), static_cast<int32>(View.GetSize() / sizeof(uint64)));
	};

	FCompressedBufferReader Reader;

	SECTION("Decompress with offset and size.")
	{
		const auto UncompressAndValidate = [&Reader, &CastToArrayView](
			const FCompressedBuffer& Compressed,
			const int32 OffsetCount,
			const int32 Count,
			const TConstArrayView<uint64> ExpectedValues)
		{
			Reader.SetSource(Compressed);
			{
				const FSharedBuffer Uncompressed = Reader.Decompress(OffsetCount * sizeof(uint64), Count * sizeof(uint64));
				CHECK(Algo::Compare(CastToArrayView(Uncompressed), ExpectedValues.Mid(OffsetCount, Count)));
			}
			{
				FUniqueBuffer Uncompressed = FUniqueBuffer::Alloc(Count * sizeof(uint64));
				CHECKED_IF(Reader.TryDecompressTo(Uncompressed, OffsetCount * sizeof(uint64)))
				{
					CHECK(Algo::Compare(CastToArrayView(Uncompressed), ExpectedValues.Mid(OffsetCount, Count)));
				}
			}
		};

		constexpr uint64 BlockSize = 64 * sizeof(uint64);
		constexpr int32 N = 5000;
		const TArray<uint64> ExpectedValues = GenerateData(N);

		const FCompressedBuffer Compressed = FCompressedBuffer::Compress(
			FSharedBuffer::MakeView(MakeMemoryView(ExpectedValues)),
			FOodleDataCompression::ECompressor::Mermaid,
			FOodleDataCompression::ECompressionLevel::Fast,
			BlockSize);

		UncompressAndValidate(Compressed, 0, N, ExpectedValues);
		UncompressAndValidate(Compressed, 1, N - 1, ExpectedValues);
		UncompressAndValidate(Compressed, N - 1, 1, ExpectedValues);
		UncompressAndValidate(Compressed, 0, 1, ExpectedValues);
		UncompressAndValidate(Compressed, 2, 4, ExpectedValues);
		UncompressAndValidate(Compressed, 0, 512, ExpectedValues);
		UncompressAndValidate(Compressed, 3, 514, ExpectedValues);
		UncompressAndValidate(Compressed, 256, 512, ExpectedValues);
		UncompressAndValidate(Compressed, 512, 512, ExpectedValues);
		UncompressAndValidate(Compressed, 512, 512, ExpectedValues);
		UncompressAndValidate(Compressed, 4993, 4, ExpectedValues);
	}

	SECTION("Decompress with offset only.")
	{
		constexpr uint64 BlockSize = 64 * sizeof(uint64);
		constexpr int32 N = 1000;
		const TArray<uint64> ExpectedValues = GenerateData(N);

		const FCompressedBuffer Compressed = FCompressedBuffer::Compress(
			FSharedBuffer::MakeView(MakeMemoryView(ExpectedValues)),
			FOodleDataCompression::ECompressor::Mermaid,
			FOodleDataCompression::ECompressionLevel::Fast,
			BlockSize);

		constexpr uint64 OffsetCount = 150;
		{
			FSharedBuffer Buffer = Compressed.GetCompressed().ToShared();
			FMemoryReaderView Ar(Buffer, /*bIsPersistent*/ true);
			FCompressedBufferReaderSourceScope Source(Reader, Ar);
			const FSharedBuffer Uncompressed = Reader.Decompress(OffsetCount * sizeof(uint64));
			CHECK(Algo::Compare(CastToArrayView(Uncompressed), MakeArrayView(ExpectedValues).RightChop(OffsetCount)));
		}
		{
			FCompressedBufferReaderSourceScope Source(Reader, Compressed);
			const FSharedBuffer Uncompressed = Reader.Decompress(OffsetCount * sizeof(uint64));
			CHECK(Algo::Compare(CastToArrayView(Uncompressed), MakeArrayView(ExpectedValues).RightChop(OffsetCount)));
		}

		// Short Buffer
		{
			const FCompressedBuffer CompressedShort =
				FCompressedBuffer::FromCompressed(Compressed.GetCompressed().Mid(0, Compressed.GetCompressedSize() - 128));
			Reader.SetSource(CompressedShort);
			const FSharedBuffer Uncompressed = Reader.Decompress();
			CHECK(Uncompressed.IsNull());
		}
	}

	SECTION("Decompress only block.")
	{
		constexpr uint64 BlockSize = 256 * sizeof(uint64);
		constexpr int32 N = 100;
		const TArray<uint64> ExpectedValues = GenerateData(N);

		const FCompressedBuffer Compressed = FCompressedBuffer::Compress(
			FSharedBuffer::MakeView(MakeMemoryView(ExpectedValues)),
			FOodleDataCompression::ECompressor::Mermaid,
			FOodleDataCompression::ECompressionLevel::Fast,
			BlockSize);

		constexpr uint64 OffsetCount = 2;
		constexpr uint64 Count = 50;
		{
			FSharedBuffer Buffer = Compressed.GetCompressed().ToShared();
			FMemoryReaderView Ar(Buffer, /*bIsPersistent*/ true);
			FCompressedBufferReaderSourceScope Source(Reader, Ar);
			const FSharedBuffer Uncompressed = Reader.Decompress(OffsetCount * sizeof(uint64), Count * sizeof(uint64));
			CHECK(Algo::Compare(CastToArrayView(Uncompressed), MakeArrayView(ExpectedValues).Mid(OffsetCount, Count)));
		}
		{
			FCompressedBufferReaderSourceScope Source(Reader, Compressed);
			const FSharedBuffer Uncompressed = Reader.Decompress(OffsetCount * sizeof(uint64), Count * sizeof(uint64));
			CHECK(Algo::Compare(CastToArrayView(Uncompressed), MakeArrayView(ExpectedValues).Mid(OffsetCount, Count)));
		}
	}

	SECTION("Decompress from an uncompressed buffer.")
	{
		constexpr int32 N = 4242;
		const TArray<uint64> ExpectedValues = GenerateData(N);

		const FCompressedBuffer Compressed = FCompressedBuffer::Compress(
			FSharedBuffer::MakeView(MakeMemoryView(ExpectedValues)),
			FOodleDataCompression::ECompressor::NotSet,
			FOodleDataCompression::ECompressionLevel::None);
		Reader.SetSource(Compressed);

		{
			constexpr uint64 OffsetCount = 0;
			constexpr uint64 Count = N;
			const FSharedBuffer Uncompressed = Reader.Decompress(OffsetCount * sizeof(uint64), Count * sizeof(uint64));
			CHECK(Algo::Compare(CastToArrayView(Uncompressed), MakeArrayView(ExpectedValues).Mid(OffsetCount, Count)));
		}

		{
			constexpr uint64 OffsetCount = 21;
			constexpr uint64 Count = 999;
			const FSharedBuffer Uncompressed = Reader.Decompress(OffsetCount * sizeof(uint64), Count * sizeof(uint64));
			CHECK(Algo::Compare(CastToArrayView(Uncompressed), MakeArrayView(ExpectedValues).Mid(OffsetCount, Count)));
		}

		// Short Buffer
		{
			const FCompressedBuffer CompressedShort =
				FCompressedBuffer::FromCompressed(Compressed.GetCompressed().Mid(0, Compressed.GetCompressedSize() - 128));
			Reader.SetSource(CompressedShort);
			const FSharedBuffer Uncompressed = Reader.Decompress();
			CHECK(Uncompressed.IsNull());
		}
	}
}

#endif