// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/BitArray.h"
#include "Misc/StringBuilder.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/Function.h"
#include "Tests/TestHarnessAdapter.h"
#include "Tests/Assertions.h"

namespace UE
{
namespace BitArrayTest
{

	TBitArray<> ConstructBitArray(const char* Bits, int32 MaxNum = TNumericLimits<int32>::Max())
	{
		TBitArray<> Out;

		for ( ; MaxNum > 0 && *Bits != '\0'; ++Bits)
		{
			check(*Bits == ' ' || *Bits == '0' || *Bits == '1');

			// Skip spaces
			if (*Bits != ' ')
			{
				Out.Add(*Bits == '1');
				--MaxNum;
			}
		}
		return Out;
	}

	FString BitArrayToString(const TBitArray<>& BitArray)
	{
		FString Out;
		int32 Index = 0;
		for (TBitArray<>::FConstIterator It(BitArray); It; ++It, ++Index)
		{
			if (Index != 0 && Index % 8 == 0)
			{
				Out.AppendChar(TEXT(' '));
			}
			Out.AppendChar(It.GetValue() ? TEXT('1') : TEXT('0'));
		}
		return Out;
	}

	/** TBitArray does not have a templated equal operator so we use this one when we have TBitArrays with different Allocators */
	template <typename Allocator1, typename Allocator2>
	bool AreEqual(TBitArray<Allocator1>& A, TBitArray<Allocator2>& B)
	{
		int32 Num = A.Num();
		if (Num != B.Num())
		{
			return false;
		}
		for (int n = 0; n < Num; ++n)
		{
			if (A[n] != B[n])
			{
				return false;
			}
		}
		return true;
	}

} // namespace BitArrayTest
} // namespace UE


class FBitArrayTest
{
public:
	FBitArrayTest()
		: bGrowingTrue{ true, false, true, true, false, true, true, true, false, true }
		, GrowingTrueInt(0x1 | /*(0 * 0x2) |*/ 0x4 | 0x8 | /*(0 * 0x10) |*/ 0x20 | 0x40 | 0x80 | /*(0 * 0x100) |*/ 0x200)
		, NumGrowingTrue(sizeof(bGrowingTrue) / sizeof(bGrowingTrue[0]))
		, ArrGrowingTrue(true, NumGrowingTrue)
		, NumSquareWave(12)
		, ArrSquareWave(true, NumSquareWave)
		, ArrTrue(true, 10)
		, ArrFalse(false, 10)
		, ArrGrowingTrueTwice(true, NumGrowingTrue * 2)
	{
	}

	bool bGrowingTrue[10];
	uint32 GrowingTrueInt;
	int NumGrowingTrue;
	TBitArray<> ArrGrowingTrue;
	int NumSquareWave;
	TBitArray<> ArrSquareWave;
	TBitArray<> ArrTrue;
	TBitArray<> ArrFalse;
	TBitArray<> ArrGrowingTrueTwice;
	TStringBuilder<128> Label;

	bool ConstructAndTestConstructors()
	{
		bool bHasAnyErrors(false);

		ArrTrue.CheckInvariants();
		CHECK_AND_SET_ERROR_ON_FAIL(TEXT("ArrTrue Size"), ArrTrue.Num() == 10, bHasAnyErrors);
		ArrFalse.CheckInvariants();
		CHECK_AND_SET_ERROR_ON_FAIL(TEXT("ArrFalse Size"), ArrFalse.Num() == 10, bHasAnyErrors);
		for (int n = 0; n < 10; ++n)
		{
			CHECK_AND_SET_ERROR_ON_FAIL(TEXT("ArrTrue elements"), ArrTrue[n] == true, bHasAnyErrors);
			CHECK_AND_SET_ERROR_ON_FAIL(TEXT("ArrFalse elements"), ArrFalse[n] == false, bHasAnyErrors);
		}

		ArrGrowingTrue.CheckInvariants();
		CHECK_AND_SET_ERROR_ON_FAIL(TEXT("ArrGrowingTrue Size"), ArrGrowingTrue.Num() == NumGrowingTrue, bHasAnyErrors);
		for (int n = 0; n < NumGrowingTrue; ++n)
		{
			ArrGrowingTrue[n] = bGrowingTrue[n];
			CHECK_AND_SET_ERROR_ON_FAIL(TEXT("ArrGrowingTrue elements"), ArrGrowingTrue[n] == bGrowingTrue[n], bHasAnyErrors);
		}
		ArrSquareWave.CheckInvariants();
		for (int n = 0; n < NumSquareWave; ++n)
		{
			ArrSquareWave[n] = n % 2 == 1;
		}
		ArrGrowingTrueTwice.CheckInvariants();
		for (int n = 0; n < NumGrowingTrue; ++n)
		{
			ArrGrowingTrueTwice[n] = bGrowingTrue[n];
			ArrGrowingTrueTwice[n + NumGrowingTrue] = bGrowingTrue[n];
		}

		return !bHasAnyErrors;
	}

	void TestEqualityOperator()
	{
		// == and != operators
		TBitArray<> ArrGrowingTrue2(true, NumGrowingTrue);
		TBitArray<> ArrAlmostGrowingTrue(true, NumGrowingTrue);
		TBitArray<> ArrSubsetGrowingTrue(true, NumGrowingTrue - 1);
		for (int n = 0; n < NumGrowingTrue - 1; ++n)
		{
			ArrGrowingTrue2[n] = bGrowingTrue[n];
			ArrAlmostGrowingTrue[n] = bGrowingTrue[n];
			ArrSubsetGrowingTrue[n] = bGrowingTrue[n];
		}
		ArrGrowingTrue2[NumGrowingTrue - 1] = bGrowingTrue[NumGrowingTrue - 1];
		ArrAlmostGrowingTrue[NumGrowingTrue - 1] = !bGrowingTrue[NumGrowingTrue - 1];
		CHECK_MESSAGE(TEXT("Equality operator on equal arrays"), ArrGrowingTrue == ArrGrowingTrue2);
		CHECK_MESSAGE(TEXT("Inequality operator on equal arrays"), !(ArrGrowingTrue != ArrGrowingTrue2));
		CHECK_MESSAGE(TEXT("Equality operator on nonequal arrays"), !(ArrGrowingTrue == ArrAlmostGrowingTrue));
		CHECK_MESSAGE(TEXT("Inequality operator on nonequal arrays"), ArrGrowingTrue != ArrAlmostGrowingTrue);
		CHECK_MESSAGE(TEXT("Equality operator when lhs is superset of rhs"), !(ArrGrowingTrue == ArrSubsetGrowingTrue));
		CHECK_MESSAGE(TEXT("Inequality operator when lhs is superset of rhs"), ArrGrowingTrue != ArrSubsetGrowingTrue);
		CHECK_MESSAGE(TEXT("Equality operator when lhs is subset of rhs"), !(ArrSubsetGrowingTrue == ArrGrowingTrue));
		CHECK_MESSAGE(TEXT("Inequality operator when lhs is subset of rhs"), ArrSubsetGrowingTrue != ArrGrowingTrue);
	}

	void TestOtherConstructorAndAssignment()
	{
		// Empty constructor with inline allocator
		{
			TBitArray<TInlineAllocator<4>> Arr;
			Arr.CheckInvariants();
		}
		// Empty constructor with default (no inline storage) allocator
		{
			TBitArray<FDefaultAllocator> Arr;
			Arr.CheckInvariants();
		}
		// Some items constructor with inline allocator
		{
			TBitArray<TInlineAllocator<4>> Arr(true, NumBitsPerDWORD + NumBitsPerDWORD / 2);
			Arr.CheckInvariants();
		}
		// Some items constructor with default (no inline storage) allocator
		{
			TBitArray<FDefaultAllocator> Arr(true, NumBitsPerDWORD + NumBitsPerDWORD / 2);
			Arr.CheckInvariants();
		}

		// Move constructor
		{
			TBitArray<> ArrVictim(ArrGrowingTrue);
			ArrVictim.CheckInvariants();
			TBitArray<> Arr(MoveTemp(ArrVictim));
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Copy Constructor"), Arr == ArrGrowingTrue);
		}

		// Copy constructor
		{
			TBitArray<> Arr(ArrGrowingTrue);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Copy Constructor"), Arr == ArrGrowingTrue);
		}

		// Assignment operator
		{
			TBitArray<> Arr;
			Arr.CheckInvariants();
			Arr = ArrGrowingTrue;
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Assignment operator"), Arr == ArrGrowingTrue);
		}

		// Move Assignment operator
		{
			TBitArray<> ArrVictim(ArrGrowingTrue);
			ArrVictim.CheckInvariants();
			TBitArray<> Arr;
			Arr.CheckInvariants();
			Arr = MoveTemp(ArrVictim);
			CHECK_MESSAGE(TEXT("Move Assignment operator"), Arr == ArrGrowingTrue);
		}
	}

	void TestLessThan()
	{
		// operator<
		TBitArray<> Short(true, 4);
		TBitArray<> MediumFalse(false, 5);
		TBitArray<> MediumTrue(true, 5);
		TBitArray<> Long(false, 6);
		CHECK_MESSAGE(TEXT("! x < x"), !(Short.operator<(Short)));
		CHECK_MESSAGE(TEXT("Sorted by length first, so Short < MediumFalse"), Short < MediumFalse);
		CHECK_MESSAGE(TEXT("Sorted by length first, so !MediumFalse < Short"), !(MediumFalse < Short));
		CHECK_MESSAGE(TEXT("Sorted by length first, so Short < MediumTrue"), Short < MediumTrue);
		CHECK_MESSAGE(TEXT("Sorted by length first, so !MediumTrue < Short"), !(MediumTrue < Short));
		CHECK_MESSAGE(TEXT("Sorted by length first, so MediumTrue < Long"), MediumFalse < Long);
		CHECK_MESSAGE(TEXT("Sorted by length first, so !Long < MediumTrue"), !(Long < MediumFalse));
		CHECK_MESSAGE(TEXT("Sorted by length first, so MediumFalse < Long"), MediumTrue < Long);
		CHECK_MESSAGE(TEXT("Sorted by length first, so !Long < MediumFalse"), !(Long < MediumTrue));

		TBitArray<> MediumTrueAtEnd(false, 5);
		MediumTrueAtEnd[4] = true;
		TBitArray<> MediumTrueAtStart(false, 5);
		MediumTrueAtStart[0] = true;
		CHECK_MESSAGE(TEXT("Sorted lexigraphically second, so MediumFalse < MediumTrueAtEnd"), MediumFalse < MediumTrueAtEnd);
		CHECK_MESSAGE(TEXT("Sorted lexigraphically second, so MediumTrueAtEnd < MediumTrueAtStart"), MediumFalse < MediumTrueAtEnd);
		CHECK_MESSAGE(TEXT("Sorted lexigraphically second, so MediumTrueAtStart < MediumTrue"), MediumTrueAtEnd < MediumTrue);
	}

	void TestRemoveAt()
	{
		{
			TBitArray<> Arr(ArrSquareWave);
			Arr.RemoveAt(Arr.Num() - 1, 1);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("RemoveAt from end size"), Arr.Num() == (NumSquareWave - 1));
			for (int n = 0; n < NumSquareWave - 1; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAt from end elements"), Arr[n] == ArrSquareWave[n]);
			}

			Arr = ArrSquareWave;
			Arr.RemoveAt(0, 1);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("RemoveAt from start size"), Arr.Num() == (NumSquareWave - 1));
			for (int n = 0; n < NumSquareWave - 1; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAt from start elements"), Arr[n] == ArrSquareWave[n + 1]);
			}

			Arr = ArrSquareWave;
			Arr.RemoveAt(5, 1);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("RemoveAt from middle size"), Arr.Num() == (NumSquareWave - 1));
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAt from middle elements"), Arr[n] == ArrSquareWave[n]);
			}
			for (int n = 5; n < NumSquareWave - 1; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAt from middle elements"), Arr[n] == ArrSquareWave[n + 1]);
			}

			Arr = TBitArray<>(true, 20);
			for (int n = 10; n < 20; ++n)
			{
				Arr[n] = false;
			}
			Arr.RemoveAt(5, 5);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("RemoveAt multiple size"), Arr.Num() == 15);
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAt multiple elements"), Arr[n] == true);
			}
			for (int n = 5; n < 15; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAt multiple elements"), Arr[n] == false);
			}
		}

		// RemoveAt zeroes bits after the end
		{
			TBitArray<FDefaultAllocator> Arr(true, 48);
			for (int n = 0; n < 48; ++n)
			{
				Arr.RemoveAt(Arr.Num() - 1, 1);
				Arr.CheckInvariants();
			}

			TBitArray<FDefaultAllocator> Arr2(true, 48);
			Arr.RemoveAt(0, Arr.Num());
			Arr.CheckInvariants();
		}
	}

	void TestRemoveAtSwap()
	{
		// RemoveAtSwap
		{
			TBitArray<> Arr(ArrSquareWave);
			Arr.RemoveAtSwap(Arr.Num() - 1, 1);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("RemoveAtSwap from end size"), Arr.Num() == (NumSquareWave - 1));
			for (int n = 0; n < NumSquareWave - 1; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAtSwap from end elements"), Arr[n] == ArrSquareWave[n]);
			}

			Arr = ArrSquareWave;
			Arr.RemoveAtSwap(0, 1);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("RemoveAtSwap from start size"), Arr.Num() == (NumSquareWave - 1));
			CHECK_MESSAGE(TEXT("RemoveAtSwap from start elements"), Arr[0] == (ArrSquareWave[NumSquareWave - 1]));
			for (int n = 1; n < NumSquareWave - 1; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAtSwap from start elements"), Arr[n] == ArrSquareWave[n]);
			}

			Arr = ArrSquareWave;
			Arr.RemoveAtSwap(5, 1);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("RemoveAtSwap from middle size"), Arr.Num() == (NumSquareWave - 1));
			for (int n = 0; n < NumSquareWave - 1; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAtSwap from middle elements"),
					Arr[n] == (n != 5 ? ArrSquareWave[n] : ArrSquareWave[NumSquareWave - 1]));
			}

			Arr = TBitArray<>(true, 20);
			for (int n = 10; n < 20; ++n)
			{
				Arr[n] = false;
			}
			Arr.RemoveAtSwap(5, 2);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("RemoveAtSwap, multiple, size"), Arr.Num() == 18);
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAtSwap, multiple, elements"), Arr[n] == true);
			}
			for (int n = 5; n < 7; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAtSwap, multiple, elements"), Arr[n] == false);
			}
			for (int n = 7; n < 10; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAtSwap, multiple, elements"), Arr[n] == true);
			}
			for (int n = 10; n < 18; ++n)
			{
				CHECK_MESSAGE(TEXT("RemoveAtSwap, multiple, elements"), Arr[n] == false);
			}
		}

		// RemoveAtSwap zeroes bits after the end
		{
			TBitArray<FDefaultAllocator> Arr(true, 48);
			for (int n = 0; n < 48; ++n)
			{
				Arr.RemoveAtSwap(0, 1);
				Arr.CheckInvariants();
			}

			TBitArray<FDefaultAllocator> Arr2(true, 48);
			Arr.RemoveAtSwap(0, Arr.Num());
			Arr.CheckInvariants();
		}
	}

	void TestSerialize()
	{
		TBitArray<> ArrEmpty;
		TBitArray<> ArrOnes(true, NumBitsPerDWORD + NumBitsPerDWORD / 2);
		TBitArray<> ArrZeroes(false, NumBitsPerDWORD + NumBitsPerDWORD / 2);
		TBitArray<> ArrEmptyOriginal;
		TBitArray<> ArrOnesOriginal(ArrOnes);
		TBitArray<> ArrZeroesOriginal(ArrZeroes);
		uint32 Spacer = 0x12345678;

		TArray<uint8> Bytes;
		{
			FMemoryWriter Writer(Bytes);
			Writer << Spacer << ArrEmpty << Spacer << ArrOnes << Spacer << ArrZeroes << Spacer;
		}
		// serialize into a loading archive should not modify the array
		CHECK_MESSAGE(TEXT("Serialize Empty"), ArrEmpty == ArrEmptyOriginal);
		CHECK_MESSAGE(TEXT("Serialize Ones"), ArrOnes == ArrOnesOriginal);
		CHECK_MESSAGE(TEXT("Serialize Ones"), ArrZeroes == ArrZeroesOriginal);

		TBitArray<> ArrEmptyCopy;
		TBitArray<> ArrZeroesCopy(false, NumBitsPerDWORD + NumBitsPerDWORD / 2);
		TBitArray<> ArrOnesCopy(true, NumBitsPerDWORD + NumBitsPerDWORD / 2);
		uint32 SpacerCopies[4];
		{
			FMemoryReader Reader(Bytes);
			Reader << SpacerCopies[0] << ArrEmptyCopy << SpacerCopies[1] << ArrOnesCopy << SpacerCopies[2] << ArrZeroesCopy << SpacerCopies[3];
		}
		CHECK_MESSAGE(TEXT("Serialize Empty Underflow"), SpacerCopies[0] == Spacer);
		CHECK_MESSAGE(TEXT("Serialize Empty"), ArrEmpty == ArrEmptyCopy);
		ArrEmptyCopy.CheckInvariants();
		CHECK_MESSAGE(TEXT("Serialize Ones Underflow"), SpacerCopies[1] == Spacer);
		CHECK_MESSAGE(TEXT("Serialize Ones"), ArrOnes == ArrOnesCopy);
		ArrOnesCopy.CheckInvariants();
		CHECK_MESSAGE(TEXT("Serialize Zeroes Underflow"), SpacerCopies[2] == Spacer);
		CHECK_MESSAGE(TEXT("Serialize Ones"), ArrZeroes == ArrZeroesCopy);
		ArrZeroesCopy.CheckInvariants();
		CHECK_MESSAGE(TEXT("Serialize Zeroes Overflow"), SpacerCopies[3] == Spacer);

		TArray<uint8> Bytes2;
		TBitArray<> ArrSmall(true, 16);
		{
			FMemoryWriter Writer(Bytes2);
			Writer << ArrSmall;
		}
		TBitArray<TInlineAllocator<4>> Arr;
		int32 InitialMax = Arr.Max();
		Arr.Add(true, NumBitsPerDWORD * 8);
		Arr.CheckInvariants();
		{
			FMemoryReader Reader(Bytes2);
			Reader << Arr;
		}
		CHECK_MESSAGE(TEXT("Serialize from a dynamic allocation with an inline allocator sets num down to the size of the loaded array"),
			Arr.Num() == ArrSmall.Num());
		CHECK_MESSAGE(TEXT("Serialize from a dynamic allocation with an inline allocator sets max back to the size of the inline allocation"),
			Arr.Max() == InitialMax);
	}

	void TestAdd()
	{
		using namespace UE::BitArrayTest;
		// Add one bit
		{
			TBitArray<FDefaultAllocator> Arr;
			CHECK_MESSAGE(TEXT("With DefaultAllocator MaxBits starts at 0"), Arr.Max() == 0);
			for (int n = 0; n < 10; ++n)
			{
				Arr.Add(n % 3 == 0);
				Arr.CheckInvariants();
			}
			CHECK_MESSAGE(TEXT("Add one bit size"), Arr.Num() == 10);
			for (int n = 0; n < 10; ++n)
			{
				CHECK_MESSAGE(TEXT("Add one bit elements"), Arr[n] == (n % 3 == 0));
			}

			Arr.RemoveAt(0, 10);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Removed all elements leaves size at 0"), Arr.Num() == 0);
			CHECK_MESSAGE(TEXT("Removed all elements keeps max at original"), Arr.Max() >= 10);
			for (int n = 0; n < 10; ++n)
			{
				Arr.Add(n % 2 == 0);
				Arr.CheckInvariants();
			}
			CHECK_MESSAGE(TEXT("Add one bit no resize size"), Arr.Num() == 10);
			for (int n = 0; n < 10; ++n)
			{
				CHECK_MESSAGE(TEXT("Add one bit no resize elements"), Arr[n] == (n % 2 == 0));
			}
		}

		// Add that takes a bool value and replicates it to multiple bits
		{
			TBitArray<FDefaultAllocator> ArrTrueCopy;
			TBitArray<FDefaultAllocator> ArrFalseCopy;
			ArrTrueCopy.Add(true, ArrTrue.Num());
			ArrTrueCopy.CheckInvariants();
			ArrFalseCopy.Add(false, ArrFalse.Num());
			ArrFalseCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add multiple true bits from empty into unallocated space"), AreEqual(ArrTrueCopy, ArrTrue));
			CHECK_MESSAGE(TEXT("Add multiple false bits from empty into unallocated space"), AreEqual(ArrFalseCopy, ArrFalse));

			ArrTrueCopy.RemoveAt(0, ArrTrueCopy.Num());
			ArrTrueCopy.CheckInvariants();
			ArrTrueCopy.Add(true, ArrTrue.Num());
			ArrTrueCopy.CheckInvariants();
			ArrFalseCopy.RemoveAt(0, ArrFalseCopy.Num());
			ArrFalseCopy.CheckInvariants();
			ArrFalseCopy.Add(false, ArrFalse.Num());
			ArrFalseCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add multiple true bits from empty into previously-allocated space"), AreEqual(ArrTrueCopy, ArrTrue));
			CHECK_MESSAGE(TEXT("Add multiple false bits from empty into previously-allocated space"), AreEqual(ArrFalseCopy, ArrFalse));

			TBitArray<> ArrTrue2(true, 5);
			TBitArray<> ArrTrue3(true, 10);
			TBitArray<> ArrFalse2(false, 5);
			TBitArray<> ArrFalse3(false, 10);

			ArrTrue2.Add(true, 5);
			ArrTrue2.CheckInvariants();
			ArrFalse2.Add(false, 5);
			ArrFalse2.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add multiple true bits on non-empty"), ArrTrue2 == ArrTrue3);
			CHECK_MESSAGE(TEXT("Add multiple false bits on non-empty"), ArrFalse2 == ArrFalse3);
		}

		// AddUninitialized
		{
			TBitArray<FDefaultAllocator> ArrTrueCopy;
			TBitArray<FDefaultAllocator> ArrFalseCopy;
			ArrTrueCopy.AddUninitialized(ArrTrue.Num());
			ArrTrueCopy.CheckInvariants();
			ArrFalseCopy.AddUninitialized(ArrFalse.Num());
			ArrFalseCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("AddUninitialized multiple true bits from empty into unallocated space"), ArrTrueCopy.Num() == ArrTrue.Num());
			CHECK_MESSAGE(TEXT("AddUninitialized multiple false bits from empty into unallocated space"), ArrFalseCopy.Num() == ArrFalse.Num());

			ArrTrueCopy.RemoveAt(0, ArrTrueCopy.Num());
			ArrTrueCopy.CheckInvariants();
			ArrTrueCopy.AddUninitialized(ArrTrue.Num());
			ArrTrueCopy.CheckInvariants();
			ArrFalseCopy.RemoveAt(0, ArrFalseCopy.Num());
			ArrFalseCopy.CheckInvariants();
			ArrFalseCopy.AddUninitialized(ArrFalse.Num());
			ArrFalseCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("AddUninitialized multiple true bits from empty into previously-allocated space"),
				ArrTrueCopy.Num() == ArrTrue.Num());
			CHECK_MESSAGE(TEXT("AddUninitialized multiple false bits from empty into previously-allocated space"),
				ArrFalseCopy.Num() == ArrFalse.Num());

			TBitArray<> ArrTrue2(true, 5);
			TBitArray<> ArrTrue3(true, 10);
			TBitArray<> ArrFalse2(false, 5);
			TBitArray<> ArrFalse3(false, 10);

			ArrTrue2.AddUninitialized(5);
			ArrTrue2.CheckInvariants();
			ArrFalse2.AddUninitialized(5);
			ArrFalse2.CheckInvariants();
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("AddUninitialized multiple true bits on non-empty"), ArrTrue2[n]);
				CHECK_MESSAGE(TEXT("AddUninitialized multiple false bits on non-empty"), !(ArrFalse2[n]));
			}
		}
	}

	void TestAddFromRange()
	{
		// AddRange that takes a uint32*
		{
			TBitArray<> Arr;
			Arr.AddRange(&GrowingTrueInt, NumGrowingTrue);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add from uint32 with ReadOffset 0 from empty"), Arr == ArrGrowingTrue);

			Arr.AddRange(&GrowingTrueInt, NumGrowingTrue);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add from uint32 with ReadOffset 0 to nonempty"), Arr == ArrGrowingTrueTwice);

			TBitArray<> Arr2;
			uint32 AllZeroes = 0;
			uint32 AllOnes = 0xffffffff;
			Arr2.AddRange(&AllZeroes, 10);
			Arr2.CheckInvariants();
			Arr2.AddRange(&AllOnes, 10);
			Arr2.CheckInvariants();
			for (int n = 0; n < 20; ++n)
			{
				CHECK_MESSAGE(TEXT("Add from uint32 with ReadOffset 0, Zeroes, Then Ones"), Arr2[n] == (n >= 10));
			}

			TBitArray<> Arr3;
			uint32 MultipleInts[] = { 0xffff0000, 0x0f0f0f0f };
			Arr3.AddRange(MultipleInts, 64);
			Arr3.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add from uint32 with ReadOffset 0, size"), Arr3.Num() == 64);
			for (int n = 0; n < 32; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Add from uint32 with ReadOffset 0, MultipleInts, %d"), n).ToString(), Arr3[n] == (n >= 16));
			}
			for (int n = 32; n < 64; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Add from uint32 with ReadOffset 0, MultipleInts, %d"), n).ToString(), Arr3[n] == ((n / 4) % 2 == 0));
			}

			TBitArray<> Arr4;
			Arr4.AddRange(MultipleInts, 32, 16);
			Arr4.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add from uint32 with ReadOffset 16, MultipleInts, size"), Arr4.Num() == 32);
			for (int n = 0; n < 16; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Add from uint32 with ReadOffset 16, MultipleInts, %d"), n).ToString(), Arr4[n] == true);
			}
			for (int n = 16; n < 32; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Add from uint32 with ReadOffset 16, MultipleInts, %d"), n).ToString(), Arr4[n] == ((n / 4) % 2 == 0));
			}
		}

		// Add that takes a bitarray
		{
			TBitArray<> Arr;
			Arr.AddRange(ArrGrowingTrue, NumGrowingTrue);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add from BitArray with ReadOffset 0 from empty"), Arr == ArrGrowingTrue);

			Arr.AddRange(ArrGrowingTrue, NumGrowingTrue);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add from BitArray with ReadOffset 0 to nonempty"), Arr == ArrGrowingTrueTwice);

			TBitArray<> Arr2;
			Arr2.AddRange(ArrFalse, 10);
			Arr2.CheckInvariants();
			Arr2.AddRange(ArrTrue, 10);
			Arr2.CheckInvariants();
			for (int n = 0; n < 20; ++n)
			{
				CHECK_MESSAGE(TEXT("Add from BitArray with ReadOffset 0, Zeroes, Then Ones"), Arr2[n] == (n >= 10));
			}

			TBitArray<> Arr3;
			uint32 MultipleInts[] = { 0xffff0000, 0x0f0f0f0f };
			TBitArray<> ArrMultipleInts;
			ArrMultipleInts.AddRange(MultipleInts, 64);
			ArrMultipleInts.CheckInvariants();
			Arr3.AddRange(ArrMultipleInts, 64);
			Arr3.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add from BitArray with ReadOffset 0"), Arr3 == ArrMultipleInts);

			TBitArray<> Arr4;
			Arr4.AddRange(ArrMultipleInts, 32, 16);
			Arr4.CheckInvariants();
			CHECK_MESSAGE(TEXT("Add from BitArray with ReadOffset 16, MultipleInts, size"), Arr4.Num() == 32);
			for (int n = 0; n < 16; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Add from BitArray with ReadOffset 16, MultipleInts, %d"), n).ToString(), Arr4[n] == true);
			}
			for (int n = 16; n < 32; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Add from BitArray with ReadOffset 16, MultipleInts, %d"), n).ToString(),
					Arr4[n] == ((n / 4) % 2 == 0));
			}
		}
	}

	void TestInsert()
	{
		using namespace UE::BitArrayTest;
		// Insert one bit
		{
			TBitArray<FDefaultAllocator> Arr;
			CHECK_MESSAGE(TEXT("With DefaultAllocator MaxBits starts at 0"), Arr.Max() == 0);
			for (int n = 0; n < 10; ++n)
			{
				Arr.Insert(n % 3 == 0, Arr.Num());
				Arr.CheckInvariants();
			}
			CHECK_MESSAGE(TEXT("Insert one bit at end size"), Arr.Num() == 10);
			for (int n = 0; n < 10; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert one bit at end elements"), Arr[n] == (n % 3 == 0));
			}

			Arr.Insert(false, 5);
			Arr.CheckInvariants();
			Arr.Insert(true, 5);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert one bit in middle size"), Arr.Num() == 12);
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert one bit in middle elements"), Arr[n] == (n % 3 == 0));
			}
			CHECK_MESSAGE(TEXT("Insert one bit in middle elements"), Arr[5] == true);
			CHECK_MESSAGE(TEXT("Insert one bit in middle elements"), Arr[6] == false);
			for (int n = 7; n < 12; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert one bit in middle elements"), Arr[n] == ((n - 2) % 3 == 0));
			}

			Arr.RemoveAt(0, 12);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Removed all elements leaves size at 0"), Arr.Num() == 0);
			CHECK_MESSAGE(TEXT("Removed all elements keeps max at original"), Arr.Max() >= 12);
			for (int n = 0; n < 10; ++n)
			{
				Arr.Insert(n % 2 == 0, Arr.Num());
				Arr.CheckInvariants();
			}
			CHECK_MESSAGE(TEXT("Insert one bit at end no resize size"), Arr.Num() == 10);
			for (int n = 0; n < 10; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert one bit at end no resize elements"), Arr[n] == (n % 2 == 0));
			}

			Arr.Insert(false, 5);
			Arr.CheckInvariants();
			Arr.Insert(true, 5);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert one bit in middle no resize size"), Arr.Num() == 12);
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert one bit in middle elements"), Arr[n] == (n % 2 == 0));
			}
			CHECK_MESSAGE(TEXT("Insert one bit in middle no resize elements"), Arr[5] == true);
			CHECK_MESSAGE(TEXT("Insert one bit in middle no resize elements"), Arr[6] == false);
			for (int n = 7; n < 12; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert one bit in middle no resize elements"), Arr[n] == ((n - 2) % 2 == 0));
			}
		}

		// Insert that takes a bool value and replicates it to multiple bits
		{
			TBitArray<FDefaultAllocator> ArrTrueCopy;
			TBitArray<FDefaultAllocator> ArrFalseCopy;
			ArrTrueCopy.Insert(true, 0, ArrTrue.Num());
			ArrTrueCopy.CheckInvariants();
			ArrFalseCopy.Insert(false, 0, ArrFalse.Num());
			ArrFalseCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert multiple true bits at end from empty into unallocated space"), AreEqual(ArrTrueCopy, ArrTrue));
			CHECK_MESSAGE(TEXT("Insert multiple false bits at end from empty into unallocated space"), AreEqual(ArrFalseCopy, ArrFalse));

			ArrTrueCopy.Insert(false, 5, 5);
			ArrTrueCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert multiple bits in middle from empty into unallocated space size"), ArrTrueCopy.Num() == (ArrTrue.Num() + 5));
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert multiple bits in middle from empty into unallocated space elements"), ArrTrueCopy[n] == true);
			}
			for (int n = 5; n < 10; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert multiple bits in middle from empty into unallocated space elements"), ArrTrueCopy[n] == false);
			}
			for (int n = 10; n < 5 + ArrTrue.Num(); ++n)
			{
				CHECK_MESSAGE(TEXT("Insert multiple bits in middle from empty into unallocated space elements"), ArrTrueCopy[n] == true);
			}

			ArrTrueCopy.RemoveAt(0, ArrTrueCopy.Num());
			ArrTrueCopy.CheckInvariants();
			ArrTrueCopy.Insert(true, 0, ArrTrue.Num());
			ArrTrueCopy.CheckInvariants();
			ArrFalseCopy.RemoveAt(0, ArrFalseCopy.Num());
			ArrFalseCopy.CheckInvariants();
			ArrFalseCopy.Insert(false, 0, ArrFalse.Num());
			ArrFalseCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert multiple true bits at end from empty into previously-allocated space"), AreEqual(ArrTrueCopy, ArrTrue));
			CHECK_MESSAGE(TEXT("Insert multiple false bits at end from empty into previously-allocated space"), AreEqual(ArrFalseCopy, ArrFalse));
			ArrTrueCopy.Insert(true, 5, 3);
			ArrTrueCopy.CheckInvariants();
			ArrTrueCopy.Insert(false, 5, 2);
			ArrTrueCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert multiple bits in middle from empty into previously-allocated size"), ArrTrueCopy.Num() == (ArrTrue.Num() + 5));
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert multiple bits in middle from empty into previously-allocated elements"), ArrTrueCopy[n] == true);
			}
			for (int n = 5; n < 7; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert multiple bits in middle from empty into previously-allocated elements"), ArrTrueCopy[n] == false);
			}
			for (int n = 7; n < 5 + ArrTrue.Num(); ++n)
			{
				CHECK_MESSAGE(TEXT("Insert multiple bits in middle from empty into previously-allocated elements"), ArrTrueCopy[n] == true);
			}

			TBitArray<> ArrTrue2(true, 5);
			TBitArray<> ArrTrue3(true, 10);
			TBitArray<> ArrFalse2(false, 5);
			TBitArray<> ArrFalse3(false, 10);

			ArrTrue2.Insert(true, ArrTrue2.Num(), 5);
			ArrTrue2.CheckInvariants();
			ArrFalse2.Insert(false, ArrFalse2.Num(), 5);
			ArrFalse2.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert multiple true bits at end on non-empty"), ArrTrue2 == ArrTrue3);
			CHECK_MESSAGE(TEXT("Insert multiple false bits at end on non-empty"), ArrFalse2 == ArrFalse3);

			TBitArray<> ArrTrue4(true, 5);
			TBitArray<> ArrFalse4(false, 5);

			ArrTrue4.Insert(true, 1, 5);
			ArrTrue4.CheckInvariants();
			ArrFalse4.Insert(false, 1, 5);
			ArrFalse4.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert multiple true bits at middle on non-empty"), ArrTrue4 == ArrTrue3);
			CHECK_MESSAGE(TEXT("Insert multiple false bits at middle on non-empty"), ArrFalse4 == ArrFalse3);
		}

		// InsertUninitialized
		{
			TBitArray<FDefaultAllocator> Arr;
			Arr.InsertUninitialized(0, ArrTrue.Num());
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("InsertUninitialized multiple bits at end from empty into unallocated space"), Arr.Num() == ArrTrue.Num());

			TBitArray<FDefaultAllocator> ArrTrueDefaultAlloc(true, ArrTrue.Num());
			TBitArray<FDefaultAllocator> ArrFalseDefaultAlloc(false, ArrFalse.Num());

			TBitArray<FDefaultAllocator> ArrTrueCopy(ArrTrueDefaultAlloc);
			TBitArray<FDefaultAllocator> ArrFalseCopy(ArrFalseDefaultAlloc);
			check(ArrTrue.Num() == ArrFalse.Num());
			ArrTrueCopy.InsertUninitialized(5, 5);
			ArrTrueCopy.CheckInvariants();
			ArrFalseCopy.InsertUninitialized(5, 5);
			ArrFalseCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("InsertUninitialized multiple bits in middle from empty into unallocated space size"), ArrTrueCopy.Num() == (ArrTrue.Num() + 5));
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("InsertUninitialized multiple bits in middle from empty into unallocated space elements"), 
					ArrTrueCopy[n] == true);
				CHECK_MESSAGE(TEXT("InsertUninitialized multiple bits in middle from empty into unallocated space elements"),
					ArrFalseCopy[n] == false);
			}
			for (int n = 10; n < 5 + ArrTrue.Num(); ++n)
			{
				CHECK_MESSAGE(TEXT("InsertUninitialized multiple bits in middle from empty into unallocated space elements"),
					ArrTrueCopy[n] == true);
				CHECK_MESSAGE(TEXT("InsertUninitialized multiple bits in middle from empty into unallocated space elements"),
					ArrFalseCopy[n] == false);
			}

			ArrTrueCopy.RemoveAt(0, ArrTrueCopy.Num());
			ArrTrueCopy.CheckInvariants();
			ArrTrueCopy.InsertUninitialized(0, ArrTrue.Num());
			ArrTrueCopy.CheckInvariants();
			ArrFalseCopy.RemoveAt(0, ArrFalseCopy.Num());
			ArrFalseCopy.CheckInvariants();
			ArrFalseCopy.InsertUninitialized(0, ArrFalse.Num());
			ArrFalseCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("InsertUninitialized multiple true bits at end from empty into previously-allocated space"),
				ArrTrueCopy.Num() == ArrTrue.Num());
			CHECK_MESSAGE(TEXT("InsertUninitialized multiple false bits at end from empty into previously-allocated space"),
				ArrFalseCopy.Num() == ArrFalse.Num());
			ArrTrueCopy.RemoveAt(0, ArrTrueCopy.Num());
			ArrTrueCopy.CheckInvariants();
			ArrTrueCopy.Insert(true, 0, ArrTrue.Num());
			ArrTrueCopy.CheckInvariants();
			ArrTrueCopy.InsertUninitialized(5, 5);
			ArrTrueCopy.CheckInvariants();
			CHECK_MESSAGE(TEXT("InsertUninitialized multiple bits in middle from empty into previously-allocated size"),
				ArrTrueCopy.Num() == (ArrTrue.Num() + 5));
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("InsertUninitialized multiple bits in middle from empty into previously-allocated elements"),
					ArrTrueCopy[n] == true);
			}
			for (int n = 10; n < 5 + ArrTrue.Num(); ++n)
			{
				CHECK_MESSAGE(TEXT("InsertUninitialized multiple bits in middle from empty into previously-allocated elements"),
					ArrTrueCopy[n] == true);
			}

			TBitArray<> ArrTrue2(true, 5);
			TBitArray<> ArrFalse2(false, 5);

			ArrTrue2.InsertUninitialized(ArrTrue2.Num(), 5);
			ArrTrue2.CheckInvariants();
			ArrFalse2.InsertUninitialized(ArrFalse2.Num(), 5);
			ArrFalse2.CheckInvariants();
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("InsertUninitialized multiple true bits at end on non-empty"), ArrTrue2[n] == true);
				CHECK_MESSAGE(TEXT("InsertUninitialized multiple false bits at end on non-empty"), ArrFalse2[n] == false);
			}

			TBitArray<> ArrTrue4(true, 5);
			TBitArray<> ArrFalse4(false, 5);

			ArrTrue4.InsertUninitialized(1, 5);
			ArrTrue4.CheckInvariants();
			ArrFalse4.InsertUninitialized(1, 5);
			ArrFalse4.CheckInvariants();
			for (int n = 0; n < 10; ++n)
			{
				if (n < 1 || n >= 6)
				{
					CHECK_MESSAGE(TEXT("InsertUninitialized multiple true bits at end on non-empty"), ArrTrue4[n] == true);
					CHECK_MESSAGE(TEXT("InsertUninitialized multiple false bits at end on non-empty"), ArrFalse4[n] == false);
				}
			}
		}
	}

	void TestInsertFromRange()
	{
		// InsertRange that takes a uint32*
		{
			TBitArray<> Arr;
			Arr.InsertRange(&GrowingTrueInt, 0, NumGrowingTrue);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at end from empty"), Arr == ArrGrowingTrue);

			Arr.InsertRange(&GrowingTrueInt, Arr.Num(), NumGrowingTrue);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at end to nonempty"), Arr == ArrGrowingTrueTwice);

			TBitArray<> Arr1Insert2;
			Arr1Insert2.InsertRange(&GrowingTrueInt, 0, NumGrowingTrue);
			Arr1Insert2.CheckInvariants();
			Arr1Insert2.InsertRange(&GrowingTrueInt, 0, NumGrowingTrue);
			Arr1Insert2.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at beginning to nonempty"), Arr1Insert2 == ArrGrowingTrueTwice);

			uint32 AllZeroes = 0;
			uint32 AllOnes = 0xffffffff;
			TBitArray<> Arr1Insert3;
			Arr1Insert3.InsertRange(&GrowingTrueInt, 0, NumGrowingTrue);
			Arr1Insert3.CheckInvariants();
			Arr1Insert3.InsertRange(&AllZeroes, 5, 5);
			Arr1Insert3.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle to nonempty size"), Arr1Insert3.Num() == (NumGrowingTrue + 5));
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle to nonempty"), Arr1Insert3[n] == ArrGrowingTrue[n]);
			}
			for (int n = 5; n < 10; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle to nonempty"), Arr1Insert3[n] == false);
			}
			for (int n = 10; n < NumGrowingTrue + 5; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle to nonempty"), Arr1Insert3[n] == ArrGrowingTrue[n - 5]);
			}

			TBitArray<> Arr2;
			Arr2.InsertRange(&AllZeroes, 0, 10);
			Arr2.CheckInvariants();
			Arr2.InsertRange(&AllOnes, Arr2.Num(), 10);
			Arr2.CheckInvariants();
			for (int n = 0; n < 20; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at end, Zeroes, Then Ones"), Arr2[n] == (n >= 10));
			}

			TBitArray<> Arr2Insert1;
			Arr2Insert1.InsertRange(&AllZeroes, 0, 10);
			Arr2Insert1.CheckInvariants();
			Arr2Insert1.InsertRange(&AllOnes, 0, 10);
			Arr2Insert1.CheckInvariants();
			for (int n = 0; n < 20; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at beginning, Zeroes, Then Ones"), Arr2Insert1[n] == (n < 10));
			}
			TBitArray<> Arr2Insert2;
			Arr2Insert2.InsertRange(&AllZeroes, 0, 10);
			Arr2Insert2.CheckInvariants();
			Arr2Insert2.InsertRange(&AllOnes, 5, 10);
			Arr2Insert2.CheckInvariants();
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle, Zeroes, Then Ones"), Arr2Insert2[n] == false);
			}
			for (int n = 5; n < 15; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle, Zeroes, Then Ones"), Arr2Insert2[n] == true);
			}
			for (int n = 15; n < 20; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle, Zeroes, Then Ones"), Arr2Insert2[n] == false);
			}

			TBitArray<> Arr3;
			uint32 MultipleInts[] = { 0xffff0000, 0x0f0f0f0f };
			Arr3.InsertRange(MultipleInts, 0, 64);
			Arr3.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at end, MutlipleInts, size"), Arr3.Num() == 64);
			for (int n = 0; n < 32; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 0 at end, MultipleInts, %d"), n).ToString(),
					Arr3[n] == (n >= 16));
			}
			for (int n = 32; n < 64; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 0 at end, MultipleInts, %d"), n).ToString(),
					Arr3[n] == ((n / 4) % 2 == 0));
			}

			TBitArray<> Arr4;
			Arr4.InsertRange(MultipleInts, 0, 32, 16);
			Arr4.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 16 at end, MultipleInts, size"), Arr4.Num() == 32);
			for (int n = 0; n < 16; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at end, MultipleInts, %d"), n).ToString(),
					Arr4[n] == true);
			}
			for (int n = 16; n < 32; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at end, MultipleInts, %d"), n).ToString(),
					Arr4[n] == ((n / 4) % 2 == 0));
			}

			TBitArray<> Arr5;
			uint32 MultipleIntsBackwards[] = { 0x0f0f0f0f, 0xffff0000 };
			Arr5.InsertRange(MultipleInts, 0, 32, 16);
			Arr5.CheckInvariants();
			Arr5.InsertRange(MultipleIntsBackwards, 0, 32, 16);
			Arr5.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 16 at beginning, MultipleInts, size"), Arr5.Num() == 64);
			for (int n = 0; n < 16; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at beginning, MultipleInts, %d"), n).ToString(),
					Arr5[n] == ((n / 4) % 2 == 0));
			}
			for (int n = 16; n < 32; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at beginning, MultipleInts, %d"), n).ToString(),
					Arr5[n] == false);
			}
			for (int n = 32; n < 48; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at beginning, MultipleInts, %d"), n).ToString(),
					Arr5[n] == true);
			}
			for (int n = 48; n < 64; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at beginning, MultipleInts, %d"), n).ToString(),
					Arr5[n] == ((n / 4) % 2 == 0));
			}
		}

		// Insert that takes a bitarray
		{
			TBitArray<> Arr;
			Arr.InsertRange(ArrGrowingTrue, 0, NumGrowingTrue);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at end from empty"), Arr == ArrGrowingTrue);

			Arr.InsertRange(ArrGrowingTrue, Arr.Num(), NumGrowingTrue);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at end to nonempty"), Arr == ArrGrowingTrueTwice);

			TBitArray<> Arr1Insert2;
			Arr1Insert2.InsertRange(ArrGrowingTrue, 0, NumGrowingTrue);
			Arr1Insert2.CheckInvariants();
			Arr1Insert2.InsertRange(ArrGrowingTrue, 0, NumGrowingTrue);
			Arr1Insert2.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at beginning to nonempty"), Arr1Insert2 == ArrGrowingTrueTwice);

			TBitArray<> Arr1Insert3;
			Arr1Insert3.InsertRange(ArrGrowingTrue, 0, NumGrowingTrue);
			Arr1Insert3.CheckInvariants();
			Arr1Insert3.InsertRange(ArrFalse, 5, 5);
			Arr1Insert3.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle to nonempty size"), Arr1Insert3.Num() == (NumGrowingTrue + 5));
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle to nonempty"), Arr1Insert3[n] == ArrGrowingTrue[n]);
			}
			for (int n = 5; n < 10; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle to nonempty"), Arr1Insert3[n] == false);
			}
			for (int n = 10; n < NumGrowingTrue + 5; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle to nonempty"), Arr1Insert3[n] == ArrGrowingTrue[n - 5]);
			}

			TBitArray<> Arr2;
			Arr2.InsertRange(ArrFalse, 0, 10);
			Arr2.CheckInvariants();
			Arr2.InsertRange(ArrTrue, Arr2.Num(), 10);
			Arr2.CheckInvariants();
			for (int n = 0; n < 20; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at end, Zeroes, Then Ones"), Arr2[n] == (n >= 10));
			}

			TBitArray<> Arr2Insert1;
			Arr2Insert1.InsertRange(ArrFalse, 0, 10);
			Arr2Insert1.CheckInvariants();
			Arr2Insert1.InsertRange(ArrTrue, 0, 10);
			Arr2Insert1.CheckInvariants();
			for (int n = 0; n < 20; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at beginning, Zeroes, Then Ones"), Arr2Insert1[n] == (n < 10));
			}
			TBitArray<> Arr2Insert2;
			Arr2Insert2.InsertRange(ArrFalse, 0, 10);
			Arr2Insert2.CheckInvariants();
			Arr2Insert2.InsertRange(ArrTrue, 5, 10);
			Arr2Insert2.CheckInvariants();
			for (int n = 0; n < 5; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle, Zeroes, Then Ones"), Arr2Insert2[n] == false);
			}
			for (int n = 5; n < 15; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle, Zeroes, Then Ones"), Arr2Insert2[n] == true);
			}
			for (int n = 15; n < 20; ++n)
			{
				CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at middle, Zeroes, Then Ones"), Arr2Insert2[n] == false);
			}

			TBitArray<> Arr3;
			uint32 MultipleInts[] = { 0xffff0000, 0x0f0f0f0f };
			TBitArray<> ArrMultipleInts;
			ArrMultipleInts.InsertRange(MultipleInts, 0, 64);
			ArrMultipleInts.CheckInvariants();
			Arr3.InsertRange(ArrMultipleInts, 0, 64);
			Arr3.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 0 at end, MutlipleInts, size"), Arr3 == ArrMultipleInts);

			TBitArray<> Arr4;
			Arr4.InsertRange(ArrMultipleInts, 0, 32, 16);
			Arr4.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 16 at end, MultipleInts, size"), Arr4.Num() == 32);
			for (int n = 0; n < 16; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at end, MultipleInts, %d"), n).ToString(),
					Arr4[n] == true);
			}
			for (int n = 16; n < 32; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at end, MultipleInts, %d"), n).ToString(),
					Arr4[n] == ((n / 4) % 2 == 0));
			}

			TBitArray<> Arr5;
			uint32 MultipleIntsBackwards[] = { 0x0f0f0f0f, 0xffff0000 };
			TBitArray<> ArrMultipleIntsBackwards;
			ArrMultipleIntsBackwards.InsertRange(MultipleIntsBackwards, 0, 64);
			ArrMultipleIntsBackwards.CheckInvariants();
			Arr5.InsertRange(ArrMultipleInts, 0, 32, 16);
			Arr5.CheckInvariants();
			Arr5.InsertRange(ArrMultipleIntsBackwards, 0, 32, 16);
			Arr5.CheckInvariants();
			CHECK_MESSAGE(TEXT("Insert from uint32 with ReadOffset 16 at beginning, MultipleInts, size"), Arr5.Num() == 64);
			for (int n = 0; n < 16; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at beginning, MultipleInts, %d"), n).ToString(),
					Arr5[n] == ((n / 4) % 2 == 0));
			}
			for (int n = 16; n < 32; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at beginning, MultipleInts, %d"), n).ToString(),
					Arr5[n] == false);
			}
			for (int n = 32; n < 48; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at beginning, MultipleInts, %d"), n).ToString(),
					Arr5[n] == true);
			}
			for (int n = 48; n < 64; ++n)
			{
				Label.Reset();
				CHECK_MESSAGE(Label.Appendf(TEXT("Insert from uint32 with ReadOffset 16 at beginning, MultipleInts, %d"), n).ToString(),
					Arr5[n] == ((n / 4) % 2 == 0));
			}
		}
	}

	void TestSetRange()
	{
		// SetRange that takes a bool value
		{
			constexpr int MaxDataLength = 20;
			TCHAR OnesExpectedText[MaxDataLength * NumBitsPerDWORD];
			TCHAR ZeroesExpectedText[MaxDataLength * NumBitsPerDWORD];
			TCHAR ZeroesText[MaxDataLength * NumBitsPerDWORD];
			TCHAR OnesText[MaxDataLength * NumBitsPerDWORD];
			constexpr int UnderflowTestBits = NumBitsPerDWORD;
			constexpr int OverflowTestBits = NumBitsPerDWORD;

			int32 BitWidths[] = { 1,2,3,4,15,16,17,31,32,33,63,64,5 * 32 - 1,5 * 32,5 * 32 + 1,5 * 32 + 3,5 * 32 + 4,5 * 32 + 5,5 * 32 + 15,5 * 32 + 16,5 * 32 + 17 };
			for (int32 BitWidth : BitWidths)
			{
				int32 WriteIndexes[] = { 0,1,2,3,4,7,8,9,15,16,17,23,24,25,31,32,33,47,48,49,63,64,65,5 * 32 - 1,5 * 32,5 * 32 + 1,5 * 32 + 15 };
				for (int32 WriteOffset : WriteIndexes)
				{
					TBitArray<> ArrZeroes(false, MaxDataLength * NumBitsPerDWORD);
					TBitArray<> ArrOnes(true, MaxDataLength * NumBitsPerDWORD);

					ArrZeroes.SetRange(WriteOffset, BitWidth, true);
					ArrZeroes.CheckInvariants();
					ArrOnes.SetRange(WriteOffset, BitWidth, false);
					ArrOnes.CheckInvariants();

					bool bOnesMatchesExpected = true;
					bool bZeroesMatchesExpected = true;

					int UnderflowTestStart = WriteOffset > UnderflowTestBits ? WriteOffset - UnderflowTestBits : 0;
					int VerifyEnd = WriteOffset + BitWidth + OverflowTestBits;
					int DisplayIndex = 0;
					for (int ResultIndex = UnderflowTestStart; ResultIndex < VerifyEnd; ++ResultIndex, ++DisplayIndex)
					{
						bool ZeroesExpectedValue, OnesExpectedValue;
						if (ResultIndex < WriteOffset || WriteOffset + BitWidth <= ResultIndex)
						{
							ZeroesExpectedValue = false;
							OnesExpectedValue = true;
						}
						else
						{
							ZeroesExpectedValue = true;
							OnesExpectedValue = false;
						}
						bool ZeroesValue = ArrZeroes[ResultIndex];
						bool OnesValue = ArrOnes[ResultIndex];
						bOnesMatchesExpected = bOnesMatchesExpected & (OnesValue == OnesExpectedValue);
						bZeroesMatchesExpected = bZeroesMatchesExpected & (ZeroesValue == ZeroesExpectedValue);
						OnesExpectedText[DisplayIndex] = OnesExpectedValue ? TEXT('1') : TEXT('0');
						OnesText[DisplayIndex] = OnesValue ? TEXT('1') : TEXT('0');
						ZeroesExpectedText[DisplayIndex] = ZeroesExpectedValue ? TEXT('1') : TEXT('0');
						ZeroesText[DisplayIndex] = ZeroesValue ? TEXT('1') : TEXT('0');
					}
					OnesExpectedText[DisplayIndex] = TEXT('\0');
					OnesText[DisplayIndex] = TEXT('\0');
					ZeroesExpectedText[DisplayIndex] = TEXT('\0');
					ZeroesText[DisplayIndex] = TEXT('\0');
					if (!bOnesMatchesExpected)
					{
						CHECK_MESSAGE(FString::Printf(TEXT("SetRange bool BitWidth=%d WriteOffset=%d, Ones\nExpected=%s\nActual  =%s"), BitWidth, WriteOffset, OnesExpectedText, OnesText),
							bOnesMatchesExpected);
					}
					else if (!bZeroesMatchesExpected)
					{
						CHECK_MESSAGE(FString::Printf(TEXT("SetRange bool BitWidth=%d WriteOffset=%d, Zeroes\nExpected=%s\nActual  =%s"), BitWidth, WriteOffset, ZeroesExpectedText, ZeroesText),
							bZeroesMatchesExpected);
					}
				}
			}
		}

		// SetRangeFromRange
		{
			// SetRangeFromRange is a pass through to MemmoveBitsWordOrder, which has its own set of tests. We just test a single case for setting range at 0,middle,end
			uint32 ZeroesInt = 0;
			uint32 OnesInt = MAX_uint32;
			TBitArray<> ArrZeroes(false, NumBitsPerDWORD);
			TBitArray<> ArrOnes(true, NumBitsPerDWORD);
			TBitArray<> ArrRefZeroes(false, NumBitsPerDWORD);
			TBitArray<> ArrRefOnes(true, NumBitsPerDWORD);

			ArrZeroes.SetRangeFromRange(0, 10, &OnesInt, 0);
			ArrZeroes.CheckInvariants();
			ArrZeroes.SetRangeFromRange(10, 10, &OnesInt, 10);
			ArrZeroes.CheckInvariants();
			ArrZeroes.SetRangeFromRange(20, 12, &OnesInt, 20);
			ArrZeroes.CheckInvariants();
			ArrOnes.SetRangeFromRange(0, 10, &ZeroesInt, 0);
			ArrOnes.CheckInvariants();
			ArrOnes.SetRangeFromRange(10, 10, &ZeroesInt, 10);
			ArrOnes.CheckInvariants();
			ArrOnes.SetRangeFromRange(20, 12, &ZeroesInt, 20);
			ArrOnes.CheckInvariants();

			CHECK_MESSAGE(TEXT("SetRangeFromRange Zeroes"), ArrZeroes == ArrRefOnes);
			CHECK_MESSAGE(TEXT("SetRangeFromRange Ones"), ArrOnes == ArrRefZeroes);
		}

		// SetRange that takes a bitarray
		{
			// SetRange that takes a bitarray is a pass through to MemmoveBitsWordOrder, which has its own set of tests. We just test a single case for setting range at 0,middle,end
			TBitArray<> ArrZeroes(false, NumBitsPerDWORD);
			TBitArray<> ArrOnes(true, NumBitsPerDWORD);
			TBitArray<> ArrRefZeroes(false, NumBitsPerDWORD);
			TBitArray<> ArrRefOnes(true, NumBitsPerDWORD);

			ArrZeroes.SetRangeFromRange(0, 10, ArrRefOnes, 0);
			ArrZeroes.CheckInvariants();
			ArrZeroes.SetRangeFromRange(10, 10, ArrRefOnes, 10);
			ArrZeroes.CheckInvariants();
			ArrZeroes.SetRangeFromRange(20, 12, ArrRefOnes, 20);
			ArrZeroes.CheckInvariants();
			ArrOnes.SetRangeFromRange(0, 10, ArrRefZeroes, 0);
			ArrOnes.CheckInvariants();
			ArrOnes.SetRangeFromRange(10, 10, ArrRefZeroes, 10);
			ArrOnes.CheckInvariants();
			ArrOnes.SetRangeFromRange(20, 12, ArrRefZeroes, 20);
			ArrOnes.CheckInvariants();

			CHECK_MESSAGE(TEXT("SetRange bitarray Zeroes"), ArrZeroes == ArrRefOnes);
			CHECK_MESSAGE(TEXT("SetRange bitarray Ones"), ArrOnes == ArrRefZeroes);
		}
	}

	void TestGetRange()
	{
		// GetRange is a pass through to MemmoveBitsWordOrder, which has its own set of tests. We just test a single case for setting range at 0,middle,end
		uint32 ZeroesIntRef = 0;
		uint32 OnesIntRef = MAX_uint32;
		uint32 ZeroesInt = 0;
		uint32 OnesInt = MAX_uint32;
		TBitArray<> ArrZeroes(false, NumBitsPerDWORD);
		TBitArray<> ArrOnes(true, NumBitsPerDWORD);

		ArrOnes.GetRange(0, 10, &ZeroesInt, 0);
		ArrOnes.GetRange(10, 10, &ZeroesInt, 10);
		ArrOnes.GetRange(20, 12, &ZeroesInt, 20);
		ArrZeroes.GetRange(0, 10, &OnesInt, 0);
		ArrZeroes.GetRange(10, 10, &OnesInt, 10);
		ArrZeroes.GetRange(20, 12, &OnesInt, 20);

		CHECK_MESSAGE(TEXT("GetRange Zeroes"), ZeroesInt == OnesIntRef);
		CHECK_MESSAGE(TEXT("GetRange Ones"), OnesInt == ZeroesIntRef);
	}

	void TestEmpty()
	{
		TBitArray<FDefaultAllocator> Arr;
		Arr.Add(true, 10);
		Arr.CheckInvariants();
		Arr.Empty();
		Arr.CheckInvariants();
		CHECK_MESSAGE(TEXT("Empty with no arguments sets num to 0"), Arr.Num() == 0);
		CHECK_MESSAGE(TEXT("Empty with no arguments sets max to 0"), Arr.Max() == 0);

		TBitArray<FDefaultAllocator> Arr2;
		Arr2.Add(true, 10);
		Arr2.CheckInvariants();
		Arr2.Empty(5);
		Arr2.CheckInvariants();
		CHECK_MESSAGE(TEXT("Empty with an arguments sets num to 0"), Arr2.Num() == 0);
		CHECK_MESSAGE(TEXT("Empty with an argument sets max to rounded up input"), Arr2.Max() == NumBitsPerDWORD);

		TBitArray<TInlineAllocator<4>> Arr3;
		int32 InitialMax = Arr3.Max();
		Arr3.Add(true, NumBitsPerDWORD * 8);
		Arr3.CheckInvariants();
		Arr3.Empty(0);
		Arr3.CheckInvariants();
		CHECK_MESSAGE(TEXT("Empty from a dynamic allocation with an inline allocator sets num to 0"), Arr3.Num() == 0);
		CHECK_MESSAGE(TEXT("Empty from a dynamic allocation with an inline allocator sets max back to the size of the inline allocation"),
			Arr3.Max() == InitialMax);
	}

	void TestReserve()
	{
		TBitArray<FDefaultAllocator> Arr;
		Arr.Reserve(NumBitsPerDWORD + NumBitsPerDWORD / 2);
		Arr.CheckInvariants();
		CHECK_MESSAGE(TEXT("Reserve from empty does not change num"), Arr.Num() == 0);
		CHECK_MESSAGE(TEXT("Reserve from empty sets max to rounded up request"), Arr.Max() >= (NumBitsPerDWORD * 2));

		TBitArray<FDefaultAllocator> Arr2Ref(true, NumBitsPerDWORD);
		TBitArray<FDefaultAllocator> Arr2(true, NumBitsPerDWORD);
		Arr2.Reserve(NumBitsPerDWORD + NumBitsPerDWORD / 2);
		Arr2.CheckInvariants();
		CHECK_MESSAGE(TEXT("Reserve from filled does not size or elements"), Arr2 == Arr2Ref);
		CHECK_MESSAGE(TEXT("Reserve from filled sets max to rounded up request"), Arr.Max() >= (NumBitsPerDWORD * 2));
	}

	void TestReset()
	{
		TBitArray<FDefaultAllocator> Arr;
		Arr.Reset();
		Arr.CheckInvariants();
		CHECK_MESSAGE(TEXT("Reset from empty keeps num at 0"), Arr.Num() == 0);
		CHECK_MESSAGE(TEXT("Reserve from empty keeps max at 0"), Arr.Max() == 0);

		TBitArray<FDefaultAllocator> Arr2(true, NumBitsPerDWORD);
		Arr2.Reset();
		Arr2.CheckInvariants();
		CHECK_MESSAGE(TEXT("Reset from filled sets num to 0"), Arr2.Num() == 0);
		CHECK_MESSAGE(TEXT("Reset from filled sets not change max"), Arr2.Max() == NumBitsPerDWORD);
	}

	void TestInitAndSetNumUninitialized()
	{
		// SetNumUninitialized
		{
			TBitArray<FDefaultAllocator> Arr;
			Arr.SetNumUninitialized(0);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("SetNumUninitialized 0 from empty keeps num at 0"), Arr.Num() == 0);
			CHECK_MESSAGE(TEXT("SetNumUninitialized 0 from empty keeps max at 0"), Arr.Max() == 0);

			TBitArray<FDefaultAllocator> Arr2(true, NumBitsPerDWORD);
			Arr2.SetNumUninitialized(0);
			Arr2.CheckInvariants();
			CHECK_MESSAGE(TEXT("SetNumUninitialized 0 from filled sets num to 0"), Arr2.Num() == 0);
			CHECK_MESSAGE(TEXT("SetNumUninitialized 0 from filled does not change max"), Arr2.Max() == NumBitsPerDWORD);

			TBitArray<FDefaultAllocator> Arr3;
			Arr3.SetNumUninitialized(20);
			Arr3.CheckInvariants();
			CHECK_MESSAGE(TEXT("SetNumUninitialized 20 from empty sets num at 20"), Arr3.Num() == 20);
				// Depending on defines, requesting the bitarray's grow to handle a single int might reserve multiple ints. Just confirm the max is a multiple of bitsperdword > num
			CHECK_MESSAGE(TEXT("SetNumUninitialized 20 sets max at rounded up"), (Arr3.Max() >= NumBitsPerDWORD) && ((Arr3.Max() % NumBitsPerDWORD) == 0));

			TBitArray<FDefaultAllocator> Arr4Ones(true, NumBitsPerDWORD);
			Arr4Ones.SetNumUninitialized(NumBitsPerDWORD + NumBitsPerDWORD / 2);
			Arr4Ones.CheckInvariants();
			CHECK_MESSAGE(TEXT("SetNumUninitialized to a higher number sets num to the higher number"),
				Arr4Ones.Num() == (NumBitsPerDWORD + NumBitsPerDWORD / 2));
			// Depending on defines, requesting the bitarray's grow to handle a single int might reserve multiple ints. Just confirm the max is a multiple of bitsperdword > num
			CHECK_MESSAGE(TEXT("SetNumUninitialized to a higher number sets max to rounded up"),
				(Arr4Ones.Max() >= NumBitsPerDWORD * 2) && ((Arr4Ones.Max() % NumBitsPerDWORD) == 0));
			for (int n = 0; n < NumBitsPerDWORD; ++n)
			{
				CHECK_MESSAGE(TEXT("SetNumUninitialized to a higher number keeps the old elements - ones"), Arr4Ones[n] == true);
			}
			TBitArray<FDefaultAllocator> Arr4Zeroes(false, NumBitsPerDWORD);
			Arr4Zeroes.SetNumUninitialized(NumBitsPerDWORD + NumBitsPerDWORD / 2);
			Arr4Zeroes.CheckInvariants();
			for (int n = 0; n < NumBitsPerDWORD; ++n)
			{
				CHECK_MESSAGE(TEXT("SetNumUninitialized to a higher number keeps the old elements - zeroes"), Arr4Zeroes[n] == false);
			}

			TBitArray<FDefaultAllocator> Arr5Ones(true, NumBitsPerDWORD * 2);
			Arr5Ones.SetNumUninitialized(NumBitsPerDWORD / 2);
			Arr5Ones.CheckInvariants();
			CHECK_MESSAGE(TEXT("SetNumUninitialized to a lower number sets num to the lowernumber"), Arr5Ones.Num() == NumBitsPerDWORD / 2);
			CHECK_MESSAGE(TEXT("SetNumUninitialized to a lower number does not change max"), Arr5Ones.Max() == NumBitsPerDWORD * 2);
			for (int n = 0; n < NumBitsPerDWORD / 2; ++n)
			{
				CHECK_MESSAGE(TEXT("SetNumUninitialized to a lower number keeps the old elements below the lower number - ones"),
					Arr5Ones[n] == true);
			}
			TBitArray<FDefaultAllocator> Arr5Zeroes(false, NumBitsPerDWORD * 2);
			Arr5Zeroes.SetNumUninitialized(NumBitsPerDWORD / 2);
			Arr5Zeroes.CheckInvariants();
			for (int n = 0; n < NumBitsPerDWORD / 2; ++n)
			{
				CHECK_MESSAGE(TEXT("SetNumUninitialized to a lower number keeps the old elements below the lower number - zeroes"),
					Arr5Zeroes[n] == false);
			}
		}

		// Init
		{
			TBitArray<FDefaultAllocator> Arr;
			Arr.Init(true, 0);
			Arr.CheckInvariants();
			CHECK_MESSAGE(TEXT("Init 0 from empty keeps num at 0"), Arr.Num() == 0);
			CHECK_MESSAGE(TEXT("Init 0 from empty keeps max at 0"), Arr.Max() == 0);

			TBitArray<FDefaultAllocator> Arr2(true, NumBitsPerDWORD);
			Arr2.Init(true, 0);
			Arr2.CheckInvariants();
			CHECK_MESSAGE(TEXT("Init 0 from filled sets num to 0"), Arr2.Num() == 0);
			CHECK_MESSAGE(TEXT("Init 0 from filled does not change max"), Arr2.Max() == NumBitsPerDWORD);

			TBitArray<FDefaultAllocator> Arr3True;
			TBitArray<FDefaultAllocator> Arr3False;
			TBitArray<FDefaultAllocator> Arr3TrueRef(true, 20);
			TBitArray<FDefaultAllocator> Arr3FalseRef(false, 20);
			Arr3True.Init(true, 20);
			Arr3True.CheckInvariants();
			CHECK_MESSAGE(TEXT("Init true 20 from empty sets size and elements"), Arr3True == Arr3TrueRef);
			CHECK_MESSAGE(TEXT("Init true 20 sets max at rounded up"), Arr3True.Max() == NumBitsPerDWORD);
			Arr3False.Init(false, 20);
			Arr3False.CheckInvariants();
			CHECK_MESSAGE(TEXT("Init false 20 from empty sets size and elements"), Arr3False == Arr3FalseRef);
			CHECK_MESSAGE(TEXT("Init false 20 sets max at rounded up"), Arr3False.Max() == NumBitsPerDWORD);

			TBitArray<FDefaultAllocator> Arr4Ones(true, NumBitsPerDWORD);
			Arr4Ones.Init(false, NumBitsPerDWORD + NumBitsPerDWORD / 2);
			Arr4Ones.CheckInvariants();
			CHECK_MESSAGE(TEXT("Init false to a higher number sets num to the higher number"),
				Arr4Ones.Num() == (NumBitsPerDWORD + NumBitsPerDWORD / 2));
			CHECK_MESSAGE(TEXT("Init false to a higher number sets max to rounded up"),
				Arr4Ones.Max() == NumBitsPerDWORD * 2);
			for (int n = 0; n < Arr4Ones.Num(); ++n)
			{
				CHECK_MESSAGE(TEXT("Init false to a higher number overwrites all elements"), Arr4Ones[n] == false);
			}
			TBitArray<FDefaultAllocator> Arr4Zeroes(false, NumBitsPerDWORD);
			Arr4Zeroes.Init(true, NumBitsPerDWORD + NumBitsPerDWORD / 2);
			Arr4Zeroes.CheckInvariants();
			CHECK_MESSAGE(TEXT("Init true to a higher number sets num to the higher number"),
				Arr4Zeroes.Num() == NumBitsPerDWORD + NumBitsPerDWORD / 2);
			CHECK_MESSAGE(TEXT("Init true to a higher number sets max to rounded up"),
				Arr4Zeroes.Max() == NumBitsPerDWORD * 2);
			for (int n = 0; n < Arr4Zeroes.Num(); ++n)
			{
				CHECK_MESSAGE(TEXT("Init true to a higher number overwrites all elements"), Arr4Zeroes[n] == true);
			}

			TBitArray<FDefaultAllocator> Arr5Ones(true, NumBitsPerDWORD * 2);
			Arr5Ones.Init(false, NumBitsPerDWORD / 2);
			Arr5Ones.CheckInvariants();
			CHECK_MESSAGE(TEXT("Init false to a lower number sets num to the lowernumber"), Arr5Ones.Num() == NumBitsPerDWORD / 2);
			CHECK_MESSAGE(TEXT("Init false to a lower number does not change max"), Arr5Ones.Max() == NumBitsPerDWORD * 2);
			for (int n = 0; n < Arr5Ones.Num(); ++n)
			{
				CHECK_MESSAGE(TEXT("Init false to a lower number overwrites all elements"), Arr5Ones[n] == false);
			}
			TBitArray<FDefaultAllocator> Arr5Zeroes(false, NumBitsPerDWORD * 2);
			Arr5Zeroes.Init(true, NumBitsPerDWORD / 2);
			Arr5Zeroes.CheckInvariants();
			CHECK_MESSAGE(TEXT("Init true to a lower number sets num to the lowernumber"), Arr5Zeroes.Num() == NumBitsPerDWORD / 2);
			CHECK_MESSAGE(TEXT("Init true to a lower number does not change max"), Arr5Zeroes.Max() == NumBitsPerDWORD * 2);
			for (int n = 0; n < Arr5Zeroes.Num(); ++n)
			{
				CHECK_MESSAGE(TEXT("Init true to a lower number overwrites all elements"), Arr5Zeroes[n] == true);
			}
		}
	}

	// TODO: GetAllocatedSize
	// TODO: CountBytes
	// TODO: FindLast
	// TODO: Contains
	// TODO: FindAndSetFirstZeroBit
	// TODO: FindAndSetLastZeroBit
	// TODO: IsValidIndex
	// TODO: AccessCorrespondingBit
	// TODO: Iteration
	// TODO: Reverse Iteration
};

TEST_CASE_NAMED(FBitArrayTestMisc, "System::Core::Containers::BitArray::Misc", "[ApplicationContextMask][EngineFilter]")
{
	FBitArrayTest Instance;

	const bool bConstructed = Instance.ConstructAndTestConstructors();
	REQUIRE(bConstructed);

	Instance.TestEqualityOperator();
	Instance.TestOtherConstructorAndAssignment();
	Instance.TestLessThan();
	Instance.TestRemoveAt();
	Instance.TestRemoveAtSwap();
	Instance.TestSerialize();
	Instance.TestAdd();
	Instance.TestAddFromRange();
	Instance.TestInsert();
	Instance.TestInsertFromRange();
	Instance.TestSetRange();
	Instance.TestGetRange();
	Instance.TestEmpty();
	Instance.TestReserve();
	Instance.TestReset();
	Instance.TestInitAndSetNumUninitialized();
}

TEST_CASE_NAMED(FBitArrayTestFor, "System::Core::Containers::BitArray::For", "[ApplicationContextMask][EngineFilter]")
{
	std::initializer_list<bool> Expected = {true, false, true, true, false, false};

	TBitArray<> Bits;
	for (bool bBit : Expected)
	{
		Bits.Add(bBit);
	}

	const bool* ExpectedIt = Expected.begin();
	for (bool bActual : Bits)
	{
		CHECK(bActual == *ExpectedIt++);
	}
}

TEST_CASE_NAMED(FBitArrayTestReverseFor, "System::Core::Containers::BitArray::ReverseFor", "[ApplicationContextMask][EngineFilter]")
{
	class FBackwards final
	{
		TBitArray<>& Obj;
	public:
		FBackwards(TBitArray<>& Obj) : Obj(Obj) {}
		auto begin() { return Obj.rbegin(); }
		auto end() { return Obj.rend(); }
	};

	std::initializer_list<bool> Expected = { true, false, true, true, false, false };

	TBitArray<> Bits;
	for (bool bBit : Expected)
	{
		Bits.Add(bBit);
	}

	unsigned Index = 0;

	for (bool bActual : FBackwards(Bits))
	{
		const bool bExpected = Expected.begin()[Expected.size() - 1 - Index++];
		CHECK(bActual == bExpected);
	}
}

TEST_CASE_NAMED(FBitArrayInvariantsTest, "System::Core::Containers::BitArray::Invariants", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::BitArrayTest;

	// CheckInvariants will fail an assertion if invariants have been broken

	// TBitArray::TBitArray
	{
		TBitArray<> Empty;
		Empty.CheckInvariants();

		TBitArray<> Partial(true, 3);
		Partial.CheckInvariants();

		TBitArray<> Full(true, 32);
		Full.CheckInvariants();
	}
	// TBitArray::Add
	{
		// Num=3
		TBitArray<> Temp(true, 3);
		// Num=5
		Temp.Add(true, 2);
		Temp.CheckInvariants();
		// Num=8
		Temp.Add(true, 3);
		Temp.CheckInvariants();
		// Num=31
		Temp.Add(true, 23);
		Temp.CheckInvariants();
		// Num=32
		Temp.Add(true, 1);
		Temp.CheckInvariants();
		// Num=65
		Temp.Add(true, 33);
		Temp.CheckInvariants();
	}

	// TBitArray::RemoveAt
	{
		// Num=65
		TBitArray<> Temp(true, 65);
		// Num=64
		Temp.RemoveAt(64);
		Temp.CheckInvariants();
		// Num=32
		Temp.RemoveAt(31, 32);
		Temp.CheckInvariants();
		// Num=16
		Temp.RemoveAt(15, 16);
		Temp.CheckInvariants();
		// Num=0
		Temp.RemoveAt(0, 16);
		Temp.CheckInvariants();
	}

	// TBitArray::RemoveAtSwap
	{
		// Num=65
		TBitArray<> Temp(true, 65);
		// Num=64
		Temp.RemoveAtSwap(64);
		Temp.CheckInvariants();
		// Num=32
		Temp.RemoveAtSwap(31, 32);
		Temp.CheckInvariants();
		// Num=16
		Temp.RemoveAtSwap(15, 16);
		Temp.CheckInvariants();
		// Num=0
		Temp.RemoveAtSwap(0, 16);
		Temp.CheckInvariants();
	}

	// TBitArray::Init
	{
		TBitArray<> Temp(false, 16);
		Temp.Init(true, 5);
		Temp.CheckInvariants();

		Temp = TBitArray<>(true, 37);
		Temp.Init(true, 33);
		Temp.CheckInvariants();

		Temp = TBitArray<>(true, 37);
		Temp.Init(true, 32);
		Temp.CheckInvariants();
	}
}

TEST_CASE_NAMED(FBitArrayCountSetBitsTest, "System::Core::Containers::BitArray::CountSetBits", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::BitArrayTest;

	// Test unconstrained CountSetBits
	{
		struct FTest { const char* Bits; int32 Expected; };
		FTest Tests[] = {
			{ "0", 0 },
			{ "10010", 2 },
			{ "100001", 2 },
			{ "00000000", 0 },
			{ "10000000", 1 },
			{ "00000001", 1 },
			{ "00000000 1", 1 },
			{ "00000000 0", 0 },
			{ "10000001 1", 3 },
			{ "01011101 11101000 10000001 00101100", 14 },
		};

		for (const FTest& Test : Tests)
		{
			const TBitArray<> Array = ConstructBitArray(Test.Bits);

			const int32 SetBits = Array.CountSetBits();
			CHECK_MESSAGE(FString::Printf(TEXT("CountSetBits: Unexpected number of set bits for array %s. Expected: %i, Actual: %i"), *BitArrayToString(Array), Test.Expected, SetBits),
				!(SetBits != Test.Expected));
		}
	}

	// Test constrained CountSetBits
	{
		struct FTest { const char* Bits; int32 StartIndex; int32 EndIndex; int32 Expected; };
		FTest Tests[] = {
			{ "0", 0, 1, 0 },
			{ "0", 0, 1, 0 },
			{ "10000000", 1, 8, 0 },
			{ "00000001", 1, 8, 1 },
			{ "00000000 1", 8, 9, 1 },
			{ "01011101 11101000 10000001 00101100", 24, 32, 3 },
			{ "01011101 11101000 10000001 00101100", 8, 24, 6 },
			{ "01011101 11101000 10000001 00101100", 12, 18, 2 },
			{ "01011101 11101000 10000001 00101100", 4, 30, 12 },
		};

		for (const FTest& Test : Tests)
		{
			const TBitArray<> Array = ConstructBitArray(Test.Bits);

			const int32 SetBits = Array.CountSetBits(Test.StartIndex, Test.EndIndex);
			CHECK_MESSAGE(FString::Printf(TEXT("CountSetBits: Unexpected number of set bits for array %s between index %d and %d. Expected: %i, Actual: %i"), *BitArrayToString(Array), Test.StartIndex, Test.EndIndex, Test.Expected, SetBits),
				!(SetBits != Test.Expected));
		}
	}
}



TEST_CASE_NAMED(FBitArrayBitwiseNOTTest, "System::Core::Containers::BitArray::BitwiseNOT", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::BitArrayTest;

	// Test bitwise NOT (~)
	struct FTest { const char* Input; const char* Expected; };
	FTest Tests[] = {
		{ "0",
		  "1" },
		{ "10010",
		  "01101" },
		{ "100001",
		  "011110" },
		{ "00000000",
		  "11111111" },
		{ "10000000",
		  "01111111" },
		{ "00000001",
		  "11111110" },
		{ "00000000 1",
		  "11111111 0" },
		{ "00000000 0",
		  "11111111 1" },
		{ "10000001 1",
		  "01111110 0" },
		{ "01011101 11101000 10000001 001011",
		  "10100010 00010111 01111110 110100" },
	};

	for (const FTest& Test : Tests)
	{
		const TBitArray<> Input    = ConstructBitArray(Test.Input);
		const TBitArray<> Expected = ConstructBitArray(Test.Expected);

		TBitArray<> Result   = Input;
		Result.BitwiseNOT();
		CHECK_MESSAGE(FString::Printf(TEXT("Bitwise NOT: Unexpected result for source %hs. Expected: %hs, Actual: %s"), Test.Input, Test.Expected, *BitArrayToString(Result)),
			!(Result != Expected));
	}
}



TEST_CASE_NAMED(FBitArrayBitwiseANDTest, "System::Core::Containers::BitArray::BitwiseAND", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::BitArrayTest;

	using BinaryCallable   = TFunctionRef<TBitArray<>(const TBitArray<>&, const TBitArray<>&)>;
	using MutatingCallable = TFunctionRef<void(TBitArray<>&, const TBitArray<>&)>;

	// Test bitwise AND (&) with all 5 combinations of flags:
	// 	EBitwiseOperatorFlags::MinSize
	// 	EBitwiseOperatorFlags::MaxSize (| EBitwiseOperatorFlags::OneFillMissingBits)
	// 	EBitwiseOperatorFlags::MaintainSize (| EBitwiseOperatorFlags::OneFillMissingBits)

	struct FTestInput 
	{
		const char* InputA;
		const char* InputB;
	};
	struct FTestResult
	{
		const char* Expected;
	};

	auto RunBinaryTestImpl = [](const TCHAR* Description, TArrayView<const FTestInput> Tests, TArrayView<const FTestResult> Results, BinaryCallable BinaryOp)
	{
		check(Tests.Num() == Results.Num());
		for (int32 Index = 0; Index < Tests.Num(); ++Index)
		{
			FTestInput Test = Tests[Index];
			FTestResult TestResult = Results[Index];

			const TBitArray<> InputA   = ConstructBitArray(Test.InputA);
			const TBitArray<> InputB   = ConstructBitArray(Test.InputB);
			const TBitArray<> Expected = ConstructBitArray(TestResult.Expected);

			TBitArray<> Result = BinaryOp(InputA, InputB);
			
			CHECK_MESSAGE(FString::Printf(TEXT("%s: Unexpected result for source %hs & %hs. Expected: %hs, Actual: %s"), Description, Test.InputA, Test.InputB, TestResult.Expected, *BitArrayToString(Result)),
				!(Result != Expected));

			Result = BinaryOp(InputB, InputA);
			CHECK_MESSAGE(FString::Printf(TEXT("%s: Unexpected result for source %hs & %hs. Expected: %hs, Actual: %s"), Description, Test.InputB, Test.InputA, TestResult.Expected, *BitArrayToString(Result)),
				!(Result != Expected));
		}
	};

	auto RunMutatingTestImpl = [](const TCHAR* Description, TArrayView<const FTestInput> Tests, TArrayView<const FTestResult> Results, MutatingCallable MutatingOp)
	{
		for (int32 Index = 0; Index < Tests.Num(); ++Index)
		{
			FTestInput Test = Tests[Index];
			FTestResult TestResult = Results[Index];

			const TBitArray<> InputA   = ConstructBitArray(Test.InputA);
			const TBitArray<> InputB   = ConstructBitArray(Test.InputB);
			const TBitArray<> Expected = ConstructBitArray(TestResult.Expected);

			TBitArray<> Result = InputA;
			MutatingOp(Result, InputB);
			CHECK_MESSAGE(FString::Printf(TEXT("%s: Unexpected result for source %hs & %hs. Expected: %hs, Actual: %s"), Description, Test.InputA, Test.InputB, TestResult.Expected, *BitArrayToString(Result)),
				!(Result != Expected));
		}
	};


	FTestInput Tests[] = {
		{ "0",
		  "1" },

		{ "1",
		  "1" },

		{ "0",
		  "0" },

		{ "0001",
		  "11111111" },

		{ "11111111 010",
		  "10000100 011111" },

		{ "11111111 001110 11111",
		  "10000100 001111" },

		{ "11111111 00111011 11111110 00000111 11110000 00000110 00001111 00000111 11111110",
		  "11111100 01111111 11100000 11110000 01100000 00001111 11100000 01111111 11011100 11111111" },
	};

	{
		FTestResult Results[] = {
			{ "0" },               // 0 & 1
			{ "1" },               // 1 & 1
			{ "0" },               // 0 & 0
			{ "0001" },            // 0001 & 11111111
			{ "10000100 010" },    // 11111111 010 & 10000100 011111
			{ "10000100 001110" }, // 11111111 001110 11111 & 10000100 001111
			// 11111111 00111011 11111110 00000111 11110000 00000110 00001111 00000111 11111110 &
			// 11111100 01111111 11100000 11110000 01100000 00001111 11100000 01111111 11011100 11111111
			{ "11111100 00111011 11100000 00000000 01100000 00000110 00000000 00000111 11011100" },
		};

		RunBinaryTestImpl(TEXT("BitwiseAND (MinSize)"), Tests, Results, [](const TBitArray<>& InA, const TBitArray<>& InB){ return TBitArray<>::BitwiseAND(InA, InB, EBitwiseOperatorFlags::MinSize); });
		RunMutatingTestImpl(TEXT("CombineWithBitwiseAND (MinSize)"), Tests, Results, [](TBitArray<>& Mutate, const TBitArray<>& InB){ Mutate.CombineWithBitwiseAND(InB, EBitwiseOperatorFlags::MinSize); });
	}

	{
		FTestResult Results[] = {
			{ "0" },                      // 0 & 1
			{ "1" },                      // 1 & 1
			{ "0" },                      // 0 & 0
			{ "00010000" },               // 0001 & 11111111
			{ "10000100 010000" },        // 11111111 010 & 10000100 011111
			{ "10000100 001110 00000" },  // 11111111 001110 11111 & 10000100 001111
			// 11111111 00111011 11111110 00000111 11110000 00000110 00001111 00000111 11111110 &
			// 11111100 01111111 11100000 11110000 01100000 00001111 11100000 01111111 11011100 11111111
			{ "11111100 00111011 11100000 00000000 01100000 00000110 00000000 00000111 11011100 00000000" },
		};

		RunBinaryTestImpl(TEXT("BitwiseAND (MaxSize)"), Tests, Results, [](const TBitArray<>& InA, const TBitArray<>& InB){ return TBitArray<>::BitwiseAND(InA, InB, EBitwiseOperatorFlags::MaxSize); });
		RunMutatingTestImpl(TEXT("CombineWithBitwiseAND (MaxSize)"), Tests, Results, [](TBitArray<>& Mutate, const TBitArray<>& InB){ Mutate.CombineWithBitwiseAND(InB, EBitwiseOperatorFlags::MaxSize); });
	}

	{
		FTestResult Results[] = {
			{ "0" },                     // 0 & 1
			{ "1" },                     // 1 & 1
			{ "0" },                     // 0 & 0
			{ "00011111" },              // 0001 & 11111111
			{ "10000100 010111" },       // 11111111 010 & 10000100 011111
			{ "10000100 001110 11111" }, // 11111111 001110 11111 & 10000100 001111
			// 11111111 00111011 11111110 00000111 11110000 00000110 00001111 00000111 11111110 &
			// 11111100 01111111 11100000 11110000 01100000 00001111 11100000 01111111 11011100 11111111
			{ "11111100 00111011 11100000 00000000 01100000 00000110 00000000 00000111 11011100 11111111" },
		};

		RunBinaryTestImpl(TEXT("BitwiseAND (MaxSize | OneFillMissingBits)"), Tests, Results, [](const TBitArray<>& InA, const TBitArray<>& InB){ return TBitArray<>::BitwiseAND(InA, InB, EBitwiseOperatorFlags::MaxSize | EBitwiseOperatorFlags::OneFillMissingBits); });
		RunMutatingTestImpl(TEXT("CombineWithBitwiseAND (MaxSize | OneFillMissingBits)"), Tests, Results, [](TBitArray<>& Mutate, const TBitArray<>& InB){ Mutate.CombineWithBitwiseAND(InB, EBitwiseOperatorFlags::MaxSize | EBitwiseOperatorFlags::OneFillMissingBits); });
	}

	{
		FTestResult Results[] = {
			{ "0" },                     // 0 & 1
			{ "1" },                     // 1 & 1
			{ "0" },                     // 0 & 0
			{ "0001" },                  // 0001 & 11111111
			{ "10000100 010" },          // 11111111 010 & 10000100 011111
			{ "10000100 001110 00000" }, // 11111111 001110 11111 & 10000100 001111
			// 11111111 00111011 11111110 00000111 11110000 00000110 00001111 00000111 11111110 &
			// 11111100 01111111 11100000 11110000 01100000 00001111 11100000 01111111 11011100 11111111
			{ "11111100 00111011 11100000 00000000 01100000 00000110 00000000 00000111 11011100" },
		};

		RunMutatingTestImpl(TEXT("CombineWithBitwiseAND (MaintainSize)"), Tests, Results, [](TBitArray<>& Mutate, const TBitArray<>& InB){ Mutate.CombineWithBitwiseAND(InB, EBitwiseOperatorFlags::MaintainSize); });
	}

	{
		FTestResult Results[] = {
			{ "0" },                     // 0 & 1
			{ "1" },                     // 1 & 1
			{ "0" },                     // 0 & 0
			{ "0001" },                  // 0001 & 11111111
			{ "10000100 010" },          // 11111111 010 & 10000100 011111
			{ "10000100 001110 11111" }, // 11111111 001110 11111 & 10000100 001111
			// 11111111 00111011 11111110 00000111 11110000 00000110 00001111 00000111 11111110 &
			// 11111100 01111111 11100000 11110000 01100000 00001111 11100000 01111111 11011100 11111111
			{ "11111100 00111011 11100000 00000000 01100000 00000110 00000000 00000111 11011100" },
		};

		RunMutatingTestImpl(TEXT("CombineWithBitwiseAND (MaintainSize | OneFillMissingBits)"), Tests, Results, [](TBitArray<>& Mutate, const TBitArray<>& InB){ Mutate.CombineWithBitwiseAND(InB, EBitwiseOperatorFlags::MaintainSize | EBitwiseOperatorFlags::OneFillMissingBits); });
	}
}



TEST_CASE_NAMED(FBitArrayBitwiseORTest, "System::Core::Containers::BitArray::BitwiseOR", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::BitArrayTest;

	// Test bitwise OR (|)
	struct FTest { const char *InputA, *InputB, *Expected; };
	FTest Tests[] = {
		{ "0",
		  "1",
		  "1" },
		{ "1",
		  "1",
		  "1" },
		{ "0",
		  "0",
		  "0" },
		{ "00011100",
		  "11111111",
		  "11111111" },
		{ "11111111 001110",
		  "10000100 001111",
		  "11111111 001111" },
		{ "11111111 00111011 111",
		  "10000100 001111",
		  "11111111 001111 11111" },
	};

	for (const FTest& Test : Tests)
	{
		const TBitArray<> InputA   = ConstructBitArray(Test.InputA);
		const TBitArray<> InputB   = ConstructBitArray(Test.InputB);
		const TBitArray<> Expected = ConstructBitArray(Test.Expected);

		TBitArray<> Result = TBitArray<>::BitwiseOR(InputA, InputB, EBitwiseOperatorFlags::MaxSize);
		CHECK_MESSAGE(FString::Printf(TEXT("BitwiseOR: Unexpected result for source %hs | %hs. Expected: %hs, Actual: %s"), Test.InputA, Test.InputB, Test.Expected, *BitArrayToString(Result)),
			!(Result != Expected));

		Result = TBitArray<>::BitwiseOR(InputB, InputA, EBitwiseOperatorFlags::MaxSize);
		CHECK_MESSAGE(FString::Printf(TEXT("BitwiseOR: Unexpected result for source %hs | %hs. Expected: %hs, Actual: %s"), Test.InputB, Test.InputA, Test.Expected, *BitArrayToString(Result)),
			!(Result != Expected));

		Result = InputA;
		Result.CombineWithBitwiseOR(InputB, EBitwiseOperatorFlags::MaxSize);
		CHECK_MESSAGE(FString::Printf(TEXT("CombineWithBitwiseOR: Unexpected result for source %hs | %hs. Expected: %hs, Actual: %s"), Test.InputA, Test.InputB, Test.Expected, *BitArrayToString(Result)),
			!(Result != Expected));
	}
}



TEST_CASE_NAMED(FBitArrayBitwiseXORTest, "System::Core::Containers::BitArray::BitwiseXOR", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::BitArrayTest;

	// Test bitwise XOR (|)
	struct FTest { const char *InputA, *InputB, *Expected; };
	FTest Tests[] = {
		{ "0",
		  "1",
		  "1" },
		{ "1",
		  "0",
		  "1" },
		{ "1",
		  "1",
		  "0" },
		{ "0",
		  "0",
		  "0" },
		{ "00011100",
		  "11111111",
		  "11100011" },
		{ "11111111 001110",
		  "10000100 001111",
		  "01111011 000001" },
		{ "11111111 00111011 111",
		  "10000100 001111",
		  "01111011 000001 11111" },
	};

	for (const FTest& Test : Tests)
	{
		const TBitArray<> InputA   = ConstructBitArray(Test.InputA);
		const TBitArray<> InputB   = ConstructBitArray(Test.InputB);
		const TBitArray<> Expected = ConstructBitArray(Test.Expected);

		TBitArray<> Result = TBitArray<>::BitwiseXOR(InputA, InputB, EBitwiseOperatorFlags::MaxSize);
		CHECK_MESSAGE(FString::Printf(TEXT("BitwiseXOR: Unexpected result for source %hs | %hs. Expected: %hs, Actual: %s"), Test.InputA, Test.InputB, Test.Expected, *BitArrayToString(Result)),
			!(Result != Expected));

		Result = TBitArray<>::BitwiseXOR(InputB, InputA, EBitwiseOperatorFlags::MaxSize);
		CHECK_MESSAGE(FString::Printf(TEXT("BitwiseXOR: Unexpected result for source %hs | %hs. Expected: %hs, Actual: %s"), Test.InputB, Test.InputA, Test.Expected, *BitArrayToString(Result)),
			!(Result != Expected));

		Result = InputA;
		Result.CombineWithBitwiseXOR(InputB, EBitwiseOperatorFlags::MaxSize);
		CHECK_MESSAGE(FString::Printf(TEXT("CombineWithBitwiseXOR: Unexpected result for source %hs | %hs. Expected: %hs, Actual: %s"), Test.InputA, Test.InputB, Test.Expected, *BitArrayToString(Result)),
			!(Result != Expected));
	}
}

TEST_CASE_NAMED(FBitArrayFindTest, "System::Core::Containers::BitArray::Find", "[ApplicationContextMask][SmokeFilter]")
{
	{
		TBitArray<> ArrEmpty;
		CHECK_MESSAGE(TEXT("Find true in an empty array"), ArrEmpty.Find(true) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find false in an empty array"), ArrEmpty.Find(false) == INDEX_NONE);
	}

	{
		TBitArray<> ArrTrue(true, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find true in an array containing 32 true"), ArrTrue.Find(true) == 0);
		CHECK_MESSAGE(TEXT("Find false in an array containing 32 true"), ArrTrue.Find(false) == INDEX_NONE);
	}

	{
		TBitArray<> ArrFalse(false, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find false in an array containing 32 false"), ArrFalse.Find(false) == 0);
		CHECK_MESSAGE(TEXT("Find true in an array containing 32 false"), ArrFalse.Find(true) == INDEX_NONE);
	}

	{
		TBitArray<> ArrFalseThenTrue(false, NumBitsPerDWORD);
		ArrFalseThenTrue.Add(true);
		CHECK_MESSAGE(TEXT("Find true in an array containing multiple false and then a single true"), ArrFalseThenTrue.Find(true) == NumBitsPerDWORD);
	}

	{
		TBitArray<> ArrTrueThenFalse(true, NumBitsPerDWORD);
		ArrTrueThenFalse.Add(false);
		CHECK_MESSAGE(TEXT("Find true in an array containing multiple true and then a single false"), ArrTrueThenFalse.Find(false) == NumBitsPerDWORD);
	}
}

TEST_CASE_NAMED(FBitArrayFindFromTest, "System::Core::Containers::BitArray::FindFrom", "[ApplicationContextMask][SmokeFilter]")
{
	{
		TBitArray<> ArrEmpty;
		REQUIRE_CHECK_SLOW(ArrEmpty.FindFrom(true, -1)); // prior to the [0, NumBits] range, we should assert
		CHECK_MESSAGE(TEXT("FindFrom true in an empty array"), ArrEmpty.FindFrom(true, 0) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("FindFrom false in an empty array"), ArrEmpty.FindFrom(false, 0) == INDEX_NONE);
		REQUIRE_CHECK_SLOW(ArrEmpty.FindFrom(true, 1)); // past the [0, NumBits] range, we should assert
	}

	{
		TBitArray<> Arr(true, 1);
		REQUIRE_CHECK_SLOW(Arr.FindFrom(true, -1)); // prior to the [-1, NumBits) range, we should assert
		REQUIRE_CHECK_SLOW(Arr.FindFrom(true, Arr.Num() + 1)); // prior to the [-1, NumBits) range, we should assert
	}

	{
		TBitArray<> ArrTrue(true, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find true in an array full of true, starting at index 0"), ArrTrue.FindFrom(true, 0) == 0);
		CHECK_MESSAGE(TEXT("Find true in an array full of true, starting at index 1"), ArrTrue.FindFrom(true, 1) == 1);
		CHECK_MESSAGE(TEXT("Find true in an array full of true, starting at index 31"), ArrTrue.FindFrom(true, NumBitsPerDWORD - 1) == NumBitsPerDWORD - 1);
		CHECK_MESSAGE(TEXT("Find true in an array full of true, starting at index 32"), ArrTrue.FindFrom(true, NumBitsPerDWORD) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find false in an array full of true, starting at index 1"), ArrTrue.FindFrom(false, 1) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find false in an array full of true, starting at index 32"), ArrTrue.FindFrom(false, NumBitsPerDWORD) == INDEX_NONE);
	}

	{
		TBitArray<> ArrFalse(false, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find false in an array full of false, starting at index 0"), ArrFalse.FindFrom(false, 0) == 0);
		CHECK_MESSAGE(TEXT("Find false in an array full of false, starting at index 1"), ArrFalse.FindFrom(false, 1) == 1);
		CHECK_MESSAGE(TEXT("Find false in an array full of false, starting at index 31"), ArrFalse.FindFrom(false, NumBitsPerDWORD - 1) == NumBitsPerDWORD - 1);
		CHECK_MESSAGE(TEXT("Find false in an array full of false, starting at index 32"), ArrFalse.FindFrom(false, NumBitsPerDWORD) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find true in an array full of false, starting at index 1"), ArrFalse.FindFrom(true, 1) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find false in an array full of false, starting at index 32"), ArrFalse.FindFrom(true, NumBitsPerDWORD) == INDEX_NONE);
	}

	{
		TBitArray<> ArrFalseThenTrue(false, NumBitsPerDWORD);
		ArrFalseThenTrue.Add(true, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find true in an array containing 32 false and then 32 true, starting after all of the false"), ArrFalseThenTrue.FindFrom(true, NumBitsPerDWORD) == NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find false in an array containing 32 false and then 32 true, starting after all of the false"), ArrFalseThenTrue.FindFrom(false, NumBitsPerDWORD) == INDEX_NONE);
	}

	{
		TBitArray<> ArrTrueThenFalse(true, NumBitsPerDWORD);
		ArrTrueThenFalse.Add(false, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find true in an array containing 32 true and then 32 false, starting after all of the true"), ArrTrueThenFalse.FindFrom(true, NumBitsPerDWORD) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find false in an array containing 32 true and then 32 false, starting after all of the true"), ArrTrueThenFalse.FindFrom(false, NumBitsPerDWORD) == NumBitsPerDWORD);
	}
}

TEST_CASE_NAMED(FBitArrayTestFindLastTest, "System::Core::Containers::BitArray::FindLast", "[ApplicationContextMask][SmokeFilter]")
{
	{
		TBitArray<> ArrEmpty;
		CHECK_MESSAGE(TEXT("Find last true in an empty array"), ArrEmpty.FindLast(true) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find last false in an empty array"), ArrEmpty.FindLast(false) == INDEX_NONE);
	}

	{
		TBitArray<> ArrTrue(true, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 true"), ArrTrue.FindLast(true) == NumBitsPerDWORD - 1);
		CHECK_MESSAGE(TEXT("Find last false in an array containing 32 true"), ArrTrue.FindLast(false) == INDEX_NONE);
	}

	{
		TBitArray<> ArrFalse(false, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find last false in an array containing 32 false"), ArrFalse.FindLast(false) == NumBitsPerDWORD - 1);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 false"), ArrFalse.FindLast(true) == INDEX_NONE);
	}

	{
		TBitArray<> ArrFalseThenTrue(false, NumBitsPerDWORD);
		ArrFalseThenTrue.Add(true);
		CHECK_MESSAGE(TEXT("Find last true in an array containing multiple false and then a single true"), ArrFalseThenTrue.FindLast(true) == NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find last false in an array containing multiple false and then a single true"), ArrFalseThenTrue.FindLast(false) == NumBitsPerDWORD - 1);
	}

	{
		TBitArray<> ArrTrueThenFalse(true, NumBitsPerDWORD);
		ArrTrueThenFalse.Add(false);
		CHECK_MESSAGE(TEXT("Find last false in an array containing multiple true and then a single false"), ArrTrueThenFalse.FindLast(false) == NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find last true in an array containing multiple true and then a single false"), ArrTrueThenFalse.FindLast(true) == NumBitsPerDWORD - 1);
	}
}

TEST_CASE_NAMED(FBitArrayTestFindLastTestFrom, "System::Core::Containers::BitArray::FindLastFrom", "[ApplicationContextMask][SmokeFilter]")
{
	{
		TBitArray<> ArrEmpty;
		REQUIRE_CHECK_SLOW(ArrEmpty.FindLastFrom(true, -2)); // prior to the [-1, NumBits) range, we should assert
		CHECK_MESSAGE(TEXT("FindLastFrom true in an empty array"), ArrEmpty.FindLastFrom(true, -1) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("FindLastFrom false in an empty array"), ArrEmpty.FindLastFrom(false, -1) == INDEX_NONE);
		REQUIRE_CHECK_SLOW(ArrEmpty.FindLastFrom(true, 0)); // past the [-1, NumBits) range, we should assert
	}

	{
		TBitArray<> Arr(true, 1);
		REQUIRE_CHECK_SLOW(Arr.FindLastFrom(true, -2)); // prior to the [-1, NumBits) range, we should assert
		REQUIRE_CHECK_SLOW(Arr.FindLastFrom(true, Arr.Num() + 1)); // prior to the [-1, NumBits) range, we should assert
	}

	{
		TBitArray<> ArrTrue(true, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 true, starting from 0"), ArrTrue.FindLastFrom(true, 0) == 0);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 true, starting from the middle"), ArrTrue.FindLastFrom(true, NumBitsPerDWORD / 2) == NumBitsPerDWORD / 2);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 true, starting from the end"), ArrTrue.FindLastFrom(true, NumBitsPerDWORD - 1) == NumBitsPerDWORD - 1);
		CHECK_MESSAGE(TEXT("Find last false in an array containing 32 true, starting from 0"), ArrTrue.FindLastFrom(false, 0) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find last false in an array containing 32 true, starting from the middle"), ArrTrue.FindLastFrom(false, NumBitsPerDWORD / 2) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find last false in an array containing 32 true, starting from the end"), ArrTrue.FindLastFrom(false, NumBitsPerDWORD - 1) == INDEX_NONE);
	}

	{
		TBitArray<> ArrFalse(false, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find last false in an array containing 32 false, starting from 0"), ArrFalse.FindLastFrom(false, 0) == 0);
		CHECK_MESSAGE(TEXT("Find last false in an array containing 32 false, starting from the middle"), ArrFalse.FindLastFrom(false, NumBitsPerDWORD / 2) == NumBitsPerDWORD / 2);
		CHECK_MESSAGE(TEXT("Find last false in an array containing 32 false, starting from the end"), ArrFalse.FindLastFrom(false, NumBitsPerDWORD - 1) == NumBitsPerDWORD - 1);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 false, starting from 0"), ArrFalse.FindLastFrom(true, 0) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 false, starting from the middle"), ArrFalse.FindLastFrom(true, NumBitsPerDWORD / 2) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 false, starting from the end"), ArrFalse.FindLastFrom(true, NumBitsPerDWORD - 1) == INDEX_NONE);
	}

	{
		TBitArray<> ArrEndTrue;
		ArrEndTrue.Add(false);
		ArrEndTrue.Add(false);
		ArrEndTrue.Add(true);
		ArrEndTrue.Add(true);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 false, starting from the end"), ArrEndTrue.FindLastFrom(true, 0) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 false, starting from the end"), ArrEndTrue.FindLastFrom(true, 1) == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 false, starting from the end"), ArrEndTrue.FindLastFrom(true, 2) == 2);
		CHECK_MESSAGE(TEXT("Find last true in an array containing 32 false, starting from the end"), ArrEndTrue.FindLastFrom(true, 3) == 3);
	}

	{
		uint32 InitialDWords[3] =
			{ 0b00010000000000000000000011110100
			, 0b10000001000000000000000000000001 
			, 0b00000000000000000000000000000010 };
		TBitArray<> Arr(false, std::size(InitialDWords) * NumBitsPerDWORD);
		Arr.SetRangeFromRange(0, Arr.Num(), InitialDWords);

		auto TestArr = [&Arr](bool bInTestValue, const TCHAR* InTestValueStr)
		{
			auto TestFindLastFrom = [&Arr](bool bInTestValue, const TCHAR* InTestValueStr, int32 InStartIndex, int32 InExpectedValue)
			{
				CHECK_MESSAGE(FString::Printf(TEXT("Find last %s in an array starting from %i"), InTestValueStr, InStartIndex), Arr.FindLastFrom(bInTestValue, InStartIndex) == InExpectedValue);
			};

			// First DWord
			TestFindLastFrom(bInTestValue, InTestValueStr, 31, 28);
			TestFindLastFrom(bInTestValue, InTestValueStr, 28, 28);
			TestFindLastFrom(bInTestValue, InTestValueStr, 27, 7);
			TestFindLastFrom(bInTestValue, InTestValueStr, 7, 7);
			TestFindLastFrom(bInTestValue, InTestValueStr, 4, 4);
			TestFindLastFrom(bInTestValue, InTestValueStr, 3, 2);
			TestFindLastFrom(bInTestValue, InTestValueStr, 1, INDEX_NONE);
			TestFindLastFrom(bInTestValue, InTestValueStr, 0, INDEX_NONE);
			// Second DWord
			TestFindLastFrom(bInTestValue, InTestValueStr, 32, 32);
			TestFindLastFrom(bInTestValue, InTestValueStr, 33, 32);
			TestFindLastFrom(bInTestValue, InTestValueStr, 55, 32);
			TestFindLastFrom(bInTestValue, InTestValueStr, 56, 56);
			TestFindLastFrom(bInTestValue, InTestValueStr, 57, 56);
			TestFindLastFrom(bInTestValue, InTestValueStr, 62, 56);
			TestFindLastFrom(bInTestValue, InTestValueStr, 63, 63);
			// Third DWord
			TestFindLastFrom(bInTestValue, InTestValueStr, 64, 63);
			TestFindLastFrom(bInTestValue, InTestValueStr, 65, 65);
			TestFindLastFrom(bInTestValue, InTestValueStr, 95, 65);
		};

		TestArr(true, TEXT("true"));
		Arr.BitwiseNOT();
		TestArr(false, TEXT("false"));
	}
}

TEST_CASE_NAMED(FBitArrayTestFindAndSetFirstZeroBitTest, "System::Core::Containers::BitArray::FindAndSetFirstZeroBit", "[ApplicationContextMask][SmokeFilter]")
{
	{
		TBitArray<> ArrEmpty;
		CHECK_MESSAGE(TEXT("Find and set first zero bit in an empty array"), ArrEmpty.FindAndSetFirstZeroBit() == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Array should remain empty and have no set bits"), ArrEmpty.IsEmpty() && ArrEmpty.CountSetBits() == 0);
	}

	{
		TBitArray<> ArrTrue(true, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find and set first zero bit in an array containing 32 true"), ArrTrue.FindAndSetFirstZeroBit() == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Array should still contain 32 true bits"), ArrTrue.CountSetBits() == NumBitsPerDWORD);
	}

	{
		TBitArray<> ArrFalse(false, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find and set first zero bit in an array containing 32 false"), ArrFalse.FindAndSetFirstZeroBit() == 0);
		CHECK_MESSAGE(TEXT("Array should contain 1 true bit"), ArrFalse.CountSetBits() == 1);
	}
}

TEST_CASE_NAMED(FBitArrayTestFindAndSetLastZeroBitTest, "System::Core::Containers::BitArray::FindAndSetLastZeroBit", "[ApplicationContextMask][SmokeFilter]")
{
	{
		TBitArray<> ArrEmpty;
		CHECK_MESSAGE(TEXT("Find and set last zero bit in an empty array"), ArrEmpty.FindAndSetLastZeroBit() == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Find and set last zero bit in an empty array"), ArrEmpty == TBitArray<>());
	}

	{
		TBitArray<> ArrTrue(true, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find and set last zero bit in an array containing 32 true"), ArrTrue.FindAndSetLastZeroBit() == INDEX_NONE);
		CHECK_MESSAGE(TEXT("Array should still contain 32 true bits"), ArrTrue.CountSetBits() == NumBitsPerDWORD);
	}

	{
		TBitArray<> ArrFalse(false, NumBitsPerDWORD);
		CHECK_MESSAGE(TEXT("Find and set last zero bit in an array containing 32 false"), ArrFalse.FindAndSetLastZeroBit() == NumBitsPerDWORD - 1);
		CHECK_MESSAGE(TEXT("Array should contain 1 true bit"), ArrFalse.CountSetBits() == 1);
	}
}

TEST_CASE_NAMED(FScriptBitArrayTestClearPartialSlackBits, "System::Core::Containers::BitArray::ScriptClearPartialSlackBits", "[ApplicationContextMask][SmokeFilter]")
{
	TTypeCompatibleBytes<TBitArray<>> Memory;
	// Inline allocation flags are not initialized and the test requires they are non-zero.
	FMemory::Memset(Memory, 0xff);
	TBitArray<>& Array = *new(&Memory) TBitArray<>();
	((FScriptBitArray&)Array).Add(true);
	Array.CheckInvariants();
}

class FBitArrayMemoryTest
{
public:
	FBitArrayMemoryTest() = default;

	void TestMemmoveBitsWordOrder()
	{
		// This function also fully tests MemmoveBitsWordOrderAlignedInternal; all of its calls occur when ReadOffset == WriteOffset
		constexpr uint32 NumPatterns = 1;
		constexpr uint32 PatternLength = 10;
		int TemplatePatternIndex = 0;

		uint32 ThreeBitRingPattern[PatternLength];
		{
			// 27 bit period, 3 bit integers: 101,001,000,111,010,110,100,011,010, repeat
			// 3 bit sub periods makes it not line up with shifts of 2,4,8,16
			// 27 bit total period makes it not line up across 32-bit words.
			constexpr uint32 Base = 0x5 | (0x1 << 3) | (0x0 << 6) | (0x7 << 9) | (0x2 << 12) | (0x6 << 15) | (0x4 << 18) | (0x3 << 21) | (0x2 << 24);
			constexpr uint32 Period = 27;
			for (int WordIndex = 0; WordIndex < PatternLength; ++WordIndex)
			{
				ThreeBitRingPattern[WordIndex] = 0;
			}

			int WriteIndex, ReadIndex;
			for (WriteIndex = 0; WriteIndex < PatternLength * NumBitsPerDWORD;)
			{
				int ReadEnd = FMath::Min(Period, PatternLength * NumBitsPerDWORD - WriteIndex);
				for (ReadIndex = 0; ReadIndex < ReadEnd; ++WriteIndex, ++ReadIndex)
				{
					ThreeBitRingPattern[WriteIndex / NumBitsPerDWORD] |= ((Base & (1 << ReadIndex)) != 0) << (WriteIndex % NumBitsPerDWORD);
				}
			}
		}

		constexpr int MaxDataLength = 20;
		constexpr int MaxResultLength = 3 * MaxDataLength;
		uint32 Zeroes[MaxResultLength];
		uint32 Ones[MaxResultLength];
		uint32 SourceBits[MaxDataLength];
		uint32 OnesExpected[MaxDataLength];
		uint32 ZeroesExpected[MaxDataLength];
		TCHAR OnesExpectedText[MaxDataLength * NumBitsPerDWORD];
		TCHAR ZeroesExpectedText[MaxDataLength * NumBitsPerDWORD];
		TCHAR ZeroesText[MaxDataLength * NumBitsPerDWORD];
		TCHAR OnesText[MaxDataLength * NumBitsPerDWORD];
		constexpr int UnderflowTestBits = NumBitsPerDWORD;
		constexpr int OverflowTestBits = NumBitsPerDWORD;
		constexpr int UnderflowTestWords = (UnderflowTestBits + NumBitsPerDWORD - 1) / NumBitsPerDWORD;
		constexpr int UnderflowTestStart = UnderflowTestWords * NumBitsPerDWORD - UnderflowTestBits;

		uint32* Pattern = ThreeBitRingPattern;
		int32 BitWidths[] = { 1,2,3,4,15,16,17,31,32,33,63,64,5 * 32 - 1,5 * 32,5 * 32 + 1,5 * 32 + 3,5 * 32 + 4,5 * 32 + 5,5 * 32 + 15,5 * 32 + 16,5 * 32 + 17 };
		for (int32 BitWidth : BitWidths)
		{
			// Testing ReadOffset >= NuMBitsPerDWORD is unnecessary, as it internally gets converted to < NumBitsPerDWORD
			int32 ReadOffsets[] = { 0,1,2,3,4,7,8,9,15,16,17,23,24,25,31 };
			for (int32 ReadOffset : ReadOffsets)
			{
				// SourceBits is the Pattern upshifted by ReadOffset and with bits beyond bitwidth clamped to 0
				{
					for (int WordIndex = 0; WordIndex < MaxDataLength; ++WordIndex)
					{
						SourceBits[WordIndex] = 0;
					}
					int WriteIndex, ReadIndex;
					int WriteEnd = ReadOffset + BitWidth;
					for (WriteIndex = ReadOffset, ReadIndex = 0; WriteIndex < WriteEnd; ++WriteIndex, ++ReadIndex)
					{
						int32 ReadValue = (Pattern[ReadIndex / NumBitsPerDWORD] & (1 << (ReadIndex % NumBitsPerDWORD))) != 0;
						SourceBits[WriteIndex / NumBitsPerDWORD] |= ReadValue << (WriteIndex % NumBitsPerDWORD);
					}
				}

				// Testing WriteOffset >= NuMBitsPerDWORD is unnecessary, as it internally gets converted to < NumBitsPerDWORD
				int32 WriteOffsets[] = { 0,1,2,3,4,7,8,9,15,16,17,23,24,25,31 };
				for (int32 WriteOffset : WriteOffsets)
				{
					int32 OverlapOffsets[] = { MaxDataLength * NumBitsPerDWORD,0,1,31,32,33,4 * 32 - 1,4 * 32,4 * 32 + 1,-1,-31,-32,-33,-4 * 32 + 1,-4 * 32,-4 * 32 - 1 };
					for (int32 Overlap : OverlapOffsets)
					{
						for (int WordIndex = 0; WordIndex < MaxResultLength; ++WordIndex)
						{
							Ones[WordIndex] = MAX_uint32;
							Zeroes[WordIndex] = 0;
						}

						// Copy the SourceBits into the dest arrays at the offset specified by the current overlap; we will be reading and writing to the same array
						uint32* OnesResult = Ones + MaxDataLength;
						uint32* ZeroesResult = Zeroes + MaxDataLength;
						int ReadInDestWordOffsetFromDest;
						if (Overlap >= 0)
						{
							ReadInDestWordOffsetFromDest = Overlap / NumBitsPerDWORD;
						}
						else
						{
							ReadInDestWordOffsetFromDest = (Overlap - NumBitsPerDWORD - 1) / NumBitsPerDWORD;
						}
						int ReadInDestStartBit = Overlap - ReadInDestWordOffsetFromDest * NumBitsPerDWORD;
						uint32* OnesReadInDest = OnesResult + ReadInDestWordOffsetFromDest;
						uint32* ZeroesReadInDest = ZeroesResult + ReadInDestWordOffsetFromDest;

						for (int32 ReadIndex = 0; ReadIndex < BitWidth + ReadOffset; ++ReadIndex)
						{
							int ReadIndexFromStart = ReadInDestStartBit + ReadIndex;
							int ReadWordFromStart = ReadIndexFromStart / NumBitsPerDWORD;
							int ReadBitFromStart = ReadIndexFromStart % NumBitsPerDWORD;
							uint32 SourceBit = (SourceBits[ReadIndex / NumBitsPerDWORD] & (1 << (ReadIndex % NumBitsPerDWORD))) != 0;
							OnesReadInDest[ReadWordFromStart] = (OnesReadInDest[ReadWordFromStart] & ~(1 << ReadBitFromStart)) | (SourceBit << ReadBitFromStart);
							ZeroesReadInDest[ReadWordFromStart] = (ZeroesReadInDest[ReadWordFromStart] & ~(1 << ReadBitFromStart)) | (SourceBit << ReadBitFromStart);
						}

						// Calculate the expected results: create an array for each dest array
						// First copy the verify area of the dest to the expected results array. This will be the background pattern (all zeroes or all ones), with the overlapped read bits overlaid on top of it
						int VerifyBitEnd = UnderflowTestStart + UnderflowTestBits + WriteOffset + BitWidth + OverflowTestBits;
						int32 VerifyWordLength = (VerifyBitEnd + NumBitsPerDWORD - 1) / NumBitsPerDWORD;
						uint32* OnesVerify = OnesResult - UnderflowTestWords;
						uint32* ZeroesVerify = ZeroesResult - UnderflowTestWords;
						FMemory::Memcpy(OnesExpected, OnesVerify, VerifyWordLength * sizeof(uint32));
						FMemory::Memcpy(ZeroesExpected, ZeroesVerify, VerifyWordLength * sizeof(uint32));

						// Then copy in the expected bits at the write offset position; the expected bits is the pattern that we wrote into the read offset
						int32 ResultIndex;
						for (ResultIndex = 0; ResultIndex < BitWidth; ++ResultIndex)
						{
							int ResultTotalBitOffset = UnderflowTestWords * NumBitsPerDWORD + (ResultIndex + WriteOffset);
							int WriteWord = ResultTotalBitOffset / NumBitsPerDWORD;
							int WriteBitOffset = ResultTotalBitOffset % NumBitsPerDWORD;
							uint32 ExpectedValue = (Pattern[ResultIndex / NumBitsPerDWORD] & (1 << (ResultIndex % NumBitsPerDWORD))) != 0;
							OnesExpected[WriteWord] = (OnesExpected[WriteWord] & ~(1 << WriteBitOffset)) | (ExpectedValue << WriteBitOffset);
							ZeroesExpected[WriteWord] = (ZeroesExpected[WriteWord] & ~(1 << WriteBitOffset)) | (ExpectedValue << WriteBitOffset);
						}

						////////////////////
						////////////////////
						// The actual function calls we're testing.
						// It's easy to miss in all this setup code
						FBitArrayMemory::MemmoveBitsWordOrder(OnesResult, WriteOffset, OnesReadInDest, ReadInDestStartBit + ReadOffset, BitWidth);
						FBitArrayMemory::MemmoveBitsWordOrder(ZeroesResult, WriteOffset, ZeroesReadInDest, ReadInDestStartBit + ReadOffset, BitWidth);
						////////////////////
						////////////////////

						bool bOnesMatchesExpected = true;
						bool bZeroesMatchesExpected = true;

						int DisplayIndex = 0;
						for (ResultIndex = UnderflowTestStart; ResultIndex < VerifyBitEnd; ++ResultIndex, ++DisplayIndex)
						{
							int ResultWord = ResultIndex / NumBitsPerDWORD;
							int ResultMask = 1 << (ResultIndex % NumBitsPerDWORD);
							bool OnesExpectedValue = (OnesExpected[ResultWord] & ResultMask) != 0;
							bool OnesValue = (OnesVerify[ResultWord] & ResultMask) != 0;
							bool ZeroesExpectedValue = (ZeroesExpected[ResultWord] & ResultMask) != 0;
							bool ZeroesValue = (ZeroesVerify[ResultWord] & ResultMask) != 0;
							bOnesMatchesExpected = bOnesMatchesExpected & (OnesValue == OnesExpectedValue);
							bZeroesMatchesExpected = bZeroesMatchesExpected & (ZeroesValue == ZeroesExpectedValue);
							OnesExpectedText[DisplayIndex] = OnesExpectedValue ? TEXT('1') : TEXT('0');
							OnesText[DisplayIndex] = OnesValue ? TEXT('1') : TEXT('0');
							ZeroesExpectedText[DisplayIndex] = ZeroesExpectedValue ? TEXT('1') : TEXT('0');
							ZeroesText[DisplayIndex] = ZeroesValue ? TEXT('1') : TEXT('0');
						}
						OnesExpectedText[DisplayIndex] = TEXT('\0');
						OnesText[DisplayIndex] = TEXT('\0');
						ZeroesExpectedText[DisplayIndex] = TEXT('\0');
						ZeroesText[DisplayIndex] = TEXT('\0');
						if (!bOnesMatchesExpected)
						{
							CHECK_MESSAGE(FString::Printf(TEXT("MemmoveBitsWordOrder BitWidth=%d ReadOffset=%d, WriteOffset=%d, Overlap=%d, Ones\nExpected=%s\nActual  =%s"), BitWidth, ReadOffset, WriteOffset, Overlap, OnesExpectedText, OnesText),
								bOnesMatchesExpected);
						}
						else if (!bZeroesMatchesExpected)
						{
							CHECK_MESSAGE(FString::Printf(TEXT("MemmoveBitsWordOrder BitWidth=%d ReadOffset=%d, WriteOffset=%d, Overlap=%d, Zeroes\nExpected=%s\nActual  =%s"), BitWidth, ReadOffset, WriteOffset, Overlap, ZeroesExpectedText, ZeroesText),
								bZeroesMatchesExpected);
						}
					}
				}
			}
		}

		// Test use of ModularizeWordOffset
		{
			uint32 ReadBuffer[MaxDataLength];
			uint32 WriteBuffer[MaxDataLength];
			char Expected[MaxDataLength * NumBitsPerDWORD];
			char Actual[MaxDataLength * NumBitsPerDWORD];
			int32 Offsets[] = { -5, NumBitsPerDWORD + 5 };
			for (int32 Offset : Offsets)
			{
				int32 BitWidth = NumBitsPerDWORD;
				int32 Index;
				for (Index = 0; Index < MaxDataLength; ++Index)
				{
					ReadBuffer[Index] = 0;
					WriteBuffer[Index] = 0;
				}

				// Write 1s into the read range
				uint32* Read = ReadBuffer + 5;
				for (Index = 0; Index < BitWidth; ++Index)
				{
					int32 ReadOffset = Offset + Index;
					int32 ReadWord = (ReadOffset >= 0) ? ReadOffset / NumBitsPerDWORD : (ReadOffset - NumBitsPerDWORD - 1) / NumBitsPerDWORD;
					int32 ReadBitOffset = ReadOffset - ReadWord * NumBitsPerDWORD;
					Read[ReadWord] |= 1 << ReadBitOffset;
				}

				uint32* Write = WriteBuffer + 5;

				FBitArrayMemory::MemmoveBitsWordOrder(Write, Offset, Read, Offset, BitWidth);

				// Verify the WriteRange is 1s
				bool bMatch = true;
				for (Index = 0; Index < BitWidth; ++Index)
				{
					int32 ReadOffset = Offset + Index;
					int32 ReadWord = (ReadOffset >= 0) ? ReadOffset / NumBitsPerDWORD : (ReadOffset - NumBitsPerDWORD - 1) / NumBitsPerDWORD;
					int32 ReadBitOffset = ReadOffset - ReadWord * NumBitsPerDWORD;
					bool Value = (Write[ReadWord] & (1 << ReadBitOffset)) != 0;
					Expected[Index] = '1';
					Actual[Index] = Value ? '1' : '0';
					bMatch = bMatch & (Value == 1);
				}
				Expected[Index] = '\0';
				Actual[Index] = '\0';
				
				CHECK_MESSAGE(FString::Printf(TEXT("MemmoveBitsWordOrder ModularizeWordOffset Offset=%d\nExpected=%s\nActual  =%s"), Offset, ANSI_TO_TCHAR(Expected), ANSI_TO_TCHAR(Actual)),
					bMatch);
			}
		}
	}

	void TestModularizeWordOffset()
	{
		constexpr int MaxDataLength = 20;
		uint32 Buffer[MaxDataLength];

		int32 WordLengths[] = { -5, -1, 0, 1, 5 };
		for (int32 WordLength : WordLengths)
		{
			int32 BitLengths[] = { 0, 5 };
			for (uint32 BitLength : BitLengths)
			{
				uint32* BaseData = Buffer + 10;
				uint32* Data = BaseData;
				int32 Offset = WordLength * NumBitsPerDWORD + BitLength;
				FBitArrayMemory::ModularizeWordOffset(Data, Offset);
				uint32* ExpectedData = BaseData + WordLength;
				uint32 ExpectedOffset = BitLength;
				CHECK_MESSAGE(FString::Printf(TEXT("ModularizeWordOffset WordLength=%d, BitLength=%d\nExpected: Data=%d, Offset=%d\nActual:  Data=%d, Offset=%d"), WordLength, BitLength, UE_PTRDIFF_TO_INT32(ExpectedData - BaseData), ExpectedOffset, UE_PTRDIFF_TO_INT32(Data - BaseData), Offset),
					!(Data != ExpectedData || Offset != ExpectedOffset));
			}
		}
	}
};

TEST_CASE_NAMED(FBitArrayMemoryTestSubClass, "System::Core::Misc::MemmoveBitsWordOrder", "[ApplicationContextMask][EngineFilter]")
{
	FBitArrayMemoryTest Instance;
	Instance.TestMemmoveBitsWordOrder();
	Instance.TestModularizeWordOffset();
}

#endif // WITH_TESTS
