// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/ChunkedArray.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::Private
{

// Use TChunkArray of bytes in order to be able to easily specify the number of elements per chunk instead of bytes per chunk.
template<uint32 ElementsPerChunk>
class TChunkedArrayForAddTest : public TChunkedArray<uint8, ElementsPerChunk>
{
public:
	int32 GetNumChunks() const
	{
		return this->Chunks.Num();
	}
};

TEST_CASE_NAMED(TChunkedArrayAddAllocatesProperAmountOfChunksTest, "System::Core::Containers::ChunkedArray::Add", "[Core][Containers][ChunkedArray][SmokeFilter]")
{
	SECTION("One element per chunk")
	{
		constexpr uint32 ElementsPerChunk = 1;
		const int32 NumElementsToAddTests[] = {0, 1, 2, 12345};
		for (const int32 NumElementsToAdd : NumElementsToAddTests)
		{
			TChunkedArrayForAddTest<ElementsPerChunk> ChunkedArray;
			ChunkedArray.Add(NumElementsToAdd);
			REQUIRE(ChunkedArray.GetNumChunks() == NumElementsToAdd);
			REQUIRE(ChunkedArray.Num() == NumElementsToAdd);
		}
	}

	SECTION("Arbitrary num elements per chunk")
	{
		constexpr uint32 ElementsPerChunk = 47;
		const int32 NumElementsToAddTests[] = {0, 1, ElementsPerChunk - 1, ElementsPerChunk, ElementsPerChunk + 1, 8999};
		for (const int32 NumElementsToAdd : NumElementsToAddTests)
		{
			TChunkedArrayForAddTest<ElementsPerChunk> ChunkedArray;
			ChunkedArray.Add(NumElementsToAdd);
			const int32 ExpectedNumChunks = (NumElementsToAdd + ElementsPerChunk - 1)/ElementsPerChunk;
			REQUIRE(ChunkedArray.GetNumChunks() == ExpectedNumChunks);
			REQUIRE(ChunkedArray.Num() == NumElementsToAdd);
		}
	}

	SECTION("Add to non-empty array")
	{
		constexpr uint32 ElementsPerChunk = 47;
		const int32 NumElementsToAddTests[] = {0, 1, ElementsPerChunk - 1, ElementsPerChunk, ElementsPerChunk + 1, 8999};
		for (const int32 NumElementsToStartWith : NumElementsToAddTests)
		{
			for (const int32 NumElementsToAdd : NumElementsToAddTests)
			{
				// Start off with an arbitrary number of elements
				TChunkedArrayForAddTest<ElementsPerChunk> ChunkedArray;
				ChunkedArray.Add(NumElementsToStartWith);

				// Add additional elements
				ChunkedArray.Add(NumElementsToAdd);

				const int32 ExpectedNumElements = NumElementsToStartWith + NumElementsToAdd;
				const int32 ExpectedNumChunks = (ExpectedNumElements + ElementsPerChunk - 1)/ElementsPerChunk;
				REQUIRE(ChunkedArray.GetNumChunks() == ExpectedNumChunks);
				REQUIRE(ChunkedArray.Num() == ExpectedNumElements);
			}
		}
	}
}

TEST_CASE_NAMED(TChunkedArrayTIteratorTest, "System::Core::Containers::ChunkedArray::TIterator", "[Core][Containers][ChunkedArray]")
{
	struct FDataTest
	{
	public:
		FDataTest() = default;
		~FDataTest() = default;

		FDataTest(int32 InValue)
			:Data{ InValue, InValue, InValue, InValue }
		{}

		int32 GetValue() const
		{
			return Data[0];
		}

	private:
		int32 Data[4];
	};

	auto TestArray = [](int32 NumToTest)
	{
		TChunkedArray<FDataTest, 16384> ChunkedData;
		for (int32 Index = 0; Index < NumToTest; ++Index)
		{
			ChunkedData.AddElement(FDataTest(Index));
		}

		int32 Count = 0;
		for (FDataTest& Data : ChunkedData)
		{
			REQUIRE(Count++ == Data.GetValue());
		}

		REQUIRE(Count == ChunkedData.Num());
	};

	// This test was originally written to test the theory that FChunkedArray had a bug
	// when used in a ranged for loop (it did) which was done via:
	// MinBytes = 1;
	// MaxBytes = 16384;
	// which created enough allocation churn to repreoduce the problem. However that test
	// took a long time to run and is unsuitable for a fast test. The following values are
	// a cutdown version of the original test which seem to reproduce the original issue
	// that as fixed in CL 29283139

	const int32 MinBytes = 2048;
	const int32 MaxBytes = 4096;

	for (int32 Index = MinBytes; Index < MaxBytes; Index++)
	{
		TestArray(Index);
	}
}


}

#endif // WITH_TESTS
