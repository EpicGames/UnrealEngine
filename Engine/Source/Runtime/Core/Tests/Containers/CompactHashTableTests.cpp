// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/CompactHashTable.h"
#include "Tests/TestHarnessAdapter.h"

struct FCommonCompactHashTableTests
{
	template<typename HashTableType>
	static uint32 TestFindExpectedIndex(const HashTableType& HashTable, uint32 Key, uint32 Index, uint32 CurrentSize)
	{
		uint32 Steps = 0;

		uint32 FindIndex = HashTable.GetFirst(Key);
		while (FindIndex != Index && FindIndex != INDEX_NONE)
		{
			++Steps;
			FindIndex = HashTable.GetNext(FindIndex, CurrentSize);
		}

		CHECK(FindIndex == Index);

		return Steps;
	}

	template<uint32 Size, typename HashTableType>
	static void TestAddRemove(HashTableType& HashTable)
	{
		uint32 Data[Size];
		for (uint32 Index = 0; Index < Size; ++Index)
		{
			Data[Index] = (Index + 1) * Size - 7; // Math isn't important, it's unique and lets us hit collisions, that's the important part
			HashTable.Add(Index, Data[Index]);
		}

		uint32 CurrentSize = Size;
		uint32 Collisions = 0;
		for (uint32 FullIndex = 0; FullIndex < Size; ++FullIndex)
		{
			// Pick a consistent random element to remove
			const uint32 Index = Data[FullIndex % CurrentSize] % CurrentSize;

			Collisions += TestFindExpectedIndex(HashTable, Data[Index], Index, CurrentSize);

			const uint32 LastElement = --CurrentSize;
			HashTable.Remove(Index, Data[Index], LastElement, Data[LastElement]);

			if (Index != LastElement)
			{
				// User needs to also move data
				Data[Index] = Data[LastElement];

				Collisions += TestFindExpectedIndex(HashTable, Data[Index], Index, CurrentSize);
			}
		}

		CHECK(Collisions > 0); // I'd expect at least a few, make sure we hit those
	}

	template<uint32 Size, typename HashTableType>
	static void TestSynchedTables(HashTableType& HashTableKeys, HashTableType& HashTableValues)
	{
		struct Pair
		{
			uint32 Key;
			uint32 Value;
		};

		Pair Data[Size];

		for (uint32 Index = 0; Index < Size; ++Index)
		{
			Data[Index].Key = (Index + 1) * Size - 7;
			Data[Index].Value = Data[Index].Key + 1;

			HashTableKeys.Add(Index, Data[Index].Key);
			HashTableValues.Add(Index, Data[Index].Value);
		}

		uint32 CurrentSize = Size;
		uint32 Collisions = 0;

		for (uint32 FullIndex = 0; FullIndex < Size; ++FullIndex)
		{
			// Pick a consistent random element to remove
			const uint32 Index = Data[FullIndex % CurrentSize].Key % CurrentSize;

			CHECK(TestFindExpectedIndex(HashTableKeys, Data[Index].Key, Index, CurrentSize) == TestFindExpectedIndex(HashTableValues, Data[Index].Value, Index, CurrentSize));

			const uint32 LastElement = --CurrentSize;
			HashTableKeys.Remove(Index, Data[Index].Key, LastElement, Data[LastElement].Key);
			HashTableValues.Remove(Index, Data[Index].Value, LastElement, Data[LastElement].Value);

			if (Index != LastElement)
			{
				// User needs to also move data
				Data[Index] = Data[LastElement];

				CHECK(TestFindExpectedIndex(HashTableKeys, Data[Index].Key, Index, CurrentSize) == TestFindExpectedIndex(HashTableValues, Data[Index].Value, Index, CurrentSize));
			}
		}
	}


};

TEST_CASE_NAMED(FStaticCompactHashTableTest, "System::Core::Containers::CompactHashTable::Static", "[Core][Containers][CompactHashTable]")
{
	SECTION("Constructor Behavior")
	{
		static_assert(sizeof(TStaticCompactHashTable<16>::IndexType) == sizeof(uint8));
		static_assert(sizeof(TStaticCompactHashTable<256>::IndexType) == sizeof(uint16));
		static_assert(sizeof(TStaticCompactHashTable<65536>::IndexType) == sizeof(uint32));

		static_assert(sizeof(TStaticCompactHashTable<16>) == (16 + 16) * sizeof(uint8));
		static_assert(sizeof(TStaticCompactHashTable<16, 32>) == (16 + 32) * sizeof(uint8));
		static_assert(sizeof(TStaticCompactHashTable<256>) == (256 + 256) * sizeof(uint16));
		static_assert(sizeof(TStaticCompactHashTable<256, 512>) == (256 + 512) * sizeof(uint16));
		static_assert(sizeof(TStaticCompactHashTable<65536>) == (65536 + 65536) * sizeof(uint32));
		static_assert(sizeof(TStaticCompactHashTable<65536, 65537>) == (65536 + 65537) * sizeof(uint32));

		const uint8 Sentinel = 0xf0;
		const uint8 Clear = 0xff;
		
		auto TestTableType = []<uint32 Size, uint32 HashSize = Size>()
		{
			using TestType = TStaticCompactHashTable<Size, HashSize>;

			alignas(TestType) uint8 TestMemory[sizeof(TestType)];
			FMemory::Memset(TestMemory, Sentinel);

			new (TestMemory) TestType(NoInit);

			for (uint8 Test : TestMemory)
			{
				CHECK(Test == Sentinel);
			}

			new (TestMemory) TestType();

			for (int32 Index = 0; Index < sizeof(TestType); ++Index)
			{
				if (Index < Size * sizeof(typename TestType::IndexType))
				{
					CHECK(TestMemory[Index] == Sentinel);
				}
				else
				{
					CHECK(TestMemory[Index] == Clear);
				}
			}
		};

		TestTableType.operator()<16>();
		TestTableType.operator()<16, 32>();
		TestTableType.operator()<256>();
		TestTableType.operator()<255, 128>();
	}

	SECTION("Add Behavior")
	{
		auto TestTableType = []<uint32 Size, uint32 HashSize = Size>()
		{
			TStaticCompactHashTable<Size, HashSize> HashTable;
			FCommonCompactHashTableTests::TestAddRemove<Size>(HashTable);
		};

		TestTableType.operator()<16>();
		TestTableType.operator()<16, 32>();
		TestTableType.operator()<256>();
		TestTableType.operator()<255, 128>();
	}

	SECTION("Synched Tables Behavior")
	{
		auto TestTableType = []<uint32 Size, uint32 HashSize = Size>()
		{
			TStaticCompactHashTable<Size, HashSize> HashTableKeys;
			TStaticCompactHashTable<Size, HashSize> HashTableValues;
			FCommonCompactHashTableTests::TestSynchedTables<Size>(HashTableKeys, HashTableValues);
		};

		TestTableType.operator()<16>();
		TestTableType.operator()<16, 32>();
		TestTableType.operator()<256>();
		TestTableType.operator()<255, 128>();
	}
}

TEST_CASE_NAMED(FCompactHashTableViewTest, "System::Core::Containers::CompactHashTable::Dynamic", "[Core][Containers][CompactHashTable]")
{
	SECTION("Constructor Behavior")
	{
		const uint8 Sentinel = 0xf0;
		const uint8 Clear = 0xff;

		auto TestTableType = []<uint32 Size, uint32 HashSize = Size>()
		{
			using IndexType = typename TStaticCompactHashTable<Size>::IndexType;
			using TestType = FCompactHashTableView;

			constexpr size_t RequiredSize = UE::Core::CompactHashTable::GetMemoryRequiredInBytes(Size, HashSize);
			constexpr uint32 IndexStorageSize = Size * UE::Core::Private::CompactSetAllocatorHelpers::KeyStorageMultiplier;
			static_assert(RequiredSize == sizeof(typename TStaticCompactHashTable<Size>::IndexType) * (IndexStorageSize + HashSize));

			uint8 TestMemory[RequiredSize];
			FMemory::Memset(TestMemory, Sentinel, RequiredSize);

			TestType HashTable(TestMemory, Size, HashSize, RequiredSize);

			for (uint8 Test : TestMemory)
			{
				CHECK(Test == Sentinel);
			}

			HashTable.Reset();
			
			for (int32 Index = 0; Index < RequiredSize; ++Index)
			{
				if (Index < IndexStorageSize * sizeof(IndexType))
				{
					CHECK(TestMemory[Index] == Sentinel);
				}
				else
				{
					CHECK(TestMemory[Index] == Clear);
				}
			}
		};

		TestTableType.operator()<16>();
		TestTableType.operator()<16, 32>();
		TestTableType.operator()<256>();
		TestTableType.operator()<255, 128>();
	}

	SECTION("Add Behavior")
	{
		auto TestTableType = []<uint32 Size, uint32 HashSize = Size>()
		{
			using IndexType = typename TStaticCompactHashTable<Size>::IndexType;
			using TestType = FCompactHashTableView;

			constexpr size_t RequiredSize = UE::Core::CompactHashTable::GetMemoryRequiredInBytes(Size, HashSize);

			uint8 TestMemory[RequiredSize];
			TestType HashTable(TestMemory, Size, HashSize, RequiredSize);
			HashTable.Reset();

			FCommonCompactHashTableTests::TestAddRemove<Size>(HashTable);
		};

		TestTableType.operator()<16>();
		TestTableType.operator()<16, 32>();
		TestTableType.operator()<256>();
		TestTableType.operator()<255, 128>();
	}

	SECTION("Synched Tables Behavior")
	{
		auto TestTableType = []<uint32 Size, uint32 HashSize = Size>()
		{
			using IndexType = typename TStaticCompactHashTable<Size>::IndexType;
			using TestType = FCompactHashTableView;

			constexpr size_t RequiredSize = UE::Core::CompactHashTable::GetMemoryRequiredInBytes(Size, HashSize);

			uint8 TestMemoryKeys[RequiredSize];
			TestType HashTableKeys(TestMemoryKeys, Size, HashSize, RequiredSize);
			HashTableKeys.Reset();

			uint8 TestMemoryValues[RequiredSize];
			TestType HashTableValues(TestMemoryValues, Size, HashSize, RequiredSize);
			HashTableValues.Reset();

			FCommonCompactHashTableTests::TestSynchedTables<Size>(HashTableKeys, HashTableValues);
		};

		TestTableType.operator()<16>();
		TestTableType.operator()<16, 32>();
		TestTableType.operator()<256>();
		TestTableType.operator()<255, 128>();
	}
}

#endif // WITH_TESTS
