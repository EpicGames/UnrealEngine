// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/CircularQueue.h"
#include "Containers/Queue.h"
#include "Containers/SpscQueue.h"
#include "Containers/MpscQueue.h"
#include "Containers/ClosableMpscQueue.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Tests/Benchmark.h"

#include <atomic>

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"


namespace UE { namespace ConcurrentQueuesTests
{
	template<typename QueueType>
	class TQueueAdapter : public QueueType
	{
	public:
		template<typename... ArgTypes>
		TQueueAdapter(ArgTypes... Args)
			: QueueType(Forward<ArgTypes>(Args)...)
		{
		}

		TOptional<typename QueueType::FElementType> Dequeue()
		{
			typename QueueType::FElementType Value;
			if (QueueType::Dequeue(Value))
			{
				return Value;
			}
			else
			{
				return {};
			}
		}
	};

	const uint32 CircularQueueSize = 1024;

	// measures performance of a queue when the producer and the consumer are run in the same thread
	template<uint32 Num, typename QueueType>
	void TestSpscQueueSingleThreadImp(QueueType& Queue)
	{
		const uint32 BatchSize = CircularQueueSize;
		const uint32 BatchNum = Num / BatchSize;
		for (uint32 BatchIndex = 0; BatchIndex != BatchNum; ++BatchIndex)
		{
			for (uint32 i = 0; i != BatchSize; ++i)
			{
				Queue.Enqueue(i);
			}

			for (uint32 i = 0; i != BatchSize; ++i)
			{
				TOptional<uint32> Consumed{ Queue.Dequeue() };
				checkSlow(Consumed.IsSet() && Consumed.GetValue() == i);
			}
		}
	}

	template<uint32 Num>
	void TestTCircularQueueSingleThread()
	{
		TQueueAdapter<TCircularQueue<uint32>> Queue{ CircularQueueSize + 1 };
		TestSpscQueueSingleThreadImp<Num>(Queue);
	}

	template<uint32 Num, typename QueueType>
	void TestQueueSingleThread()
	{
		QueueType Queue;
		TestSpscQueueSingleThreadImp<Num>(Queue);
	}

	template<uint32 Num, typename QueueType>
	void TestSpscQueue_Impl(QueueType& Queue)
	{
		std::atomic<bool> bStop{ false };

		FGraphEventRef Producer = FFunctionGraphTask::CreateAndDispatchWhenReady(
			[&bStop, &Queue]
			{
				while (!bStop)
				{
					Queue.Enqueue(0);
				}
			}
			);

		// consumer
		uint32 It = 0;
		while (It != Num)
		{
			if (Queue.Dequeue().IsSet())
			{
				++It;
			}
		}

		bStop = true;

		Producer->Wait(ENamedThreads::GameThread);
	}

	template<uint32 Num>
	void TestTCircularQueue()
	{
		TQueueAdapter<TCircularQueue<uint32>> Queue{ CircularQueueSize + 1 };
		TestSpscQueue_Impl<Num>(Queue);
	}

	template<uint32 Num, typename QueueType>
	void TestSpscQueue()
	{
		QueueType Queue;
		TestSpscQueue_Impl<Num>(Queue);
	}

	template<uint32 Num, typename QueueType>
	void TestSpscQueueCorrectness()
	{
		QueueType Queue;
		int32 NumProduced = 0;

		// producer
		FGraphEventRef Task = FFunctionGraphTask::CreateAndDispatchWhenReady(
			[&Queue, &NumProduced]
			{
				do
				{
					Queue.Enqueue(0);
				} while (++NumProduced != Num);
			}
		);

		// consumer
		uint32 NumConcumed = 0;
		while (NumConcumed != Num)
		{
			if (Queue.Dequeue().IsSet())
			{
				++NumConcumed;
			}
		}

		while (Queue.Dequeue().IsSet())
		{
			++NumConcumed;
		}

		Task->Wait();

		check(NumProduced == NumConcumed);
	}

	template<uint32 Num, typename QueueType>
	void TestMpscQueue()
	{
		QueueType Queue;
		std::atomic<bool> bStop{ false };

		int32 NumProducers = FTaskGraphInterface::Get().GetNumWorkerThreads();
		FGraphEventArray Producers;
		for (int32 i = 0; i != NumProducers; ++i)
		{
			Producers.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(
				[&bStop, &Queue]
				{
					while (!bStop)
					{
						Queue.Enqueue(0);
					}
				}
				));
		}

		uint32 It = 0;
		while (It != Num)
		{
			if (Queue.Dequeue().IsSet())
			{
				++It;
			}
		}

		bStop = true;

		FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(Producers), ENamedThreads::GameThread);
	}

	template<uint32 Num, typename QueueType>
	void TestMpscQueueCorrectness()
	{
		struct alignas(PLATFORM_CACHE_LINE_SIZE) FCounter
		{
			uint32 Count = 0;
		};

		QueueType Queue;

		const int32 NumProducers = FTaskGraphInterface::Get().GetNumWorkerThreads();
		const uint32 NumPerProducer = Num / NumProducers;
		TArray<FCounter, TAlignedHeapAllocator<PLATFORM_CACHE_LINE_SIZE>> NumsProduced;
		NumsProduced.AddDefaulted(NumProducers);
		for (int32 i = 0; i != NumProducers; ++i)
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[&Queue, &NumsProduced, i, NumPerProducer]()
			{
				do
				{
					Queue.Enqueue(0);
				} while (++NumsProduced[i].Count < NumPerProducer);
			}
			);
		}

		// consumer
		uint32 NumConcumed = 0;
		while (NumConcumed != NumPerProducer * NumProducers)
		{
			if (Queue.Dequeue().IsSet())
			{
				++NumConcumed;
			}
		}

		while (Queue.Dequeue().IsSet())
		{
			++NumConcumed;
		}

		uint32 Produced = 0;
		for (int i = 0; i != NumProducers; ++i)
		{
			Produced += NumsProduced[i].Count;
		}
		check(Produced == NumConcumed);
	}

	TEST_CASE_NAMED(FConcurrentQueuesTest, "System::Core::Async::ConcurrentQueuesTest", "[.][ApplicationContextMask][EngineFilter][Perf]")
	{
		{	// test for support of not default constructible types
			struct FNonDefaultConstructable
			{
				int Value;

				explicit FNonDefaultConstructable(int InValue)
					: Value(InValue)
				{
					UE_LOGF(LogTemp, Display, "ctor");
				}

				~FNonDefaultConstructable()
				{
					UE_LOGF(LogTemp, Display, "dctor");
				}

				FNonDefaultConstructable(const FNonDefaultConstructable&) = delete;
				FNonDefaultConstructable& operator=(const FNonDefaultConstructable&) = delete;

				FNonDefaultConstructable(FNonDefaultConstructable&& Other)
					: Value(Other.Value)
				{
					UE_LOGF(LogTemp, Display, "move-ctor");
				}

				FNonDefaultConstructable& operator=(FNonDefaultConstructable&& Other)
				{
					Value = Other.Value;
					UE_LOGF(LogTemp, Display, "move=");
					return *this;
				}
			};

			{
				TSpscQueue<FNonDefaultConstructable> Q;
				Q.Enqueue(1);
				TOptional<FNonDefaultConstructable> Res{ Q.Dequeue() };
				verify(Res.IsSet() && Res.GetValue().Value == 1);
			}

			{
				TMpscQueue<FNonDefaultConstructable> Q;
				Q.Enqueue(1);
				TOptional<FNonDefaultConstructable> Res{ Q.Dequeue() };
				verify(Res.IsSet() && Res.GetValue().Value == 1);
			}
		}

		{	// test queue destruction and not default movable types
			struct FNonTrivial
			{
				int* Value;

				explicit FNonTrivial(TUniquePtr<int> InValue)
					: Value(InValue.Release())
				{}

				~FNonTrivial()
				{
					// check for double delete
					check(Value != (int*)0xcdcdcdcdcdcdcdcd);
					delete Value;
					Value = (int*)0xcdcdcdcdcdcdcdcd;
				}

				FNonTrivial(const FNonTrivial&) = delete;
				FNonTrivial& operator=(const FNonTrivial&) = delete;

				FNonTrivial(FNonTrivial&& Other)
				{
					Value = Other.Value;
					Other.Value = nullptr;
				}

				FNonTrivial& operator=(FNonTrivial&& Other)
				{
					Swap(Value, Other.Value);
					return *this;
				}
			};

			// SPSC

			{	// destroy queue while it's holding one unconsumed item
				TSpscQueue<FNonTrivial> Q;
				Q.Enqueue(MakeUnique<int>(1));
			}

			{	// destroy queue while it's holding one cached consumed time
				TSpscQueue<FNonTrivial> Q;
				Q.Enqueue(MakeUnique<int>(1));
				TOptional<FNonTrivial> Res{ Q.Dequeue() };
				verify(Res.IsSet() && *Res.GetValue().Value == 1);
			}

			{	// destroy queue while it's holding one chached consumed item and one unconsumed item
				TSpscQueue<FNonTrivial> Q;
				Q.Enqueue(MakeUnique<int>(1));
				Q.Enqueue(MakeUnique<int>(2));
				TOptional<FNonTrivial> Res{ Q.Dequeue() };
				verify(Res.IsSet() && *Res.GetValue().Value == 1);
			}

			// MPSC
			{	// destroy untouched queue
				TMpscQueue<FNonTrivial> Q;
			}

			{	// destroy never consumed queue with one unconsumed item
				TMpscQueue<FNonTrivial> Q;
				Q.Enqueue(MakeUnique<int>(1));
			}

			{	// destroy empty queue
				TMpscQueue<FNonTrivial> Q;
				Q.Enqueue(MakeUnique<int>(1));
				TOptional<FNonTrivial> Res{ Q.Dequeue() };
				verify(Res.IsSet() && *Res.GetValue().Value == 1);
			}

			{	// destroy queue with one unconsumed item
				TMpscQueue<FNonTrivial> Q;
				Q.Enqueue(MakeUnique<int>(1));
				Q.Enqueue(MakeUnique<int>(2));
				TOptional<FNonTrivial> Res{ Q.Dequeue() };
				verify(Res.IsSet() && *Res.GetValue().Value == 1);
			}

			{	// destroy queue with two items
				TMpscQueue<FNonTrivial> Q;
				Q.Enqueue(MakeUnique<int>(1));
				Q.Enqueue(MakeUnique<int>(2));
			}

			{	// enqueue and dequeue multiple items
				TMpscQueue<FNonTrivial> Q;
				Q.Enqueue(MakeUnique<int>(1));
				Q.Enqueue(MakeUnique<int>(2));
				TOptional<FNonTrivial> Res{ Q.Dequeue() };
				verify(Res.IsSet() && *Res.GetValue().Value == 1);
				Res = Q.Dequeue();
				verify(Res.IsSet() && *Res.GetValue().Value == 2);
			}

			{	// enqueue and dequeue (interleaved) multiple items
				TMpscQueue<FNonTrivial> Q;
				Q.Enqueue(MakeUnique<int>(1));
				TOptional<FNonTrivial> Res{ Q.Dequeue() };
				verify(Res.IsSet() && *Res.GetValue().Value == 1);
				Q.Enqueue(MakeUnique<int>(2));
				Res = Q.Dequeue();
				verify(Res.IsSet() && *Res.GetValue().Value == 2);
			}
		}

		{ // test `TSpscQueue::IsEmpty()`
			TSpscQueue<int> Q;
			check(Q.IsEmpty());
			Q.Enqueue(42);
			check(!Q.IsEmpty());
			Q.Dequeue();
			check(Q.IsEmpty());
		}

		{ // test `TMpscQueue::IsEmpty()`
			TMpscQueue<int> Q;
			check(Q.IsEmpty());
			Q.Enqueue(42);
			check(!Q.IsEmpty());
			Q.Dequeue();
			check(Q.IsEmpty());
		}

		{	// test `Peek()`
			struct FMovableOnly
			{
				explicit FMovableOnly(int32 InId)
					: Id(InId)
				{
				}

				FMovableOnly(FMovableOnly&&) = default;
				FMovableOnly& operator=(FMovableOnly&&) = default;

				int32 Id;
			};

			{
				TSpscQueue<FMovableOnly> Q;
				Q.Enqueue(42);
				FMovableOnly* Peeked = Q.Peek();
				check(Peeked);
				check(Peeked->Id == 42);
				TOptional<FMovableOnly> Dequeued = Q.Dequeue();
				check(Dequeued);
				check(Dequeued.GetValue().Id == Peeked->Id);
			}

			{
				TMpscQueue<FMovableOnly> Q;
				Q.Enqueue(42);
				FMovableOnly* Peeked = Q.Peek();
				check(Peeked);
				check(Peeked->Id == 42);
				TOptional<FMovableOnly> Dequeued = Q.Dequeue();
				check(Dequeued);
				check(Dequeued.GetValue().Id == Peeked->Id);
			}
		}

		UE_BENCHMARK(5, TestTCircularQueueSingleThread<5'000'000>);
		UE_BENCHMARK(5, TestQueueSingleThread<5'000'000, TQueueAdapter<TQueue<uint32, EQueueMode::Spsc>>>);
		UE_BENCHMARK(5, TestQueueSingleThread<5'000'000, TQueueAdapter<TQueue<uint32, EQueueMode::Mpsc>>>);
		UE_BENCHMARK(5, TestQueueSingleThread<5'000'000, TMpscQueue<uint32>>);
		UE_BENCHMARK(5, TestQueueSingleThread<5'000'000, TSpscQueue<uint32>>);

		UE_BENCHMARK(5, TestSpscQueueCorrectness<1'000'000, TMpscQueue<uint32>>);
		UE_BENCHMARK(5, TestSpscQueueCorrectness<1'000'000, TSpscQueue<uint32>>);

		UE_BENCHMARK(5, TestTCircularQueue<5'000'000>);
		UE_BENCHMARK(5, TestSpscQueue<5'000'000, TQueueAdapter<TQueue<uint32, EQueueMode::Spsc>>>);
		UE_BENCHMARK(5, TestSpscQueue<5'000'000, TQueueAdapter<TQueue<uint32, EQueueMode::Mpsc>>>);
		UE_BENCHMARK(5, TestSpscQueue<5'000'000, TMpscQueue<uint32>>);
		UE_BENCHMARK(5, TestSpscQueue<5'000'000, TSpscQueue<uint32>>);

#if 0 // the test seems to be broken
		UE_BENCHMARK(5, TestMpscQueueCorrectness<5'000'000, TQueueAdapter<TQueue<uint32, EQueueMode::Mpsc>>>);
		UE_BENCHMARK(5, TestMpscQueueCorrectness<5'000'000, TMpscQueue<uint32>>);
#endif

		UE_BENCHMARK(5, TestMpscQueue<100'000, TQueueAdapter<TQueue<uint32, EQueueMode::Mpsc>>>);
		UE_BENCHMARK(5, TestMpscQueue<100'000, TMpscQueue<uint32>>);
	}
}

namespace ClosableMpscQueueTests
{
	// straightforward implementation over MPSC TQueue for comparison with TClosableMpscQueue
	template<typename T>
	class TClosableMpscQueueOnTQueue
	{
	public:
		bool Enqueue(T Value)
		{
			bool bRes = false;

			// `acquire` to make it "happen-before" reading `bIsComplete` and `release` to "synchronise-with" reading from other threads
			verify(SubscribingThreadsNum.fetch_add(1, std::memory_order_acq_rel) >= 0);
			// `acquire` to "synchronise-with" the consumer
			if (!bIsComplete.load(std::memory_order_acquire))
			{
				ensureAlways(Queue.Enqueue(MoveTemp(Value)));
				bRes = true;
			}
			// `release` to "synchonise-with" other threads
			verify(SubscribingThreadsNum.fetch_sub(1, std::memory_order_release) >= 1);

			return bRes;
		}

		template<typename F>
		void Close(F&& Consumer)
		{
			// `seq_cst` to make it "happen-before" reading `SubscribingThreadsNum`, and to "synchronise-with" reading from other threads
			bIsComplete.store(true, std::memory_order_seq_cst);

			// `acquire` to "synchronise-with" other threads and to make it "happen-before" dequeueing
			while (SubscribingThreadsNum.load(std::memory_order_acquire) != 0)
			{
				FPlatformProcess::Yield();
			}

			T Value;
			while (Queue.Dequeue(Value))
			{
				Consumer(MoveTemp(Value));
			}
		}

	private:
		TQueue<T, EQueueMode::Mpsc> Queue;
		std::atomic<bool> bIsComplete{ false };
		std::atomic<uint32> SubscribingThreadsNum{ 0 };
	};

	// straightforward implementation over TMpscQueue for comparison with TClosableMpscQueue
	template<typename T>
	class TClosableMpscQueueOnTMpscQueue
	{
	public:
		bool Enqueue(T Value)
		{
			bool bRes = false;

			// `acquire` to make it "happen-before" reading `bIsComplete` and `release` to "synchronise-with" reading from other threads
			verify(SubscribingThreadsNum.fetch_add(1, std::memory_order_acq_rel) >= 0);
			// `acquire` to "synchronise-with" the consumer
			if (!bIsComplete.load(std::memory_order_acquire))
			{
				Queue.Enqueue(MoveTemp(Value));
				bRes = true;
			}
			// `release` to "synchonise-with" other threads
			verify(SubscribingThreadsNum.fetch_sub(1, std::memory_order_release) >= 1);

			return bRes;
		}

		template<typename F>
		void Close(F&& Consumer)
		{
			// `seq_cst` to make it "happen-before" reading `SubscribingThreadsNum`, and to "synchronise-with" reading from other threads
			bIsComplete.store(true, std::memory_order_seq_cst);

			// `acquire` to "synchronise-with" other threads and to make it "happen-before" dequeueing
			while (SubscribingThreadsNum.load(std::memory_order_acquire) != 0)
			{
				FPlatformProcess::Yield();
			}

			while (TOptional<T> Value = Queue.Dequeue())
			{
				Consumer(MoveTemp(Value.GetValue()));
			}
		}

	private:
		TMpscQueue<T> Queue;
		std::atomic<bool> bIsComplete{ false };
		std::atomic<uint32> SubscribingThreadsNum{ 0 };
	};

	// wrapper over `TClosableLockFreePointerListUnorderedSingleConsumer`
	template<typename T>
	class TClosableLockFreePointerListUnorderedSingleConsumer_Adapter
	{
	public:
		bool Enqueue(T* Value)
		{
			return Queue.PushIfNotClosed(MoveTemp(Value));
		}

		template<typename F>
		void Close(F&& Consumer)
		{
			TArray<T*> Items;
			Queue.PopAllAndClose(Items);
			for (T* Item : Items)
			{
				Consumer(MoveTemp(Item));
			}
		}

	private:
		TClosableLockFreePointerListUnorderedSingleConsumer<T, 0 /* as was used in FGraphEvent */> Queue;
	};

	template<uint32 Num, typename QueueT>
	void TestClosingEmptyQueue()
	{
		for (uint32 i = 0; i != Num; ++i)
		{
			QueueT{}.Close([](void*) {});
		}
	}

	// launches a number of producer tasks that push incremental numbers into a queue until it's closed. then, on the main thread,
	// the queue is closed, supposedly concurrently to producers trying pushing into it. after that it's checked that the same total sum was 
	// pushed and popped from the queue
	template<int32 Num, typename QueueT>
	void TestCorrectness()
	{
		for (int32 Run = 0; Run != Num; ++Run)
		{
			QueueT Queue;

			int32 ProducersNum = FPlatformMisc::NumberOfWorkerThreadsToSpawn();

			FGraphEventArray Producers;
			Producers.Reserve(ProducersNum);

			TArray<uint64> NumsProduced;
			NumsProduced.AddZeroed(ProducersNum);

			for (int32 i = 0; i != ProducersNum; ++i)
			{
				Producers.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(
					[&Queue, NumProduced = &NumsProduced[i]]
					{
						uint64 i = 0;
						while (Queue.Enqueue((void*)(intptr_t)(++i)))
						{
						}

						*NumProduced = i - 1;
					}
				));
			}

			FPlatformProcess::Yield();

			uint64 Consumed = 0;
			Queue.Close([&Consumed](void* Value) { Consumed += (uint64)(intptr_t)Value; });

			FTaskGraphInterface::Get().WaitUntilTasksComplete(Producers);

			uint64 Produced = 0;
			for (int32 i = 0; i != ProducersNum; ++i)
			{
				uint64 N = NumsProduced[i];
				Produced += N * (N + 1) / 2;
			}

			checkf(Produced == Consumed, TEXT("%" INT64_FMT " - %" INT64_FMT ", %d run"), Produced, Consumed, Run);

			if (Produced == 0)
			{
				Run -= 1; // discard this run
			}
		}
	}

	template<uint32 Num, typename QueueType>
	void TestPerf()
	{
		TArray<QueueType, TAlignedHeapAllocator<alignof(QueueType)>> Queues;
		Queues.AddDefaulted(Num);
		std::atomic<int> Index{ 0 };

		const int32 NumProducers = FPlatformMisc::NumberOfWorkerThreadsToSpawn();
		TArray<int32> ProducersResults;
		ProducersResults.AddDefaulted(NumProducers);

		FGraphEventArray Events;
		for (int ProducerIndex = 0; ProducerIndex != NumProducers; ++ProducerIndex)
		{
			Events.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(
				[&Queues, &Index, &ProducersResults, ProducerIndex]
				{
					int NumProduced = 0;
					int LocalIndex = Index.load(std::memory_order_acquire);

					while (LocalIndex != Num && Queues[LocalIndex].Enqueue((void*)((intptr_t)NumProduced + 1)))
					{
						++NumProduced;
						LocalIndex = Index.load(std::memory_order_acquire);
					}

					ProducersResults[ProducerIndex] = NumProduced;
				}
				));
			Events.Last()->SetDebugName(TEXT("ClosableMpscQueueTest.TestPerf.Producer"));
		}

		int32 NumConsumed = 0;
		for (int LocalIndex = 0; LocalIndex != Num; ++LocalIndex)
		{
			Queues[LocalIndex].Close([&NumConsumed](void*) { ++NumConsumed; });
			Index.store(LocalIndex + 1, std::memory_order_release);
			FPlatformProcess::Yield();
			FPlatformProcess::Yield();
			FPlatformProcess::Yield();
		}

		FTaskGraphInterface::Get().WaitUntilTasksComplete(Events);

		int32 NumProduced = 0;
		for (int i = 0; i != NumProducers; ++i)
		{
			NumProduced += ProducersResults[i];
		}

		check(NumProduced == NumConsumed);
		UE_LOGF(LogTemp, Display, "items queued %d", NumProduced);
	}

	TEST_CASE_NAMED(FClosableMpscQueueTest, "System::Core::Async::ClosableMpscQueueTest", "[.][ApplicationContextMask][EngineFilter][Perf]")
	{
		{
			TClosableMpscQueue<int> Q;
			Q.Close([](int) { check(false); });
		}
		{
			TClosableMpscQueue<int> Q;
			Q.Enqueue(1);
			bool Done = false;
			Q.Close([&Done](int Item) { check(!Done && Item == 1); Done = true; });
		}

		{
			TClosableMpscQueue<int> Q;
			Q.Enqueue(1);
			Q.Enqueue(2);
			int Step = 0;
			Q.Close(
				[&Step](int Item)
				{
					switch (Step)
					{
					case 0:
						check(Item == 1);
						++Step;
						break;
					case 1:
						check(Item == 2);
						++Step;
						break;
					default:
						check(false);
					}
				}
			);
		}

		//for (int i = 0; i != 1000000; ++i)
		{
			TClosableMpscQueue<int> Q;
			Q.Close([](int) { check(false); });
			Q.Close([](int) { check(false); });
		}

		//for (int i = 0; i != 1000000; ++i)
		{
			TClosableMpscQueue<int> Q;
			check(!Q.IsClosed());
			Q.Close([](int) { check(false); });
			check(Q.IsClosed());
		}

		UE_BENCHMARK(5, TestClosingEmptyQueue<10'000'000, TClosableMpscQueue<void*>>);
		UE_BENCHMARK(5, TestClosingEmptyQueue<10'000'000, TClosableMpscQueueOnTQueue<void*>>);
		UE_BENCHMARK(5, TestClosingEmptyQueue<10'000'000, TClosableMpscQueueOnTMpscQueue<void*>>);
		UE_BENCHMARK(5, TestClosingEmptyQueue<10'000'000, TClosableLockFreePointerListUnorderedSingleConsumer_Adapter<void>>);

		UE_BENCHMARK(5, TestCorrectness<20, TClosableMpscQueue<void*>>);
		UE_BENCHMARK(5, TestCorrectness<20, TClosableMpscQueueOnTQueue<void*>>);
		UE_BENCHMARK(5, TestCorrectness<20, TClosableMpscQueueOnTMpscQueue<void*>>);
		UE_BENCHMARK(5, TestCorrectness<20, TClosableLockFreePointerListUnorderedSingleConsumer_Adapter<void>>);

		UE_BENCHMARK(5, TestPerf<5'000'000, TClosableMpscQueue<void*>>);
		UE_BENCHMARK(5, TestPerf<5'000'000, TClosableMpscQueueOnTQueue<void*>>);
		UE_BENCHMARK(5, TestPerf<5'000'000, TClosableMpscQueueOnTMpscQueue<void*>>);
		UE_BENCHMARK(5, TestPerf<5'000'000, TClosableLockFreePointerListUnorderedSingleConsumer_Adapter<void>>);
	}
}

namespace ConsumeAllMpmcQueueTests
{
	// straightforward implementation over MPSC TQueue for comparison with TConsumeAllMpmcQueue
	template<typename T>
	class TConsumeAllMpmcQueueOnTQueue
	{
		using FQueue = TQueue<T, EQueueMode::Mpsc>;

	public:
		~TConsumeAllMpmcQueueOnTQueue()
		{
			checkf(SubscribingThreadsNum.load(std::memory_order_relaxed) == 0, TEXT("No other threads can use the object during its destruction"));
			delete Queue.load(std::memory_order_relaxed);
		}

		void ProduceItem(T Value)
		{
			// `acquire`/`release` to keep `Queue` usage inside the scope
			verify(SubscribingThreadsNum.fetch_add(1, std::memory_order_acquire) >= 0);
			Queue.load(std::memory_order_relaxed)->Enqueue(MoveTemp(Value));
			// `release` to "synchonise-with" other threads
			verify(SubscribingThreadsNum.fetch_sub(1, std::memory_order_release) >= 1);
		}

		template<typename F>
		void ConsumeAllFifo(const F& Consumer)
		{
			// `acquire` to make it "happen-before" reading `SubscribingThreadsNum`
			FQueue* LocalQueue = Queue.exchange(new FQueue, std::memory_order_acquire);

			T Value;
			bool bHasValue = false;
			// it's possible that a queue is exchanged and depleted after a producer has gotten the queue but before it has finished enqueuing. 
			// we use the following counter to make the consumer wait until all producers have finished enqueing to the current queue
			do
			{
				bHasValue = LocalQueue->Dequeue(Value);
				if (bHasValue)
				{
					Consumer(MoveTemp(Value));
				}
			} while (bHasValue || SubscribingThreadsNum.load(std::memory_order_relaxed) != 0);

			delete LocalQueue;
		}

	private:
		std::atomic<FQueue*> Queue{ new FQueue };
		std::atomic<uint32> SubscribingThreadsNum{ 0 };
	};

	// straightforward implementation over MPSC TQueue for comparison with TResettableMpscQueue
	template<typename T>
	class TConsumeAllMpmcQueueOnTMpscQueue
	{
	public:
		~TConsumeAllMpmcQueueOnTMpscQueue()
		{
			checkf(SubscribingThreadsNum.load(std::memory_order_relaxed) == 0, TEXT("No other threads can use the object during its destration"));
			delete Queue.load(std::memory_order_relaxed);
		}

		void ProduceItem(T Value)
		{
			// `acquire`/`release` to keep `Queue` usage inside the scope
			verify(SubscribingThreadsNum.fetch_add(1, std::memory_order_acquire) >= 0);
			Queue.load(std::memory_order_relaxed)->Enqueue(MoveTemp(Value));
			// `release` to "synchonise-with" other threads
			verify(SubscribingThreadsNum.fetch_sub(1, std::memory_order_release) >= 1);
		}

		template<typename F>
		void ConsumeAllFifo(const F& Consumer)
		{
			// `acquire` to make it "happen-before" reading `SubscribingThreadsNum`
			TMpscQueue<T>* LocalQueue = Queue.exchange(new TMpscQueue<T>, std::memory_order_acquire);

			TOptional<T> Value;
			// it's possible that a queue is exchanged and depleted after a producer has gotten the queue but before it has finished enqueuing. 
			// we use the following counter to make the consumer wait until all producers have finished enqueing to the current queue
			do
			{
				Value = LocalQueue->Dequeue();
				if (Value.IsSet())
				{
					Consumer(MoveTemp(Value.GetValue()));
				}
			} while (Value.IsSet() || SubscribingThreadsNum.load(std::memory_order_relaxed) != 0);

			delete LocalQueue;
		}

	private:
		std::atomic<TMpscQueue<T>*> Queue{ new TMpscQueue<T> };
		std::atomic<uint32> SubscribingThreadsNum{ 0 };
	};

	// wrapper over `TClosableLockFreePointerListUnorderedSingleConsumer`
	template<typename T>
	class TConsumeAllMpmcQueueOnTClosableLockFreePointerListUnorderedSingleConsumer
	{
		using FQueue = TClosableLockFreePointerListUnorderedSingleConsumer<T, 0 /* as was used in FGraphEvent */>;

	public:
		~TConsumeAllMpmcQueueOnTClosableLockFreePointerListUnorderedSingleConsumer()
		{
			checkf(SubscribingThreadsNum.load(std::memory_order_relaxed) == 0, TEXT("No other threads can use the object during its destration"));
			delete Queue.load(std::memory_order_relaxed);
		}

		void ProduceItem(T* Value)
		{
			// `acquire`/`release` to keep `Queue` usage inside the scope
			verify(SubscribingThreadsNum.fetch_add(1, std::memory_order_acquire) >= 0);
			ensureAlways(Queue.load(std::memory_order_relaxed)->PushIfNotClosed(MoveTemp(Value)));
			// `release` to "synchonise-with" other threads
			verify(SubscribingThreadsNum.fetch_sub(1, std::memory_order_release) >= 1);
		}

		template<typename F>
		void ConsumeAllFifo(const F& Consumer)
		{
			// `acquire` to make it "happen-before" reading `SubscribingThreadsNum`
			FQueue* LocalQueue = Queue.exchange(new FQueue, std::memory_order_acquire);

			// wait for all producers to finish enqueuing to this queue instance before closing it, because enqueuing to a closed queue would fail
			// `acquire` to make it "happen-before" dequeueing
			while (SubscribingThreadsNum.load(std::memory_order_acquire) != 0)
			{
				FPlatformProcess::Yield();
			}

			TArray<T*> Items;
			LocalQueue->PopAllAndClose(Items);
			for (T* Item : Items)
			{
				Consumer(MoveTemp(Item));
			}

			delete LocalQueue;
		}

	private:
		std::atomic<FQueue*> Queue{ new FQueue };
		std::atomic<uint32> SubscribingThreadsNum{ 0 };
	};

	// wrapper over `TClosableMpscQueue`
	template<typename T>
	class TConsumeAllMpmcQueueOnTClosableMpscQueue
	{
		using FQueue = TClosableMpscQueue<T>;

	public:
		~TConsumeAllMpmcQueueOnTClosableMpscQueue()
		{
			checkf(SubscribingThreadsNum.load(std::memory_order_relaxed) == 0, TEXT("No other threads can use the object during its destration"));
			delete Queue.load(std::memory_order_relaxed);
		}

		void ProduceItem(T Value)
		{
			// `acquire`/`release` to keep `Queue` usage inside the scope
			verify(SubscribingThreadsNum.fetch_add(1, std::memory_order_acquire) >= 0);
			ensureAlways(Queue.load(std::memory_order_relaxed)->Enqueue(MoveTemp(Value)));
			// `release` to "synchonise-with" other threads
			verify(SubscribingThreadsNum.fetch_sub(1, std::memory_order_release) >= 1);
		}

		template<typename F>
		void ConsumeAllFifo(const F& Consumer)
		{
			// `acquire` to make it "happen-before" reading `SubscribingThreadsNum`
			FQueue* LocalQueue = Queue.exchange(new FQueue, std::memory_order_acquire);

			// wait for all producers to finish enqueuing to this queue instance before closing it, because enqueuing to a closed queue would fail
			// `acquire` to make it "happen-before" dequeueing
			while (SubscribingThreadsNum.load(std::memory_order_acquire) != 0)
			{
				FPlatformProcess::Yield();
			}

			LocalQueue->Close(Consumer);

			delete LocalQueue;
		}

	private:
		std::atomic<FQueue*> Queue{ new FQueue };
		std::atomic<uint32> SubscribingThreadsNum{ 0 };
	};

	template<uint32 Num, typename QueueT>
	void TestConsumingEmptyQueue()
	{
		QueueT Q;
		for (uint32 i = 0; i != Num; ++i)
		{
			Q.ConsumeAllFifo([](void*) { checkNoEntry(); });
		}
	}

	template<uint64 Num, typename QueueT>
	void TestPerf()
	{
		// spawns producers num = workers num. Every producer pushes `Num` items into the queue. The consumer continuously
		// consumes queue until all produced items are consumed
		QueueT Queue;

		int32 ProducersNum = FPlatformMisc::NumberOfWorkerThreadsToSpawn();

		FGraphEventArray Producers;
		Producers.Reserve(ProducersNum);

		for (int32 i = 0; i != ProducersNum; ++i)
		{
			Producers.Add(FFunctionGraphTask::CreateAndDispatchWhenReady
			(
				[&Queue]
				{
					uint64 Total = 0;
					for (uint64 Product = 1; Product != Num + 1; ++Product)
					{
						Queue.ProduceItem((void*)(intptr_t)Product);
						Total += Product;
					}
					check(Total == Num * (Num + 1) / 2);
				}
			));
		}

		uint64 Produced = Num * (Num + 1) / 2 * ProducersNum;
		uint64 Consumed = 0;
		while (Consumed != Produced)
		{
			Queue.ConsumeAllFifo
			(
				[&Consumed](void* Value) 
				{ 
					uint64 IntValue = (uint64)(intptr_t)Value;
					Consumed += IntValue;
				}
			);
			checkf(Produced >= Consumed, TEXT("%llu - %llu"), Produced, Consumed);
		}

		checkf(Produced == Consumed, TEXT("%llu - %llu"), Produced, Consumed);

		FTaskGraphInterface::Get().WaitUntilTasksComplete(Producers);

		Queue.ConsumeAllFifo([](void*) { checkNoEntry(); }); // empty
	}

	TEST_CASE_NAMED(FConsumeAllMpmcQueueTest, "System::Core::Async::ConsumeAllMpmcQueueTest", "[.][ApplicationContextMask][EngineFilter][Perf]")
	{
		{
			TConsumeAllMpmcQueue<int> Q;
			Q.ProduceItem(1);
			Q.ProduceItem(2);
			Q.ConsumeAllLifo([](int) {});
			Q.ProduceItem(3);
			Q.ConsumeAllLifo([](int) {});
			Q.ConsumeAllLifo([](int) {});
			Q.ProduceItem(4); // to destroy non-empty queue
		}

		{
			TConsumeAllMpmcQueue<int> Q;
			Q.ConsumeAllLifo([](int) { check(false); });
		}
		{
			TConsumeAllMpmcQueue<int> Q;
			Q.ProduceItem(1);
			bool Done = false;
			Q.ConsumeAllLifo([&Done](int Item) { check(!Done && Item == 1); Done = true; });
		}

		{
			TConsumeAllMpmcQueue<int> Q;
			Q.ProduceItem(1);
			Q.ProduceItem(2);
			int Step = 0;
			Q.ConsumeAllFifo(
				[&Step](int Item)
				{
					switch (Step)
					{
					case 0:
						check(Item == 1);
						++Step;
						break;
					case 1:
						check(Item == 2);
						++Step;
						break;
					default:
						check(false);
					}
				}
			);
		}

		//for (int i = 0; i != 1000000; ++i)
		{
			TConsumeAllMpmcQueue<int> Q;
			Q.ConsumeAllLifo([](int) { check(false); });
			Q.ConsumeAllLifo([](int) { check(false); });
		}

		//for (int i = 0; i != 1000000; ++i)
		{
			int Res = 0;
			TConsumeAllMpmcQueue<int> Q;
			Q.ProduceItem(1);
			Q.ConsumeAllLifo([&Res](int i) { Res += i; });
			Q.ProduceItem(2);
			Q.ConsumeAllLifo([&Res](int i) { Res += i; });
			checkf(Res == 3, TEXT("%d"), Res);
		}

		{	// EnqueueAndReturnWasEmpty
			TConsumeAllMpmcQueue<int> Q;
			verify(Q.ProduceItem(1) == EConsumeAllMpmcQueueResult::WasEmpty);
			verify(Q.ProduceItem(2) == EConsumeAllMpmcQueueResult::HadItems);
			Q.ConsumeAllLifo([](int) {});
			verify(Q.ProduceItem(1) == EConsumeAllMpmcQueueResult::WasEmpty);
			verify(Q.ProduceItem(2) == EConsumeAllMpmcQueueResult::HadItems);
			Q.ConsumeAllLifo([](int) {});
			verify(Q.ProduceItem(1) == EConsumeAllMpmcQueueResult::WasEmpty);
			verify(Q.ProduceItem(2) == EConsumeAllMpmcQueueResult::HadItems);
		}

		UE_BENCHMARK(5, TestPerf<10'000, TConsumeAllMpmcQueue<void*>>);
		UE_BENCHMARK(5, TestPerf<10'000, TConsumeAllMpmcQueueOnTQueue<void*>>);
		UE_BENCHMARK(5, TestPerf<10'000, TConsumeAllMpmcQueueOnTMpscQueue<void*>>);
		UE_BENCHMARK(5, TestPerf<10'000, TConsumeAllMpmcQueueOnTClosableLockFreePointerListUnorderedSingleConsumer<void>>);
		UE_BENCHMARK(5, TestPerf<10'000, TConsumeAllMpmcQueueOnTClosableMpscQueue<void*>>);

		UE_BENCHMARK(5, TestConsumingEmptyQueue<10'000'000, TConsumeAllMpmcQueue<void*>>);
		UE_BENCHMARK(5, TestConsumingEmptyQueue<10'000'000, TConsumeAllMpmcQueueOnTQueue<void*>>);
		UE_BENCHMARK(5, TestConsumingEmptyQueue<10'000'000, TConsumeAllMpmcQueueOnTMpscQueue<void*>>);
		UE_BENCHMARK(5, TestConsumingEmptyQueue<10'000'000, TConsumeAllMpmcQueueOnTClosableLockFreePointerListUnorderedSingleConsumer<void>>);
		UE_BENCHMARK(5, TestConsumingEmptyQueue<10'000'000, TConsumeAllMpmcQueueOnTClosableMpscQueue<void*>>);
	}
}}

#endif // WITH_TESTS
