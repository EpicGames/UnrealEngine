// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/Deque.h"
#include "CoreMinimal.h"
#include "TokenTest.h"
#include "TestUtils.h"

#include "Tests/TestHarnessAdapter.h"

namespace Deque
{ 
namespace Test
{
static constexpr int32 DefaultCapacity = 4;

/**
 * Emplaces the parameter Count elements into the parameter queue and pops them one by one validating FIFO ordering.
 * This method also verifies ther correctness of: Last() and First()
 */
bool EmplaceLastPopFirst(TDeque<int32Token>& Deque);
bool EmplaceLastPopFirst(TDeque<int32Token>& Deque, int32 Count);
bool EmplaceFirstPopLast(TDeque<int32Token>& Deque);
bool EmplaceFirstPopLast(TDeque<int32Token>& Deque, int32 Count);

}  // namespace Test

//---------------------------------------------------------------------------------------------------------------------
// Deque::Test
//---------------------------------------------------------------------------------------------------------------------

bool Test::EmplaceLastPopFirst(TDeque<int32Token>& Deque)
{
	ensure(Deque.Max());
	return EmplaceLastPopFirst(Deque, Deque.Max());
}

bool Test::EmplaceLastPopFirst(TDeque<int32Token>& Deque, int32 Count)
{
	const int32 SeedValue = FMath::RandRange(1, 999);
	for (int32 i = 0; i < Count; ++i)
	{
		Deque.EmplaceLast(SeedValue + i);
		CHECK(Deque.Num() == i + 1);
		CHECK(Deque.First() == SeedValue);
		CHECK(Deque.Last() == SeedValue + i);
	}
	for (int32 i = 0; i < Count; ++i)
	{
		CHECK(Deque.First() == SeedValue + i);
		CHECK(Deque.Last() == SeedValue + Count - 1);
		Deque.PopFirst();
		CHECK(Deque.Num() == Count - (i + 1));
	}
	return true;
}

bool Test::EmplaceFirstPopLast(TDeque<int32Token>& Deque)
{
	ensure(Deque.Max());
	return EmplaceFirstPopLast(Deque, Deque.Max());
}

bool Test::EmplaceFirstPopLast(TDeque<int32Token>& Deque, int32 Count)
{
	const int32 SeedValue = FMath::RandRange(1, 999);
	for (int32 i = 0; i < Count; ++i)
	{
		Deque.EmplaceFirst(SeedValue + i);
		CHECK(Deque.Num() == i + 1);
		CHECK(Deque.First() == SeedValue + i);
		CHECK(Deque.Last() == SeedValue);
	}
	for (int32 i = 0; i < Count; ++i)
	{
		CHECK(Deque.First() == SeedValue + Count - 1);
		CHECK(Deque.Last() == SeedValue + i);
		Deque.PopLast();
		CHECK(Deque.Num() == Count - (i + 1));
	}
	return true;
}

//---------------------------------------------------------------------------------------------------------------------
// Unit tests
//---------------------------------------------------------------------------------------------------------------------

TEST_CASE_NAMED(FDequeReserveWithoutDataTest, "System::Core::Containers::TDeque::Reserve without data", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	TDeque<int32Token> Deque;
	CHECK(!Deque.Max());
	CHECK(!Deque.Num());
	CHECK(Deque.IsEmpty());
	Deque.Reserve(Test::DefaultCapacity);
	CHECK(Deque.Max() >= Test::DefaultCapacity);
	CHECK(!Deque.Num());
	CHECK(Deque.IsEmpty());
	CHECK(int32Token::EvenConstructionDestructionCalls(0));
}

TEST_CASE_NAMED(FDequeReserveEmplaceLastSingleElementTest, "System::Core::Containers::TDeque::Reserve EmplaceLast single element", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		Deque.Reserve(Test::DefaultCapacity);
		Deque.EmplaceLast(0);
		CHECK(Deque.Max() >= Test::DefaultCapacity);
		CHECK(Deque.Max() < Test::DefaultCapacity * 2);
		CHECK(Deque.Num() == 1);
		Deque.Reserve(Test::DefaultCapacity * 2);
		CHECK(Deque.Max() >= Test::DefaultCapacity * 2);
		CHECK(Deque.Num() == 1);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(1));
}

TEST_CASE_NAMED(FDequeResetTest, "System::Core::Containers::TDeque::Reset", "[SmokeFilter][Core][Containers][Deque]")
{
	SECTION("Default allocator")
	{
		int32Token::Reset();
		{
			TDeque<int32Token> Deque;
			Deque.Reset();	// Should be innocuous
			CHECK(!Deque.Max());
			CHECK(!Deque.Num());
			CHECK(Deque.IsEmpty());
			Deque.EmplaceLast(0);
			CHECK(Deque.Max());
			CHECK(Deque.Num() == 1);
			CHECK(!Deque.IsEmpty());
			Deque.Reset();
			CHECK(Deque.Max());
			CHECK(!Deque.Num());
			CHECK(Deque.IsEmpty());
		}
		CHECK(int32Token::EvenConstructionDestructionCalls(1));
	}

	SECTION("Inline Allocator")
	{
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> Deque;
		Deque.Reset();
		CHECK(Deque.Max() == Test::DefaultCapacity);
		Deque.EmplaceLast(0);
		CHECK(Deque.Num() == 1);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		Deque.Reset();
		CHECK(Deque.IsEmpty());
		CHECK(Deque.Max() == Test::DefaultCapacity);
	}

	SECTION("Fixed allocator")
	{
		TDeque<int32Token, TFixedAllocator<Test::DefaultCapacity>> Deque;
		Deque.Reset();
		CHECK(Deque.Max() == Test::DefaultCapacity);
		Deque.EmplaceLast(0);
		CHECK(Deque.Num() == 1);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		Deque.Reset();
		CHECK(Deque.IsEmpty());
		CHECK(Deque.Max() == Test::DefaultCapacity);
	}
}

TEST_CASE_NAMED(FDequeEmptyTest, "System::Core::Containers::TDeque::Empty", "[SmokeFilter][Core][Containers][Deque]")
{
	SECTION("Default allocator")
	{
		int32Token::Reset();
		{
			TDeque<int32Token> Deque;
			Deque.Empty();	// Should be innocuous
			CHECK(!Deque.Max());
			CHECK(!Deque.Num());
			CHECK(Deque.IsEmpty());
		}
		CHECK(int32Token::EvenConstructionDestructionCalls(0));
	}

	SECTION("Inline allocator")
	{
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> Deque;
		Deque.Empty();
		CHECK(Deque.Max() == Test::DefaultCapacity);
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
	}

	SECTION("Fixed allocator")
	{
		TDeque<int32Token, TFixedAllocator<Test::DefaultCapacity>> Deque;
		Deque.Empty();
		CHECK(Deque.Max() == Test::DefaultCapacity);
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
	}
}

TEST_CASE_NAMED(FDequeEmptyAfterSingleElementEmplaceLastTest, "System::Core::Containers::TDeque::Empty after single element EmplaceLast", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.EmplaceLast(0);
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
		Deque.Empty();
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(1));
}

TEST_CASE_NAMED(FDequeEmplaceLastSingleElementTest, "System::Core::Containers::TDeque::EmplaceLast single element", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
		Deque.EmplaceLast(0);
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(1));
}

TEST_CASE_NAMED(FDequeEmplaceLastRangeToCapacityTest, "System::Core::Containers::TDeque::EmplaceLast range to capacity", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		Deque.Reserve(Test::DefaultCapacity * 10);
		CHECK(Deque.Max() == Test::DefaultCapacity * 10);
		while (Deque.Num() < Deque.Max())
		{
			Deque.EmplaceLast();
		}
		CHECK(Deque.Max() == Deque.Num());
		CHECK(Deque.Max() == Test::DefaultCapacity * 10);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity * 10));
}

TEST_CASE_NAMED(FDequeEmplaceLastRangePastCapacityTest, "System::Core::Containers::TDeque::EmplaceLast range past capacity", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		while (Deque.Num() < Deque.Max())
		{
			Deque.EmplaceLast();
		}
		CHECK(Deque.Max() == Deque.Num());
		CHECK(Deque.Max() == Test::DefaultCapacity);
		Deque.EmplaceLast();
		CHECK(Deque.Max() > Deque.Num());
		CHECK(Deque.Max() > Test::DefaultCapacity);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity + 1));
}

TEST_CASE_NAMED(FDequeEmplaceFirstSingleElementTest, "System::Core::Containers::TDeque::EmplaceFirst single element", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
		Deque.EmplaceFirst(0);
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(1));
}

TEST_CASE_NAMED(FDequeEmplaceFirstRangeToCapacityTest, "System::Core::Containers::TDeque::EmplaceFirst range to capacity", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		Deque.Reserve(Test::DefaultCapacity * 10);
		CHECK(Deque.Max() == Test::DefaultCapacity * 10);
		while (Deque.Num() < Deque.Max())
		{
			Deque.EmplaceFirst();
		}
		CHECK(Deque.Max() == Deque.Num());
		CHECK(Deque.Max() == Test::DefaultCapacity * 10);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity * 10));
}

TEST_CASE_NAMED(FDequePushLastSingleElementTest, "System::Core::Containers::TDeque::PushLast single element (implicit move)", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
		Deque.PushLast(0);	// implicit conversion from temporary
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
	CHECK(int32Token::NumConstructorCalls() == 1);
	CHECK(int32Token::NumCopyConstructorCalls() == 0);
	CHECK(int32Token::NumMoveConstructorCalls() == 1);
}

TEST_CASE_NAMED(FDequePushLastSingleElementFromMoveTest, "System::Core::Containers::TDeque::PushLast single element from move", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
		int32Token TempToken;
		Deque.PushLast(MoveTemp(TempToken));
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
	CHECK(int32Token::NumConstructorCalls() == 1);
	CHECK(int32Token::NumCopyConstructorCalls() == 0);
	CHECK(int32Token::NumMoveConstructorCalls() == 1);
}

TEST_CASE_NAMED(FDequePushLastSingleElementFromCopyTest, "System::Core::Containers::TDeque::PushLast single element from copy", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
		int32Token TempToken;
		Deque.PushLast(TempToken);
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
	CHECK(int32Token::NumConstructorCalls() == 1);
	CHECK(int32Token::NumCopyConstructorCalls() == 1);
	CHECK(int32Token::NumMoveConstructorCalls() == 0);
}

TEST_CASE_NAMED(FDequePushFirstSingleElementImplicitMoveTest, "System::Core::Containers::TDeque::PushFirst single element (implicit move)", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
		Deque.PushFirst(0);	 // implicit conversion from temporary
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
	CHECK(int32Token::NumConstructorCalls() == 1);
	CHECK(int32Token::NumCopyConstructorCalls() == 0);
	CHECK(int32Token::NumMoveConstructorCalls() == 1);
}

TEST_CASE_NAMED(FDequePushFirstSingleElementFromMoveTest, "System::Core::Containers::TDeque::PushFirst single element from move", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
		int32Token TempToken;
		Deque.PushFirst(MoveTemp(TempToken));
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
	CHECK(int32Token::NumConstructorCalls() == 1);
	CHECK(int32Token::NumCopyConstructorCalls() == 0);
	CHECK(int32Token::NumMoveConstructorCalls() == 1);
}

TEST_CASE_NAMED(FDequePushFirstSingleElementFromCopyTest, "System::Core::Containers::TDeque::PushFirst single element from copy", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
		int32Token TempToken;
		Deque.PushFirst(TempToken);
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
	CHECK(int32Token::NumConstructorCalls() == 1);
	CHECK(int32Token::NumCopyConstructorCalls() == 1);
	CHECK(int32Token::NumMoveConstructorCalls() == 0);
}

void PopOne(TDeque<int32Token>& Deque)
{
	Deque.PopFirst();
}

TEST_CASE_NAMED(FDequeEmplaceLastPopFirstSingleElementTest, "System::Core::Containers::TDeque::EmplaceLast/PopFirst single element", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
		Deque.EmplaceLast(0);
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
		PopOne(Deque);
		CHECK(Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(1));
}

TEST_CASE_NAMED(FDequeEmplaceLastPopFirstSingleElementMultipleTest, "System::Core::Containers::TDeque::EmplaceLast/PopFirst single element multiple times causing head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity * 2; ++i)
		{
			CHECK(Test::EmplaceLastPopFirst(Deque, 1));  // Rotates head and tail
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity * 2));
}

TEST_CASE_NAMED(FDequeEmplaceLastPopFirstRangeNoWrapTest, "System::Core::Containers::TDeque::EmplaceLast/PopFirst range without head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Test::EmplaceLastPopFirst(Deque));
		CHECK(Deque.Max() == Test::DefaultCapacity);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity));
}

TEST_CASE_NAMED(FDequeEmplaceLastPopFirstRangeWithReallocationNoWrapTest, "System::Core::Containers::TDeque::EmplaceLast/PopFirst range with reallocation without head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		CHECK(Test::EmplaceLastPopFirst(Deque, Test::DefaultCapacity + 1));
		CHECK(Deque.Max() > Test::DefaultCapacity);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity + 1));
}

TEST_CASE_NAMED(FDequeEmplaceLastPopFirstRangeWithWrapTest, "System::Core::Containers::TDeque::EmplaceLast/PopFirst range with head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			CHECK(Test::EmplaceLastPopFirst(Deque, Test::DefaultCapacity - 1));  // Rotates head and tail
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity * (Test::DefaultCapacity - 1)));
}

TEST_CASE_NAMED(FDequeEmplaceFirstPopLastSingleElementTest, "System::Core::Containers::TDeque::EmplaceFirst/PopLast single element", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		CHECK(!Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
		Deque.EmplaceFirst(0);
		CHECK(Deque.Max());
		CHECK(Deque.Num() == 1);
		CHECK(!Deque.IsEmpty());
		Deque.PopLast();
		CHECK(Deque.Max());
		CHECK(!Deque.Num());
		CHECK(Deque.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(1));
}

TEST_CASE_NAMED(FDequeEmplaceFirstPopLastRangeTest, "System::Core::Containers::TDeque::EmplaceFirst/PopLast range", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Test::EmplaceFirstPopLast(Deque));
		CHECK(Deque.Max() == Test::DefaultCapacity);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity));
}

TEST_CASE_NAMED(FDequeEmplaceFirstPopLastRangeWithReallocationTest, "System::Core::Containers::TDeque::EmplaceFirst/PopLast range with reallocation", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		CHECK(Test::EmplaceFirstPopLast(Deque, Test::DefaultCapacity + 1));
		CHECK(Deque.Max() > Test::DefaultCapacity);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity + 1));
}

TEST_CASE_NAMED(FDequeTryPopFirstTest, "System::Core::Containers::TDeque::TryPopFirst", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			Deque.EmplaceLast(i);
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
		int32Token Result;
		int CheckValue = 0;
		while (Deque.TryPopFirst(Result))
		{
			CHECK(*Result == CheckValue++);
		}
		CHECK(CheckValue == Test::DefaultCapacity);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity + 1));  // + 1 for Result
}

TEST_CASE_NAMED(FDequeTryPopFirstWithReallocationTest, "System::Core::Containers::TDeque::TryPopFirst with reallocation", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			Deque.EmplaceLast(i);
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
		Deque.EmplaceLast(Test::DefaultCapacity);
		CHECK(Deque.Max() > Test::DefaultCapacity);

		int32Token Result;
		int32 CheckValue = 0;
		while (Deque.TryPopFirst(Result))
		{
			CHECK(*Result == CheckValue++);
		}
		CHECK(CheckValue == Test::DefaultCapacity + 1);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity + 2));
}

TEST_CASE_NAMED(FDequeTryPopLastTest, "System::Core::Containers::TDeque::TryPopLast", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			Deque.EmplaceFirst(i);
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
		int32Token Result;
		int CheckValue = 0;
		while (Deque.TryPopLast(Result))
		{
			CHECK(*Result == CheckValue++);
		}
		CHECK(CheckValue == Test::DefaultCapacity);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity + 1));  // + 1 for Result
}

TEST_CASE_NAMED(FDequeTryPopWithReallocationTest, "System::Core::Containers::TDeque::TryPopLast with reallocation", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			Deque.EmplaceFirst(i);
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
		Deque.EmplaceFirst(Test::DefaultCapacity);
		CHECK(Deque.Max() > Test::DefaultCapacity);

		int32Token Result;
		int32 CheckValue = 0;
		while (Deque.TryPopLast(Result))
		{
			CHECK(*Result == CheckValue++);
		}
		CHECK(CheckValue == Test::DefaultCapacity + 1);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity + 2));
}

TEST_CASE_NAMED(FDequeComparisonSimpleTest, "System::Core::Containers::TDeque::Comparison simple", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque, DestQueue;
		Deque.Reserve(Test::DefaultCapacity);
		DestQueue.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			Deque.EmplaceLast(i);
			DestQueue.EmplaceLast(i);
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
		CHECK(Deque == DestQueue);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity * 2));
}

TEST_CASE_NAMED(FDequeComparisonWithHeadTailWrapTest, "System::Core::Containers::TDeque::Comparison with head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque, DestQueue;
		Deque.Reserve(Test::DefaultCapacity);
		DestQueue.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			Deque.EmplaceLast(i);
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
		for (int32 Iteration = 0; Iteration < Test::DefaultCapacity; ++Iteration)
		{
			CHECK(Test::EmplaceLastPopFirst(DestQueue, 1));  // Rotates head and tail
			for (int32 i = 0; i < Test::DefaultCapacity; ++i)
			{
				DestQueue.EmplaceLast(i);
				CHECK(DestQueue.Max() == Test::DefaultCapacity);
			}
			CHECK(Deque == DestQueue);
			DestQueue.Reset();
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity * 6));
}

TEST_CASE_NAMED(FDequeCopySimpleTest, "System::Core::Containers::TDeque::Copy simple", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			Deque.EmplaceLast(i);
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
		TDeque<int32Token> DestQueue(Deque);
		CHECK(Deque == DestQueue);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity * 2));
}

TEST_CASE_NAMED(FDequeCopyWithHeadTailWrapTest, "System::Core::Containers::TDeque::Copy with head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 Iteration = 0; Iteration < Test::DefaultCapacity; ++Iteration)
		{
			CHECK(Test::EmplaceLastPopFirst(Deque, 1));  // Rotates head and tail
			for (int32 i = 0; i < Test::DefaultCapacity; ++i)
			{
				Deque.EmplaceLast(i);
				CHECK(Deque.Max() == Test::DefaultCapacity);
			}
			TDeque<int32Token> DestQueue(Deque);
			CHECK(Deque == DestQueue);
			Deque.Reset();
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity * 9));
}

TEST_CASE_NAMED(FDequeCopyVariableSizeWithHeadTailWrapTest, "System::Core::Containers::TDeque::Copy variable size with head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 Iteration = 0; Iteration < Test::DefaultCapacity; ++Iteration)
		{
			for (int32 Size = 1; Size <= Test::DefaultCapacity; ++Size)
			{
				CHECK(Test::EmplaceLastPopFirst(Deque, 1));  // Rotates head and tail
				const int32 SeedValue = FMath::RandRange(1, 999);
				for (int32 i = 0; i < Size; ++i)
				{
					Deque.EmplaceLast(SeedValue + i);
					CHECK(Deque.Max() == Test::DefaultCapacity);
				}
				TDeque<int32Token> DestQueue(Deque);
				CHECK(Deque == DestQueue);
				CHECK(DestQueue.Max() <= Test::DefaultCapacity);
				Deque.Reset();
			}
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls());
}

TEST_CASE_NAMED(FDequeCopyFullQueueAndPopFirstEmplaceLastTest, "System::Core::Containers::TDeque::Copy with source deque at capacity and PopFirst/EmplaceLast on copied deque", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 Value = 0; Value < Test::DefaultCapacity; ++Value)
		{
			Deque.EmplaceLast(Value);
		}
		CHECK(Deque.Num() == Test::DefaultCapacity);

		TDeque<int32Token> DestQueue(Deque);
		CHECK(Deque == DestQueue);
		CHECK(DestQueue.Max() == Test::DefaultCapacity);

		DestQueue.PopFirst();
		DestQueue.EmplaceLast(Test::DefaultCapacity);
		int32 CheckValue = 1;
		for (const auto& Value : DestQueue)
		{
			CHECK(Value == CheckValue++);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls());
}

TEST_CASE_NAMED(FDequeCopyWithInlineAllocator, "System::Core::Containers::TDeque Copy with inline allocator", "[SmokeFilter][Core][Containers][Deque]")
{
	SECTION("Copy empty TDeque")
	{
		int32Token::Reset();
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> Deque;
		CHECK(Deque.Max() == Test::DefaultCapacity);
		CHECK(Deque.Num() == 0);
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> DequeCopy(Deque);
		CHECK_MESSAGE("Expected copy to retain inline capacity.", DequeCopy.Max() == Test::DefaultCapacity);
	}

	SECTION("Copy TDeque with 1 element")
	{
		int32Token::Reset();
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> Deque;
		Deque.PushLast(1);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		CHECK(Deque.Num() == 1);
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> DequeCopy(Deque);
		CHECK_MESSAGE("Expected copy to retain inline capacity.", DequeCopy.Max() == Test::DefaultCapacity);
		CHECK(DequeCopy.Num() == 1);
	}

	SECTION("Copy Full TDeque")
	{
		int32Token::Reset();
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> Deque{ 0, 1, 2, 3 };
		CHECK(Deque.Max() == Test::DefaultCapacity);
		CHECK(Deque.Num() == 4);
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> DequeCopy(Deque);
		CHECK_MESSAGE("Expected copy to retain inline capacity.", DequeCopy.Max() == Test::DefaultCapacity);
		CHECK(DequeCopy.Num() == 4);
	}

	SECTION("Copy TDeque that exceeds inline allocation")
	{
		int32Token::Reset();
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> Deque;
		const int32 ExpectedMax = (Test::DefaultCapacity + 2);
		for (int32 i = 0; i < ExpectedMax; ++i)
		{
			Deque.PushLast(i);
		}
		CHECK(Deque.Num() == ExpectedMax);
		CHECK(Deque.Max() >= ExpectedMax); // >= as default storage may grow by more than what we're expanding by.

		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> DequeCopy(Deque);
		CHECK(DequeCopy.Num() == ExpectedMax);
		CHECK(DequeCopy.Max() >= ExpectedMax);
	}
}

TEST_CASE_NAMED(FDequeMoveSimpleTest, "System::Core::Containers::TDeque::Move simple", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			Deque.EmplaceLast(i);
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
		TDeque<int32Token> DestQueue(MoveTemp(Deque));
		CHECK(Deque.IsEmpty());
		int32Token Result;
		int CheckValue = 0;
		while (DestQueue.TryPopFirst(Result))
		{
			CHECK(*Result == CheckValue++);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity + 1));
}

TEST_CASE_NAMED(FDequeMoveWithHEadTailWrapTest, "System::Core::Containers::TDeque::Move with head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		for (int32 Iteration = 0; Iteration < Test::DefaultCapacity; ++Iteration)
		{
			Deque.Reserve(Test::DefaultCapacity);
			CHECK(Deque.Max() == Test::DefaultCapacity);
			CHECK(Test::EmplaceLastPopFirst(Deque, 1));  // Rotates head and tail
			for (int32 i = 0; i < Test::DefaultCapacity; ++i)
			{
				Deque.EmplaceLast(i);
				CHECK(Deque.Max() == Test::DefaultCapacity);
			}
			TDeque<int32Token> DestQueue(MoveTemp(Deque));
			CHECK(Deque.IsEmpty());
			int32Token Result;
			int32 CheckValue = 0;
			while (DestQueue.TryPopFirst(Result))
			{
				CHECK(*Result == CheckValue++);
			}
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity * 6));
}

TEST_CASE_NAMED(FDequeMoveVariableSizeWithHeadTailWrapTest, "System::Core::Containers::TDeque::Move variable size with head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		for (int32 Iteration = 0; Iteration < Test::DefaultCapacity; ++Iteration)
		{
			for (int32 Size = 1; Size <= Test::DefaultCapacity; ++Size)
			{
				Deque.Reserve(Test::DefaultCapacity);
				CHECK(Deque.Max() == Test::DefaultCapacity);
				CHECK(Test::EmplaceLastPopFirst(Deque, 1));  // Rotates head and tail
				const int32 SeedValue = FMath::RandRange(1, 999);
				for (int32 i = 0; i < Size; ++i)
				{
					Deque.EmplaceLast(SeedValue + i);
					CHECK(Deque.Max() == Test::DefaultCapacity);
				}
				TDeque<int32Token> DestQueue(MoveTemp(Deque));
				CHECK(Deque.IsEmpty());
				int32Token Result;
				int32 CheckValue = SeedValue;
				while (DestQueue.TryPopFirst(Result))
				{
					CHECK(*Result == CheckValue++);
				}
			}
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls());
}

TEST_CASE_NAMED(FDequeMoveWithInlineAllocator, "System::Core::Containers::TDeque Move with inline allocator", "[SmokeFilter][Core][Containers][Deque]")
{
	SECTION("Move empty TDeque")
	{
		int32Token::Reset();
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> Deque;
		CHECK(Deque.Max() == Test::DefaultCapacity);
		CHECK(Deque.Num() == 0);
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> DequeCopy(MoveTemp(Deque));
		CHECK_MESSAGE("Expected copy to retain inline capacity.", DequeCopy.Max() == Test::DefaultCapacity);
	}

	SECTION("Move TDeque with 1 element")
	{
		int32Token::Reset();
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> Deque;
		Deque.PushLast(1);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		CHECK(Deque.Num() == 1);
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> DequeCopy(MoveTemp(Deque));
		CHECK_MESSAGE("Expected copy to retain inline capacity.", DequeCopy.Max() == Test::DefaultCapacity);
		CHECK(DequeCopy.Num() == 1);
	}

	SECTION("Move TDeque that exceeds inline allocation")
	{
		int32Token::Reset();
		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> Deque;
		const int32 ExpectedMax = (Test::DefaultCapacity + 2);
		for (int32 i = 0; i < ExpectedMax; ++i)
		{
			Deque.PushLast(i);
		}
		CHECK(Deque.Num() == ExpectedMax);
		CHECK(Deque.Max() >= ExpectedMax); // >= as default storage may grow by more than what we're expanding by.

		TDeque<int32Token, TInlineAllocator<Test::DefaultCapacity>> DequeCopy(MoveTemp(Deque));
		CHECK(DequeCopy.Num() == ExpectedMax);
		CHECK(DequeCopy.Max() >= ExpectedMax);
	}
}

TEST_CASE_NAMED(FDequeIterationWIthoutWrapTest, "System::Core::Containers::TDeque::Iteration without head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			Deque.EmplaceLast(i);
			CHECK(Deque.Max() == Test::DefaultCapacity);
		}
		for (int32 i = 0; i < Test::DefaultCapacity; ++i)
		{
			CHECK(Deque[i] == i);
		}
		int32 CheckValue = 0;
		for (const auto& Value : Deque)
		{
			CHECK(Value == CheckValue++);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity));
}

TEST_CASE_NAMED(FDequeIterationWithWrapTest, "System::Core::Containers::TDeque::Iteration with head/tail wrap around", "[SmokeFilter][Core][Containers][Deque]")
{
	int32Token::Reset();
	{
		TDeque<int32Token> Deque;
		Deque.Reserve(Test::DefaultCapacity);
		CHECK(Deque.Max() == Test::DefaultCapacity);
		for (int32 Iteration = 0; Iteration < Test::DefaultCapacity; ++Iteration)
		{
			CHECK(Test::EmplaceLastPopFirst(Deque, 1));  // Rotates head and tail
			for (int32 i = 0; i < Test::DefaultCapacity; ++i)
			{
				Deque.EmplaceLast(i);
				CHECK(Deque.Max() == Test::DefaultCapacity);
			}
			for (int32 i = 0; i < Test::DefaultCapacity; ++i)
			{
				CHECK(Deque[i] == i);
			}
			int32 CheckValue = 0;
			for (const auto& Value : Deque)
			{
				CHECK(Value == CheckValue++);
			}
			Deque.Reset();
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(Test::DefaultCapacity * (Test::DefaultCapacity + 1)));
}

TEST_CASE_NAMED(FDequeIteratorArithmeticTest, "System::Core::Containers::TDeque::Iterator arithmetic", "[SmokeFilter][Core][Containers][Deque]")
{
	TDeque<int32Token> Deque;
	Deque.Reserve(Test::DefaultCapacity);
	CHECK(Deque.Max() == Test::DefaultCapacity);
	for (int32 Iteration = 0; Iteration < Test::DefaultCapacity; ++Iteration)
	{
		CHECK(Test::EmplaceLastPopFirst(Deque, 1));  // Rotates head and tail
		Deque.EmplaceLast(13);
		Deque.EmplaceLast(42);
		Deque.EmplaceLast(19);

		auto It = Deque.begin();
		CHECK(**It == 13);
		CHECK(*It == int32Token(13));
		CHECK(It->Value == 13);
		auto It2 = It++;
		CHECK(It2 != It);
		CHECK(It2->Value == 13);
		CHECK(**It == 42);
		CHECK(*It == int32Token(42));
		CHECK(It->Value == 42);

		Deque.Reset();
	}
}

TEST_CASE_NAMED(FDequeConstructFromStdInitializerListTest, "System::Core::Containers::TDeque::Construct from std initializer_list", "[SmokeFilter][Core][Containers][Deque]")
{
	SECTION("Default allocator")
	{
		int32Token::Reset();
		{
			TDeque<int32Token> Deque({ 0, 1, 2, 3, 4, 5 });
			int32Token Result;
			int32 CheckValue = 0;
			while (Deque.TryPopFirst(Result))
			{
				CHECK(*Result == CheckValue++);
			}
			CHECK(CheckValue == 6);
		}
		CHECK(int32Token::EvenConstructionDestructionCalls(6 * 2 + 1));
	}

	SECTION("Inline allocator without growing")
	{
		TDeque<int32Token, TInlineAllocator<8>> Deque({ 0, 1, 2, 3, 4, 5 });
		CHECK(Deque.Num() == 6);
		CHECK(Deque.Max() == 8);
	}

	SECTION("Inline allocator with growing")
	{
		TDeque<int32Token, TInlineAllocator<4>> Deque({ 0, 1, 2, 3, 4, 5 });
		CHECK(Deque.Num() == 6);
		CHECK(Deque.Max() >= 6); // Growth amount can vary per platform
	}
}

TEST_CASE_NAMED(FDequeConstructFromEmptyStdInitializerListTest, "System::Core::Containers::TDeque::Construct from empty std initializer_list", "[SmokeFilter][Core][Containers][Deque]")
{
	SECTION("Default allocator")
	{
		int32Token::Reset();
		{
			TDeque<int32Token> Deque({});
			CHECK(Deque.IsEmpty());
		}
		CHECK(int32Token::EvenConstructionDestructionCalls(0));
	}

	SECTION("Inline allocator")
	{
		TDeque<int32Token, TInlineAllocator<8>> Deque({});
		CHECK(Deque.IsEmpty());
		CHECK(Deque.Max() == 8);
	}
}

TEST_CASE_NAMED(FDequeAssignFromStdInitializerListTest, "System::Core::Containers::TDeque::Assign from std initializer_list", "[SmokeFilter][Core][Containers][Deque]")
{
	SECTION("Default allocator")
	{
		int32Token::Reset();
		{
			TDeque<int32Token> Deque;
			Deque.EmplaceLast(0);
			Deque = { 0, 1, 2, 3, 4, 5 };
			int32Token Result;
			int32 CheckValue = 0;
			while (Deque.TryPopFirst(Result))
			{
				CHECK(*Result == CheckValue++);
			}
			CHECK(CheckValue == 6);
		}
		CHECK(int32Token::EvenConstructionDestructionCalls(6 * 2 + 2));
	}

	SECTION("Inline allocation without growing")
	{
		TDeque<int32Token, TInlineAllocator<8>> Deque;
		Deque = { 0, 1, 2, 3, 4, 5 };
		CHECK(Deque.Num() == 6);
		CHECK(Deque.Max() == 8);
	}

	SECTION("Inline allocation with growing")
	{
		TDeque<int32Token, TInlineAllocator<4>> Deque;
		Deque = { 0, 1, 2, 3, 4, 5 };
		CHECK(Deque.Num() == 6);
		CHECK(Deque.Max() >= 6); // Growth amount can vary per platform
	}
}

TEST_CASE_NAMED(FDequeCopyAssignTest, "System::Core::Containers::TDeque::Assign copy", "[SmokeFilter][Core][Containers][Deque]")
{
	SECTION("Default allocator")
	{
		TDeque<int32Token> Deque;
		TDeque<int32Token> DequeCopy{ 0, 1, 2, 3, 4, 5 };
		Deque = DequeCopy;
		CHECK(Deque.Num() == 6);
		CHECK(Deque.Num() == DequeCopy.Num());
		CHECK(Deque.Max() == DequeCopy.Max());
	}

	SECTION("Inline allocation without growing")
	{
		TDeque<int32Token, TInlineAllocator<8>> Deque;
		TDeque<int32Token, TInlineAllocator<8>> DequeCopy{ 0, 1, 2, 3, 4, 5 };
		Deque = DequeCopy;
		CHECK(Deque.Num() == 6);
		CHECK(Deque.Max() == 8);
		CHECK(Deque.Num() == DequeCopy.Num());
		CHECK(Deque.Max() == DequeCopy.Max());
	}

	SECTION("Inline allocation without growing 2")
	{
		TDeque<int32Token, TInlineAllocator<8>> DequeCopy{ 0, 1, 2, 3, 4, 5 };
		TDeque<int32Token, TInlineAllocator<8>> Deque = DequeCopy;
		CHECK(Deque.Num() == 6);
		CHECK(Deque.Max() == 8);
		CHECK(Deque.Num() == DequeCopy.Num());
		CHECK(Deque.Max() == DequeCopy.Max());
	}

	SECTION("Inline allocation with growing")
	{
		TDeque<int32Token, TInlineAllocator<4>> Deque;
		TDeque<int32Token, TInlineAllocator<4>> DequeCopy{ 0, 1, 2, 3, 4, 5 };
		Deque = DequeCopy;
		CHECK(Deque.Num() == DequeCopy.Num());
		CHECK(Deque.Max() == DequeCopy.Max());
	}
}

}  // namespace Deque

#endif
