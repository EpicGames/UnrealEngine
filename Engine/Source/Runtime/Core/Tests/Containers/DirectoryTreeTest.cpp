// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/DirectoryTree.h"

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FDirectoryTreeContainsChildPathTests, "System::Core::Containers::DirectoryTree::ContainsChildPaths", "[Core][Containers][DirectoryTree]")
{
	TDirectoryTree<int32> Tree;

	Tree.FindOrAdd(TEXTVIEW("/Game/Dir1"));
	Tree.FindOrAdd(TEXTVIEW("/Game/Dir2"));
	Tree.FindOrAdd(TEXTVIEW("/Game/Dir2/Grandchild"));
	Tree.FindOrAdd(TEXTVIEW("/Plugin1/Path1"));
	Tree.FindOrAdd(TEXTVIEW("/Plugin1/Path2"));

	CHECK(Tree.ContainsChildPaths(TEXTVIEW("/")));
	CHECK(Tree.ContainsChildPaths(TEXTVIEW("/Game")));
	CHECK(Tree.ContainsChildPaths(TEXTVIEW("/Game/Dir2")));
	CHECK(Tree.ContainsChildPaths(TEXTVIEW("/Plugin1")));

	CHECK_FALSE(Tree.ContainsChildPaths(TEXTVIEW("/Game/Dir1")));
	CHECK_FALSE(Tree.ContainsChildPaths(TEXTVIEW("/Game/Dir2/Grandchild")));
	CHECK_FALSE(Tree.ContainsChildPaths(TEXTVIEW("/Plugin1/Path1")));
	CHECK_FALSE(Tree.ContainsChildPaths(TEXTVIEW("/Plugin1/Path2")));
}

TEST_CASE_NAMED(FDirectoryTreeAPITests, "System::Core::Containers::DirectoryTree::API", "[Core][Containers][DirectoryTree]")
{
	constexpr int32 NumPathTypes = 5;
	constexpr int32 NumPaths = 9;
	FStringView PathsByTypeAndIndex[NumPathTypes][NumPaths] =
	{
		{
			TEXTVIEW("/Game/Dir2"),
			TEXTVIEW("/Game/Path1"),
			TEXTVIEW("/Game/Dir2/Path2"),
			TEXTVIEW("/Plugin1/Path1"),
			TEXTVIEW("/Plugin1/Dir2/Path2"),
			TEXTVIEW("/Engine/Path1"),
			TEXTVIEW("/Plugin2/Path1"),
			// Make sure we handle suffixes of an existing string with a leading value that sorts after /
			TEXTVIEW("/Game/Foo/Leaf"),
			TEXTVIEW("/Game/Foo-Bar/Leaf"),
		},
		{
			TEXTVIEW("d:\\root\\Project\\Content\\Dir2"),
			TEXTVIEW("d:\\root\\Project\\Content\\Path1.uasset"),
			TEXTVIEW("d:\\root\\Project\\Content\\Dir2\\Path2.uasset"),
			TEXTVIEW("d:\\root\\Project\\Plugins\\Plugin1\\Content\\Path1.uasset"),
			TEXTVIEW("d:\\root\\Project\\Plugins\\Plugin1\\Content\\Dir2\\Path2.uasset"),
			TEXTVIEW("d:\\root\\Engine\\Content\\Path1.uasset"),
			TEXTVIEW("e:\\root\\Project\\Plugins\\Plugin2\\Content\\Path1.uasset"),
			TEXTVIEW("d:\\root\\Project\\Content\\Foo\\Leaf"),
			TEXTVIEW("d:\\root\\Project\\Content\\Foo-Bar\\Leaf"),
		},
		{
			TEXTVIEW("d:/root/Project/Content/Dir2"),
			TEXTVIEW("d:/root/Project/Content/Path1.uasset"),
			TEXTVIEW("d:/root/Project/Content/Dir2/Path2.uasset"),
			TEXTVIEW("d:/root/Project/Plugins/Plugin1/Content/Path1.uasset"),
			TEXTVIEW("d:/root/Project/Plugins/Plugin1/Content/Dir2/Path2.uasset"),
			TEXTVIEW("d:/root/Engine/Content/Path1.uasset"),
			TEXTVIEW("e:/root/Project/Plugins/Plugin2/Content/Path1.uasset"),
			TEXTVIEW("d:/root/Project/Content/Foo/Leaf"),
			TEXTVIEW("d:/root/Project/Content/Foo-Bar/Leaf"),
		},
		{
			TEXTVIEW("..\\..\\..\\Project\\Content\\Dir2"),
			TEXTVIEW("..\\..\\..\\Project\\Content\\Path1.uasset"),
			TEXTVIEW("..\\..\\..\\Project\\Content\\Dir2\\Path2.uasset"),
			TEXTVIEW("..\\..\\..\\Project\\Plugins\\Plugin1\\Content\\Path1.uasset"),
			TEXTVIEW("..\\..\\..\\Project\\Plugins\\Plugin1\\Content\\Dir2\\Path2.uasset"),
			TEXTVIEW("..\\..\\..\\Engine\\Content\\Path1.uasset"),
			TEXTVIEW("e:\\root\\Project\\Plugins\\Plugin2\\Content\\Path1.uasset"),
			TEXTVIEW("..\\..\\..\\Project\\Content\\Foo\\Leaf"),
			TEXTVIEW("..\\..\\..\\Project\\Content\\Foo-Bar\\Leaf"),
		},
		{
			TEXTVIEW("../../../Project/Content/Dir2"),
			TEXTVIEW("../../../Project/Content/Path1.uasset"),
			TEXTVIEW("../../../Project/Content/Dir2/Path2.uasset"),
			TEXTVIEW("../../../Project/Plugins/Plugin1/Content/Path1.uasset"),
			TEXTVIEW("../../../Project/Plugins/Plugin1/Content/Dir2/Path2.uasset"),
			TEXTVIEW("../../../Engine/Content/Path1.uasset"),
			TEXTVIEW("e:/root/Project/Plugins/Plugin2/Content/Path1.uasset"),
			TEXTVIEW("../../../Project/Content/Foo/Leaf"),
			TEXTVIEW("../../../Project/Content/Foo-Bar/Leaf"),
		},
	};
	// PathSub0SubPath[i] provides a sub path of PathsByTypeAndIndex[i][0]
	FStringView Path0SubPath[NumPathTypes] =
	{
		TEXTVIEW("/Game/Dir2/Sub"),
		TEXTVIEW("d:\\root\\Project\\Content\\Dir2\\Sub"),
		TEXTVIEW("d:/root/Project/Content/Dir2/Sub"),
		TEXTVIEW("..\\..\\..\\Project\\Content\\Dir2"),
		TEXTVIEW("../../../Project/Content/Dir2/Sub"),
	};
	int32 ValueByIndex[NumPaths] = { 1,2,3,4,5,6,7,8,9 };
	FStringView NonPathsByTypeAndIndex0[] =
		{
			TEXTVIEW(""),
			TEXTVIEW("/"),
			TEXTVIEW("/Game"),
			TEXTVIEW("/Game/"),
			TEXTVIEW("/Plugin1"),
			TEXTVIEW("/Plugin1/"),
			TEXTVIEW("/Plugin1/Dir2"),
			TEXTVIEW("/Plugin1/Dir2/"),
			TEXTVIEW("/Engine"),
			TEXTVIEW("/Engine/"),
		};
	FStringView NonPathsByTypeAndIndex1[] =
		{
			TEXTVIEW(""),
			TEXTVIEW("d:\\"),
			TEXTVIEW("d:\\root1"),
			TEXTVIEW("d:\\root1\\"),
			TEXTVIEW("d:\\root1\\Project"),
			TEXTVIEW("d:\\root1\\Project\\"),
			TEXTVIEW("d:\\root1\\Project\\Content"),
			TEXTVIEW("d:\\root1\\Project\\Plugins\\"),
			TEXTVIEW("d:\\root1\\Project\\Plugins\\Content"),
			TEXTVIEW("d:\\root1\\Project\\Plugins\\Content\\"),
			TEXTVIEW("d:\\root1\\Project\\Plugins\\Content\\Plugin1"),
			TEXTVIEW("d:\\root1\\Project\\Plugins\\Content\\Plugin1\\"),
			TEXTVIEW("d:\\root1\\Project\\Plugins\\Content\\Plugin1\\Dir2"),
			TEXTVIEW("d:\\root1\\Project\\Plugins\\Content\\Plugin1\\Dir2\\"),
			TEXTVIEW("d:\\root1\\Engine"),
			TEXTVIEW("d:\\root1\\Engine\\"),
			TEXTVIEW("d:\\root1\\Engine\\Content"),
			TEXTVIEW("d:\\root1\\Engine\\Content\\"),
		};
	FStringView NonPathsByTypeAndIndex2[] =
		{
			TEXTVIEW(""),
			TEXTVIEW("d:/"),
			TEXTVIEW("d:/root1"),
			TEXTVIEW("d:/root1/Project"),
			TEXTVIEW("d:/root1/Project/Content"),
			TEXTVIEW("d:/root1/Project/Plugins/Content"),
			TEXTVIEW("d:/root1/Project/Plugins/Content/Plugin1"),
			TEXTVIEW("d:/root1/Project/Plugins/Content/Plugin1/Dir2"),
			TEXTVIEW("d:/root1/Engine"),
			TEXTVIEW("d:/root1/Engine/Content"),
		};
	FStringView NonPathsByTypeAndIndex3[] =
		{
			TEXTVIEW(""),
			TEXTVIEW("\\"),
			TEXTVIEW(".."),
			TEXTVIEW("..\\"),
			TEXTVIEW("..\\.."),
			TEXTVIEW("..\\..\\"),
			TEXTVIEW("..\\..\\.."),
			TEXTVIEW("..\\..\\..\\"),
			TEXTVIEW("..\\..\\..\\Project"),
			TEXTVIEW("..\\..\\..\\Project\\Content"),
			TEXTVIEW("..\\..\\..\\Project\\Plugins\\Content"),
			TEXTVIEW("..\\..\\..\\Project\\Plugins\\Content\\Plugin1"),
			TEXTVIEW("..\\..\\..\\Project\\Plugins\\Content\\Plugin1\\Dir2"),
			TEXTVIEW("..\\..\\..\\Engine"),
			TEXTVIEW("..\\..\\..\\Engine\\Content"),
		};
	FStringView NonPathsByTypeAndIndex4[] =
		{
			TEXTVIEW(""),
			TEXTVIEW("/"),
			TEXTVIEW(".."),
			TEXTVIEW("../.."),
			TEXTVIEW("../../.."),
			TEXTVIEW("../../../Project"),
			TEXTVIEW("../../../Project/Content"),
			TEXTVIEW("../../../Project/Plugins/Content"),
			TEXTVIEW("../../../Project/Plugins/Content/Plugin1"),
			TEXTVIEW("../../../Project/Plugins/Content/Plugin1/Dir2"),
			TEXTVIEW("../../../Engine"),
			TEXTVIEW("../../../Engine/Content"),
		};
	TConstArrayView<FStringView> NonPathsByTypeAndIndex[NumPathTypes] =
	{
		NonPathsByTypeAndIndex0, NonPathsByTypeAndIndex1, NonPathsByTypeAndIndex2, NonPathsByTypeAndIndex3,
		NonPathsByTypeAndIndex4
	};

	constexpr int32 NumPermutations = 2;
	int32 Permutations[NumPermutations][NumPaths] =
	{
		{ 0, 1, 2, 3, 4, 5, 6, 7, 8 },
		{ 8, 7, 6, 5, 4, 3, 2, 1, 0 },
	};

	auto GetDirTreeTestName = [](FStringView InTestName, int32 PathType, int32 Permutation, int32 EditPermutationIndex,
		int32 OtherPermutationIndex)
	{
		return FString::Printf(TEXT("%.*s(%d, %d, %d, %d)"),
			InTestName.Len(), InTestName.GetData(),
			PathType, Permutation, EditPermutationIndex, OtherPermutationIndex);
	};

	TArray<const TCHAR*> ScratchA;
	TArray<const TCHAR*> ScratchB;
	auto UnorderedEquals = [&ScratchA, &ScratchB](const TArray<FString>& A, TConstArrayView<const TCHAR*> B)
		{
			if (A.Num() != B.Num())
			{
				return false;
			}
			ScratchA.Reset(A.Num());
			ScratchB.Reset(B.Num());
			for (const FString& AStr : A)
			{
				ScratchA.Add(*AStr);
			}
			for (const TCHAR* BStr : B)
			{
				ScratchB.Add(BStr);
			}
			Algo::Sort(ScratchA, [](const TCHAR* StrA, const TCHAR* StrB)
				{
					return FCString::Stricmp(StrA, StrB) < 0;
				});
			Algo::Sort(ScratchB, [](const TCHAR* StrA, const TCHAR* StrB)
				{
					return FCString::Stricmp(StrA, StrB) < 0;
				});
			for (int32 Index = 0; Index < A.Num(); ++Index)
			{
				if (FCString::Stricmp(ScratchA[Index], ScratchB[Index]) != 0)
				{
					return false;
				}
			}
			return true;
		};

	for (int32 PathType = 0; PathType < NumPathTypes; ++PathType)
	{
		int32 NumNonPaths = NonPathsByTypeAndIndex[PathType].Num();
		for (int32 Permutation = 0; Permutation < NumPermutations; ++Permutation)
		{
			TDirectoryTree<int32> Tree;

			// Add all the Paths in the given order and make Contains assertions after each addition
			for (int32 AddPathPermutationIndex = 0; AddPathPermutationIndex < NumPaths; ++AddPathPermutationIndex)
			{
				int32 AddPathIndex = Permutations[Permutation][AddPathPermutationIndex];
				FStringView AddPath = PathsByTypeAndIndex[PathType][AddPathIndex];

				// Add the path
				Tree.FindOrAdd(AddPath) = ValueByIndex[AddPathIndex];

				if (Tree.Num() != AddPathPermutationIndex+1)
				{
					ADD_ERROR(GetDirTreeTestName(TEXTVIEW("TreeNum has expected value"), PathType, Permutation,
						AddPathPermutationIndex, 0));
				}

				// Assert all paths up to and including this one are included
				for (int32 OtherPermutationIndex = 0; OtherPermutationIndex <= AddPathPermutationIndex;
					++OtherPermutationIndex)
				{
					int32 OtherIndex = Permutations[Permutation][OtherPermutationIndex];
					FStringView OtherPath = PathsByTypeAndIndex[PathType][OtherIndex];
					int32 OtherValue = ValueByIndex[OtherIndex];
					int32* ExistingValue = Tree.Find(OtherPath);
					if (!ExistingValue)
					{
						ADD_ERROR(GetDirTreeTestName(TEXTVIEW("OtherInList"), PathType, Permutation,
							AddPathPermutationIndex, OtherPermutationIndex));
					}
					else
					{
						if (*ExistingValue != OtherValue)
						{
							ADD_ERROR(GetDirTreeTestName(TEXTVIEW("OtherInListMatchesValue"), PathType, Permutation,
								AddPathPermutationIndex, OtherPermutationIndex));
						}
						if (!Tree.ContainsPathOrParent(OtherPath))
						{
							ADD_ERROR(GetDirTreeTestName(TEXTVIEW("ContainsPathOrParentOtherInList"), PathType, Permutation,
								AddPathPermutationIndex, OtherPermutationIndex));
						}
						else
						{
							FString ClosestPath;
							if (!Tree.TryFindClosestPath(OtherPath, ClosestPath))
							{
								ADD_ERROR(GetDirTreeTestName(TEXTVIEW("TryFindClosestPathOtherInListSucceeds"), PathType, Permutation,
									AddPathPermutationIndex, OtherPermutationIndex));
							}
							else if (ClosestPath != OtherPath)
							{
								ADD_ERROR(GetDirTreeTestName(TEXTVIEW("TryFindClosestPathOtherInListMatches"), PathType, Permutation,
									AddPathPermutationIndex, OtherPermutationIndex));
							}
						}
					}
				}

				// Assert all paths not yet added are not included
				for (int32 OtherPermutationIndex = AddPathPermutationIndex + 1; OtherPermutationIndex < NumPaths;
					++OtherPermutationIndex)
				{
					int32 OtherIndex = Permutations[Permutation][OtherPermutationIndex];
					FStringView OtherPath = PathsByTypeAndIndex[PathType][OtherIndex];
					if (Tree.Contains(OtherPath))
					{
						ADD_ERROR(GetDirTreeTestName(TEXTVIEW("OtherNotInList"), PathType, Permutation,
							AddPathPermutationIndex, OtherPermutationIndex));
					}
					// Call ContainsPathOrParent to test whether it crashes, but not yet implemented that we verify 
					// what its return value should be.
					(void)Tree.ContainsPathOrParent(OtherPath);
				}

				// Assert all non paths are not included
				for (int32 NonPathIndex = 0; NonPathIndex < NumNonPaths; ++NonPathIndex)
				{
					FStringView NonPath = NonPathsByTypeAndIndex[PathType][NonPathIndex];
					if (Tree.Contains(NonPath))
					{
						ADD_ERROR(GetDirTreeTestName(TEXTVIEW("NonPathNotInList"), PathType, Permutation,
							AddPathPermutationIndex, NonPathIndex));
					}
					// Call ContainsPathOrParent to test whether it crashes, but not yet implemented that we verify 
					// what its return value should be.
					(void)Tree.ContainsPathOrParent(NonPath);
				}
			}

			// Verify that the SubPath is present
			FString ExistingSubParentPath;
			int32* ExistingSubParentValue;
			if (!Tree.TryFindClosestPath(Path0SubPath[PathType], ExistingSubParentPath, &ExistingSubParentValue))
			{
				ADD_ERROR(GetDirTreeTestName(TEXTVIEW("SubPathInTree"), PathType, Permutation, 0, 0));
			}
			else if (ExistingSubParentPath != PathsByTypeAndIndex[PathType][0] || *ExistingSubParentValue != ValueByIndex[0])
			{
				ADD_ERROR(GetDirTreeTestName(TEXTVIEW("SubPathInTreeMatches"), PathType, Permutation, 0, 0));
			}

			// Remove all the Paths (in specified order) and make Contains assertions after each removal
			// Currently we only test removal in FIFO order; bugs that are specific to a removal-order
			// should be dependent only on the final added state and should not be dependent on the earlier add-order.
			for (int32 RemovePathPermutationIndex = 0; RemovePathPermutationIndex < NumPaths;
				++RemovePathPermutationIndex)
			{
				int32 RemovePathIndex = Permutations[Permutation][RemovePathPermutationIndex];
				FStringView RemovePath = PathsByTypeAndIndex[PathType][RemovePathIndex];

				// Remove the path
				bool bExisted;
				Tree.Remove(RemovePath, &bExisted);
				if (!bExisted)
				{
					ADD_ERROR(GetDirTreeTestName(TEXTVIEW("RemoveFoundSomethingToRemove"), PathType, Permutation,
						RemovePathPermutationIndex, 0));
				}
				if (Tree.Num() != NumPaths - (RemovePathPermutationIndex+1))
				{
					ADD_ERROR(GetDirTreeTestName(TEXTVIEW("TreeNum has expected value"), PathType, Permutation,
						RemovePathPermutationIndex, 0));
				}


				// Assert all paths not yet removed are still included
				for (int32 OtherPermutationIndex = RemovePathPermutationIndex+1; OtherPermutationIndex < NumPaths;
					++OtherPermutationIndex)
				{
					int32 OtherIndex = Permutations[Permutation][OtherPermutationIndex];
					FStringView OtherPath = PathsByTypeAndIndex[PathType][OtherIndex];
					int32 OtherValue = ValueByIndex[OtherIndex];
					int32* ExistingValue = Tree.Find(OtherPath);
					if (!ExistingValue)
					{
						ADD_ERROR(GetDirTreeTestName(TEXTVIEW("OtherInListAfterRemoval"), PathType, Permutation,
							RemovePathPermutationIndex, OtherPermutationIndex));
					}
					else
					{
						if (*ExistingValue != OtherValue)
						{
							ADD_ERROR(GetDirTreeTestName(TEXTVIEW("OtherInListAfterRemovalMatches"), PathType, Permutation,
								RemovePathPermutationIndex, OtherPermutationIndex));
						}
						if (!Tree.ContainsPathOrParent(OtherPath))
						{
							ADD_ERROR(GetDirTreeTestName(TEXTVIEW("OtherContainsPathOrParentAfterRemoval"), PathType, Permutation,
								RemovePathPermutationIndex, OtherPermutationIndex));
						}
					}
				}

				// Assert all paths up to and including this one have been removed
				for (int32 OtherPermutationIndex = 0; OtherPermutationIndex <= RemovePathPermutationIndex;
					++OtherPermutationIndex)
				{
					int32 OtherIndex = Permutations[Permutation][OtherPermutationIndex];
					FStringView OtherPath = PathsByTypeAndIndex[PathType][OtherIndex];
					if (Tree.Contains(OtherPath))
					{
						ADD_ERROR(GetDirTreeTestName(TEXT("OtherNotInListAfterRemoval"), PathType, Permutation,
							RemovePathPermutationIndex, OtherPermutationIndex));
					}
					// Call ContainsPathOrParent to test whether it crashes, but not yet implemented that we verify 
					// what its return value should be.
					(void)Tree.ContainsPathOrParent(OtherPath);
				}

				// Assert all non paths are not still included
				for (int32 NonPathIndex = 0; NonPathIndex < NumNonPaths; ++NonPathIndex)
				{
					FStringView NonPath = NonPathsByTypeAndIndex[PathType][NonPathIndex];
					if (Tree.Contains(NonPath))
					{
						ADD_ERROR(GetDirTreeTestName(TEXTVIEW("NonPathNotInListAfterRemoval"), PathType, Permutation,
							RemovePathPermutationIndex, NonPathIndex));
					}
					// Call ContainsPathOrParent to test whether it crashes, but not yet implemented that we verify 
					// what its return value should be.
					(void)Tree.ContainsPathOrParent(NonPath);
				}
			}
			if (!Tree.IsEmpty())
			{
				ADD_ERROR(GetDirTreeTestName(TEXTVIEW("TreeEmptyAfterRemoval"), PathType, Permutation, 0, 0));
			}
		}
	}

	// Testing some pathtype-independent scenarios
	{
		TDirectoryTree<int32> Tree;
		Tree.FindOrAdd(TEXTVIEW("/Root/Path1")) = 1;
		Tree.FindOrAdd(TEXTVIEW("/Root/Path2")) = 2;
		int32* FoundRoot = Tree.FindClosestValue(TEXTVIEW("/Root"));
		int32* FoundPath1 = Tree.FindClosestValue(TEXTVIEW("/Root/Path1"));
		int32* FoundPath1Sub = Tree.FindClosestValue(TEXTVIEW("/Root/Path1/Sub"));
		int32* FoundPath2 = Tree.FindClosestValue(TEXTVIEW("/Root/Path2"));
		int32* FoundPath2Sub = Tree.FindClosestValue(TEXTVIEW("/Root/Path2/Sub"));
		CHECK_MESSAGE(TEXT("TwoPaths Root does not exist"), FoundRoot == nullptr);
		CHECK_MESSAGE(TEXT("TwoPaths Path1 Value matches"), FoundPath1 && *FoundPath1 == 1);
		CHECK_MESSAGE(TEXT("TwoPaths Path1Sub Value matches"), FoundPath1Sub &&* FoundPath1Sub == 1);
		CHECK_MESSAGE(TEXT("TwoPaths Path2 Value matches"), FoundPath2 && *FoundPath2 == 2);
		CHECK_MESSAGE(TEXT("TwoPaths Path2Sub Value matches"), FoundPath2Sub && *FoundPath2Sub == 2);
	}
	{
		TDirectoryTree<int32> Tree;
		Tree.FindOrAdd(TEXTVIEW("/Root/Path1/A/B/C")) = 1;
		Tree.FindOrAdd(TEXTVIEW("/Root/Path2/A/B/C")) = 2;
		int32* FoundRoot = Tree.FindClosestValue(TEXTVIEW("/Root"));
		int32* FoundPath1 = Tree.FindClosestValue(TEXTVIEW("/Root/Path1/A/B/C"));
		int32* FoundPath1Sub = Tree.FindClosestValue(TEXTVIEW("/Root/Path1/A/B/C/Sub"));
		int32* FoundPath1Parent = Tree.FindClosestValue(TEXTVIEW("/Root/Path1/A"));
		int32* FoundPath2 = Tree.FindClosestValue(TEXTVIEW("/Root/Path2/A/B/C"));
		int32* FoundPath2Sub = Tree.FindClosestValue(TEXTVIEW("/Root/Path2/A/B/C/Sub"));
		int32* FoundPath2Parent = Tree.FindClosestValue(TEXTVIEW("/Root/Path2/A"));
		CHECK_MESSAGE(TEXT("TwoPathsLong Root does not exist"), FoundRoot == nullptr);
		CHECK_MESSAGE(TEXT("TwoPathsLong Path1 Value matches"), FoundPath1 && *FoundPath1 == 1);
		CHECK_MESSAGE(TEXT("TwoPathsLong Path1Sub Value matches"), FoundPath1Sub && *FoundPath1Sub == 1);
		CHECK_MESSAGE(TEXT("TwoPathsLong Path1 Parent does not exist"), FoundPath1Parent == nullptr);
		CHECK_MESSAGE(TEXT("TwoPathsLong Path2 Value matches"), FoundPath2 && *FoundPath2 == 2);
		CHECK_MESSAGE(TEXT("TwoPathsLong Path2Sub Value matches"), FoundPath2Sub && *FoundPath2Sub == 2);
		CHECK_MESSAGE(TEXT("TwoPathsLong Path2 Parent does not exist"), FoundPath2Parent == nullptr);
	}

	struct FMoveConstructOnly
	{
		FMoveConstructOnly() { Value = 437; }
		FMoveConstructOnly(FMoveConstructOnly&& Other) { Value = Other.Value; Other.Value = -1; }
		FMoveConstructOnly(const FMoveConstructOnly& Other) = delete;
		FMoveConstructOnly& operator=(FMoveConstructOnly&& Other) = delete;
		FMoveConstructOnly& operator=(const FMoveConstructOnly& Other) = delete;
		
		int32 Value;
	};

	{
		TDirectoryTree<FMoveConstructOnly> Tree;
		Tree.FindOrAdd(TEXTVIEW("/Root/PathM")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Root/PathP")).Value = 2;
		Tree.FindOrAdd(TEXTVIEW("/Root/PathA")).Value = 3;
		Tree.FindOrAdd(TEXTVIEW("/Root/PathZ"));

		FMoveConstructOnly* Value = Tree.Find(TEXTVIEW("/Root/PathA"));
		CHECK_MESSAGE(TEXT("MoveConstructOnlyValueA correct"), Value && Value->Value == 3);
		Value = Tree.Find(TEXTVIEW("/Root/PathM"));
		CHECK_MESSAGE(TEXT("MoveConstructOnlyValueM correct"), Value && Value->Value == 1);
		Value = Tree.Find(TEXTVIEW("/Root/PathP"));
		CHECK_MESSAGE(TEXT("MoveConstructOnlyValueP correct"), Value && Value->Value == 2);
		Value = Tree.Find(TEXTVIEW("/Root/PathZ"));
		CHECK_MESSAGE(TEXT("MoveConstructOnlyValueZ correct"), Value && Value->Value == 437);
	}

	// Handling special case of drive specifiers without a path
	{
		TStringBuilder<16> FoundPath;
		TArray<FString> ChildNames;
		int* FoundValue = nullptr;
		auto Reset = [&FoundPath, &ChildNames, &FoundValue]()
			{
				FoundPath.Reset();
				ChildNames.Reset();
				FoundValue = nullptr;
			};

		{
			TDirectoryTree<int32> Tree;
			Tree.FindOrAdd(TEXTVIEW("D:")) = 1;

			Reset();
			CHECK_MESSAGE(TEXT("DriveSpecifier: Before PathSep: Without PathSep: Tree.Contains"), Tree.Contains(TEXTVIEW("D:")));
			CHECK_MESSAGE(TEXT("DriveSpecifier: Before PathSep: Without PathSep: Tree.Find"), Tree.Find(TEXTVIEW("D:")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: Before PathSep: Without PathSep: Tree.ContainsPathOrParent"), Tree.ContainsPathOrParent(TEXTVIEW("D:")));
			CHECK_MESSAGE(TEXT("DriveSpecifier: Before PathSep: Without PathSep: Tree.FindClosestValue"), Tree.FindClosestValue(TEXTVIEW("D:")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: Before PathSep: Without PathSep: Tree.TryFindClosestPath"),
				Tree.TryFindClosestPath(TEXTVIEW("D:"), FoundPath, &FoundValue) && FoundPath.Len() > 0 && FoundValue != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: Before PathSep: Without PathSep: Tree.TryGetChildren"), Tree.TryGetChildren(TEXTVIEW("D:"), ChildNames));

			Tree.FindOrAdd(TEXTVIEW("D:/root")) = 1;

			Reset();
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): Without PathSep: Tree.Contains"), Tree.Contains(TEXTVIEW("D:")));
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): Without PathSep: Tree.Find"), Tree.Find(TEXTVIEW("D:")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): Without PathSep: Tree.ContainsPathOrParent"), Tree.ContainsPathOrParent(TEXTVIEW("D:")));
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): Without PathSep: Tree.FindClosestValue"), Tree.FindClosestValue(TEXTVIEW("D:")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): Without PathSep: Tree.TryFindClosestPath"),
				Tree.TryFindClosestPath(TEXTVIEW("D:"), FoundPath, &FoundValue) && FoundPath.Len() > 0 && FoundValue != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): Without PathSep: Tree.TryGetChildren"), Tree.TryGetChildren(TEXTVIEW("D:"), ChildNames));

			Reset();
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): With PathSep: Tree.Contains"), Tree.Contains(TEXTVIEW("D:/")));
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): With PathSep: Tree.Find"), Tree.Find(TEXTVIEW("D:/")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): With PathSep: Tree.ContainsPathOrParent"), Tree.ContainsPathOrParent(TEXTVIEW("D:/")));
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): With PathSep: Tree.FindClosestValue"), Tree.FindClosestValue(TEXTVIEW("D:/")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): With PathSep: Tree.TryFindClosestPath"),
				Tree.TryFindClosestPath(TEXTVIEW("D:/"), FoundPath, &FoundValue) && FoundPath.Len() > 0 && FoundValue != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('/'): With PathSep: Tree.TryGetChildren"), Tree.TryGetChildren(TEXTVIEW("D:/"), ChildNames));
		}
		{
			TDirectoryTree<int32> Tree;
			Tree.FindOrAdd(TEXTVIEW("D:")) = 1;
			Tree.FindOrAdd(TEXTVIEW("D:\\root")) = 1;

			Reset();
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): Without PathSep: Tree.Contains"), Tree.Contains(TEXTVIEW("D:")));
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): Without PathSep: Tree.Find"), Tree.Find(TEXTVIEW("D:")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): Without PathSep: Tree.ContainsPathOrParent"), Tree.ContainsPathOrParent(TEXTVIEW("D:")));
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): Without PathSep: Tree.FindClosestValue"), Tree.FindClosestValue(TEXTVIEW("D:")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): Without PathSep: Tree.TryFindClosestPath"),
				Tree.TryFindClosestPath(TEXTVIEW("D:"), FoundPath, &FoundValue) && FoundPath.Len() > 0 && FoundValue != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): Without PathSep: Tree.TryGetChildren"), Tree.TryGetChildren(TEXTVIEW("D:"), ChildNames));

			Reset();
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): With PathSep: Tree.Contains"), Tree.Contains(TEXTVIEW("D:\\")));
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): With PathSep: Tree.Find"), Tree.Find(TEXTVIEW("D:\\")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): With PathSep: Tree.ContainsPathOrParent"), Tree.ContainsPathOrParent(TEXTVIEW("D:\\")));
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): With PathSep: Tree.FindClosestValue"), Tree.FindClosestValue(TEXTVIEW("D:\\")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): With PathSep: Tree.TryFindClosestPath"),
				Tree.TryFindClosestPath(TEXTVIEW("D:\\"), FoundPath, &FoundValue) && FoundPath.Len() > 0 && FoundValue != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifier: After PathSep('\\'): With PathSep: Tree.TryGetChildren"), Tree.TryGetChildren(TEXTVIEW("D:\\"), ChildNames));
		}
		{
			TDirectoryTree<int32> Tree;
			Tree.FindOrAdd(TEXTVIEW("D:root")) = 1;

			Reset();
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: Before PathSep: Without PathSep: Tree.Contains"), Tree.Contains(TEXTVIEW("D:root")));
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: Before PathSep: Without PathSep: Tree.Find"), Tree.Find(TEXTVIEW("D:root")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: Before PathSep: Without PathSep: Tree.ContainsPathOrParent"), Tree.ContainsPathOrParent(TEXTVIEW("D:root")));
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: Before PathSep: Without PathSep: Tree.FindClosestValue"), Tree.FindClosestValue(TEXTVIEW("D:root")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: Before PathSep: Without PathSep: Tree.TryFindClosestPath"),
				Tree.TryFindClosestPath(TEXTVIEW("D:root"), FoundPath, &FoundValue) && FoundPath.Len() > 0 && FoundValue != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: Before PathSep: Without PathSep: Tree.TryGetChildren"), Tree.TryGetChildren(TEXTVIEW("D:root"), ChildNames));

			Tree.FindOrAdd(TEXTVIEW("D:\\root\\path")) = 1;

			Reset();
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): Without PathSep: Tree.Contains"), Tree.Contains(TEXTVIEW("D:root")));
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): Without PathSep: Tree.Find"), Tree.Find(TEXTVIEW("D:root")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): Without PathSep: Tree.ContainsPathOrParent"), Tree.ContainsPathOrParent(TEXTVIEW("D:root")));
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): Without PathSep: Tree.FindClosestValue"), Tree.FindClosestValue(TEXTVIEW("D:root")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): Without PathSep: Tree.TryFindClosestPath"),
				Tree.TryFindClosestPath(TEXTVIEW("D:root"), FoundPath, &FoundValue) && FoundPath.Len() > 0 && FoundValue != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): Without PathSep: Tree.TryGetChildren"), Tree.TryGetChildren(TEXTVIEW("D:root"), ChildNames));

			Reset();
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): With PathSep: Tree.Contains"), Tree.Contains(TEXTVIEW("D:\\root")));
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): With PathSep: Tree.Find"), Tree.Find(TEXTVIEW("D:\\root")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): With PathSep: Tree.ContainsPathOrParent"), Tree.ContainsPathOrParent(TEXTVIEW("D:\\root")));
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): With PathSep: Tree.FindClosestValue"), Tree.FindClosestValue(TEXTVIEW("D:\\root")) != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): With PathSep: Tree.TryFindClosestPath"),
				Tree.TryFindClosestPath(TEXTVIEW("D:root"), FoundPath, &FoundValue) && FoundPath.Len() > 0 && FoundValue != nullptr);
			CHECK_MESSAGE(TEXT("DriveSpecifierLong: After PathSep('\\'): With PathSep: Tree.TryGetChildren"), Tree.TryGetChildren(TEXTVIEW("D:\\root"), ChildNames));
		}
	}

	// Testing accessors
	{
		// GetChildren and Iteration
		TDirectoryTree<FMoveConstructOnly> Tree;
		bool bExists;
		TArray<FString> Children;
		TArray<FString> IterKeys;
		TArray<FMoveConstructOnly*> IterValues;
		TArray<FString> PtrIterKeys;
		TArray<FMoveConstructOnly*> PtrIterValues;

		auto ReadIterPairs = [&Tree, &IterKeys, &IterValues, &PtrIterKeys, &PtrIterValues]()
			{
				IterKeys.Reset();
				IterValues.Reset();
				for (TDirectoryTree<FMoveConstructOnly>::FIterator Iter(Tree.CreateIterator()); Iter; ++Iter)
				{
					IterKeys.Emplace(Iter->Key);
					IterValues.Add(&Iter->Value);
				}
				PtrIterKeys.Reset();
				PtrIterValues.Reset();
				for (TDirectoryTree<FMoveConstructOnly>::FPointerIterator PtrIter(Tree.CreateIteratorForImplied()); PtrIter; ++PtrIter)
				{
					PtrIterKeys.Emplace(PtrIter->Key);
					PtrIterValues.Add(PtrIter->Value);
				}
			};

		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::None);
		CHECK_MESSAGE(TEXT("GetChildrenEmpty, Root, !ImpliedParent"),
			bExists == false && Children.IsEmpty());

		CHECK_MESSAGE(TEXT("Iterate, Empty"), !Tree.CreateIterator());
		CHECK_MESSAGE(TEXT("IterateImplied, Empty"), !Tree.CreateIteratorForImplied());

		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenEmpty, Root, ImpliedParent"),
			bExists == true && Children.IsEmpty());

		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/SomePath"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenEmpty, Non-root"),
			bExists == false && Children.IsEmpty());

		Tree.FindOrAdd(TEXTVIEW("")).Value = 1;
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::None);
		CHECK_MESSAGE(TEXT("GetChildrenRoot, !ImpliedParent, !ImpliedChildren"),
			bExists == true && Children.IsEmpty());

		ReadIterPairs();
		CHECK_MESSAGE(TEXT("Iterate, RootNodeOnly"),
			IterKeys.Num() == 1 && IterKeys[0] == TEXTVIEW("") && IterValues[0] && IterValues[0]->Value == 1);
		CHECK_MESSAGE(TEXT("IterateImplied, RootNodeOnly"),
			PtrIterKeys.Num() == 1 && PtrIterKeys[0] == TEXTVIEW("") && PtrIterValues[0] && PtrIterValues[0]->Value == 1);

		Tree.Empty();
		Tree.FindOrAdd(TEXTVIEW("/")).Value = 1;

		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenRoot, !ImpliedParent, ImpliedChildren"),
			bExists == false && Children.IsEmpty());

		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenRoot, ImpliedParent, ImpliedChildren"),
			bExists == true && UnorderedEquals(Children, { TEXT("/") }));

		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildren appends to the outdir rather than resetting"),
			bExists == true && Children.Num() == 2);

		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenRoot, ImpliedParent, !ImpliedChildren"),
			bExists == true && UnorderedEquals(Children, { TEXT("/") }));

		ReadIterPairs();
		CHECK_MESSAGE(TEXT("Iterate, RootNodeOnlyWithPath"),
			IterKeys.Num() == 1 && IterKeys[0] == TEXTVIEW("/") && IterValues[0] && IterValues[0]->Value == 1);
		CHECK_MESSAGE(TEXT("IterateImplied, RootNodeOnlyWithPath"),
			PtrIterKeys.Num() == 1 && PtrIterKeys[0] == TEXTVIEW("/") && PtrIterValues[0] && PtrIterValues[0]->Value == 1);

		Tree.Empty();
		Tree.FindOrAdd(TEXTVIEW("/Root/Child")).Value = 1;

		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren);
		CHECK_MESSAGE(TEXT("GetChildrenRootImpliedChild, ImpliedParent, ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("/") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::ImpliedParent);
		CHECK_MESSAGE(TEXT("GetChildrenRootImpliedChild, ImpliedParent, !ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("/Root/Child") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenRootImpliedChild, ImpliedParent, ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("/"), TEXT("/Root"), TEXT("/Root/Child") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW(""), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenRootImpliedChild, ImpliedParent, !ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("/Root/Child") }));

		ReadIterPairs();
		CHECK_MESSAGE(TEXT("Iterate, RootChildSingleNode"),
			IterKeys.Num() == 1 && IterKeys[0] == TEXTVIEW("/Root/Child") && IterValues[0] && IterValues[0]->Value == 1);
		CHECK_MESSAGE(TEXT("IterateImplied, RootChildSingleNode"),
			PtrIterKeys.Num() == 3 && PtrIterKeys[0] == TEXTVIEW("/") && !PtrIterValues[0]
			&& PtrIterKeys[1] == TEXTVIEW("/Root") && !PtrIterValues[1]
			&& PtrIterKeys[2] == TEXTVIEW("/Root/Child") && PtrIterValues[2] && PtrIterValues[2]->Value == 1);

		Tree.FindOrAdd(TEXTVIEW("/Root/Child2")).Value = 1;

		Tree.Empty();
		Tree.FindOrAdd(TEXTVIEW("/Stem/A_OtherChild")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/B_ImpliedRoot/AddedChild")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/B_ImpliedRoot/AddedChild/Child")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/B_ImpliedRoot/ImpliedChild/AddedChild")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/B_ImpliedRoot/ImpliedChild/AddedChild/AddedChild")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/B_ImpliedRoot/ImpliedChild/AddedChild/ImpliedChild/AddedChild")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePath/ImpliedChild/AddedChild")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePath/ImpliedChild/AddedChild/Child")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/D_MiddleRoot/MiddlePath/AddedChild")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/D_MiddleRoot/MiddlePath/AddedChild/Child")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/E_AddedRoot")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/E_AddedRoot/AddedChild")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/E_AddedRoot/ImpliedChild/AddedChild")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/E_AddedRoot/ImpliedChild/AddedChild/ImpliedChild/AddedChild")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/F_AddedRoot")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Stem/F_AddedRoot/ImpliedChild/AddedChild")).Value = 1;

		TArray<const TCHAR*> ExpectedAdded = {
			TEXT("/Stem/A_OtherChild"),
			TEXT("/Stem/B_ImpliedRoot/AddedChild"),
			TEXT("/Stem/B_ImpliedRoot/AddedChild/Child"),
			TEXT("/Stem/B_ImpliedRoot/ImpliedChild/AddedChild"),
			TEXT("/Stem/B_ImpliedRoot/ImpliedChild/AddedChild/AddedChild"),
			TEXT("/Stem/B_ImpliedRoot/ImpliedChild/AddedChild/ImpliedChild/AddedChild"),
			TEXT("/Stem/C_MiddleRoot/MiddlePath/ImpliedChild/AddedChild"),
			TEXT("/Stem/C_MiddleRoot/MiddlePath/ImpliedChild/AddedChild/Child"),
			TEXT("/Stem/D_MiddleRoot/MiddlePath/AddedChild"),
			TEXT("/Stem/D_MiddleRoot/MiddlePath/AddedChild/Child"),
			TEXT("/Stem/E_AddedRoot"),
			TEXT("/Stem/E_AddedRoot/AddedChild"),
			TEXT("/Stem/E_AddedRoot/ImpliedChild/AddedChild"),
			TEXT("/Stem/E_AddedRoot/ImpliedChild/AddedChild/ImpliedChild/AddedChild"),
			TEXT("/Stem/F_AddedRoot"),
			TEXT("/Stem/F_AddedRoot/ImpliedChild/AddedChild")
		};
		TArray<const TCHAR*> ExpectedImplied = {
			TEXT("/"),
			TEXT("/Stem"),
			TEXT("/Stem/A_OtherChild"),
			TEXT("/Stem/B_ImpliedRoot"),
			TEXT("/Stem/B_ImpliedRoot/AddedChild"),
			TEXT("/Stem/B_ImpliedRoot/AddedChild/Child"),
			TEXT("/Stem/B_ImpliedRoot/ImpliedChild"),
			TEXT("/Stem/B_ImpliedRoot/ImpliedChild/AddedChild"),
			TEXT("/Stem/B_ImpliedRoot/ImpliedChild/AddedChild/AddedChild"),
			TEXT("/Stem/B_ImpliedRoot/ImpliedChild/AddedChild/ImpliedChild"),
			TEXT("/Stem/B_ImpliedRoot/ImpliedChild/AddedChild/ImpliedChild/AddedChild"),
			TEXT("/Stem/C_MiddleRoot"),
			TEXT("/Stem/C_MiddleRoot/MiddlePath"),
			TEXT("/Stem/C_MiddleRoot/MiddlePath/ImpliedChild"),
			TEXT("/Stem/C_MiddleRoot/MiddlePath/ImpliedChild/AddedChild"),
			TEXT("/Stem/C_MiddleRoot/MiddlePath/ImpliedChild/AddedChild/Child"),
			TEXT("/Stem/D_MiddleRoot"),
			TEXT("/Stem/D_MiddleRoot/MiddlePath"),
			TEXT("/Stem/D_MiddleRoot/MiddlePath/AddedChild"),
			TEXT("/Stem/D_MiddleRoot/MiddlePath/AddedChild/Child"),
			TEXT("/Stem/E_AddedRoot"),
			TEXT("/Stem/E_AddedRoot/AddedChild"),
			TEXT("/Stem/E_AddedRoot/ImpliedChild"),
			TEXT("/Stem/E_AddedRoot/ImpliedChild/AddedChild"),
			TEXT("/Stem/E_AddedRoot/ImpliedChild/AddedChild/ImpliedChild"),
			TEXT("/Stem/E_AddedRoot/ImpliedChild/AddedChild/ImpliedChild/AddedChild"),
			TEXT("/Stem/F_AddedRoot"),
			TEXT("/Stem/F_AddedRoot/ImpliedChild"),
			TEXT("/Stem/F_AddedRoot/ImpliedChild/AddedChild")
		};
		ReadIterPairs();

		// Make sure the ranged for interface, which forwards to CreateIterator(), compiles correctly
		TArray<FString> RangedForKeys;
		TArray<FMoveConstructOnly*> RangedForValues;
		for (TPair<FStringView, FMoveConstructOnly&> Pair : Tree)
		{
			RangedForKeys.Emplace(Pair.Key);
			RangedForValues.Add(&Pair.Value);
		}
		// Make sure the const iterators work correctly
		TArray<FString> ConstItKeys;
		TArray<const FMoveConstructOnly*> ConstItValues;
		TArray<FString> ConstItImpliedKeys;
		TArray<const FMoveConstructOnly*> ConstItImpliedValues;
		for (TDirectoryTree<FMoveConstructOnly>::FConstIterator ConstIt(Tree.CreateConstIterator()); ConstIt; ++ConstIt)
		{
			ConstItKeys.Emplace(ConstIt->Key);
			ConstItValues.Add(&ConstIt->Value);
		}
		for (TDirectoryTree<FMoveConstructOnly>::FConstPointerIterator ConstIt(Tree.CreateConstIteratorForImplied()); ConstIt; ++ConstIt)
		{
			ConstItImpliedKeys.Emplace(ConstIt->Key);
			ConstItImpliedValues.Add(ConstIt->Value);
		}
		CHECK_MESSAGE(TEXT("Iterate, ComplicatedTree1"), UnorderedEquals(IterKeys, ExpectedAdded));
		CHECK_MESSAGE(TEXT("IterateImplied, ComplicatedTree1"), UnorderedEquals(PtrIterKeys, ExpectedImplied));
		CHECK_MESSAGE(TEXT("RangedFor, ComplicatedTree1"), UnorderedEquals(RangedForKeys, ExpectedAdded));
		CHECK_MESSAGE(TEXT("CreateConstIterator, ComplicatedTree1"), UnorderedEquals(ConstItKeys, ExpectedAdded));
		CHECK_MESSAGE(TEXT("CreateConstIteratorForImplied, ComplicatedTree1"), UnorderedEquals(ConstItImpliedKeys, ExpectedImplied));

		for (FMoveConstructOnly* Value : RangedForValues)
		{
			CHECK_MESSAGE(TEXT("RangedForValues, ComplicatedTree1"), Value && Value->Value == 1);
		}
		for (const FMoveConstructOnly* Value : ConstItValues)
		{
			CHECK_MESSAGE(TEXT("ConstItValues, ComplicatedTree1"), Value && Value->Value == 1);
		}
		for (const FMoveConstructOnly* Value : ConstItImpliedValues)
		{
			CHECK_MESSAGE(TEXT("ConstItImpliedValues, ComplicatedTree1"), !Value || Value->Value == 1);
		}

		// Case: Requested path is an implied path that is a stored child in the tree.
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/B_ImpliedRoot"), Children,
			EDirectoryTreeGetFlags::None);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA B_ImpliedRoot, !ImpliedParent, !ImpliedChildren, !Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/B_ImpliedRoot"), Children,
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA B_ImpliedRoot, !ImpliedParent, !ImpliedChildren, Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/B_ImpliedRoot"), Children,
			EDirectoryTreeGetFlags::ImpliedChildren);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA B_ImpliedRoot, !ImpliedParent, ImpliedChildren, !Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/B_ImpliedRoot"), Children,
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA B_ImpliedRoot, !ImpliedParent, ImpliedChildren, Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/B_ImpliedRoot"), Children,
			EDirectoryTreeGetFlags::ImpliedParent);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA B_ImpliedRoot, ImpliedParent, !ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild"), TEXT("ImpliedChild/AddedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/B_ImpliedRoot"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA B_ImpliedRoot, ImpliedParent, !ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild"), TEXT("AddedChild/Child"),
				TEXT("ImpliedChild/AddedChild"), TEXT("ImpliedChild/AddedChild/AddedChild"),
				TEXT("ImpliedChild/AddedChild/ImpliedChild/AddedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/B_ImpliedRoot"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA B_ImpliedRoot, ImpliedParent, ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild"), TEXT("ImpliedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/B_ImpliedRoot"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA B_ImpliedRoot, ImpliedParent, ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild"), TEXT("AddedChild/Child"),
				TEXT("ImpliedChild"), TEXT("ImpliedChild/AddedChild"), TEXT("ImpliedChild/AddedChild/AddedChild"),
				TEXT("ImpliedChild/AddedChild/ImpliedChild"), TEXT("ImpliedChild/AddedChild/ImpliedChild/AddedChild") }));

		// Case: Requested path is an implied path that is not a stored child in the tree - it is an in-between dir in a
		// relpath - and it has an implied child
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::None);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA C_MiddleRoot, !ImpliedParent, !ImpliedChildren, !Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA C_MiddleRoot, !ImpliedParent, !ImpliedChildren, Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedChildren);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA C_MiddleRoot, !ImpliedParent, ImpliedChildren, !Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA C_MiddleRoot, !ImpliedParent, ImpliedChildren, Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedParent);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA C_MiddleRoot, ImpliedParent, !ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("ImpliedChild/AddedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA C_MiddleRoot, ImpliedParent, !ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("ImpliedChild/AddedChild"),
				TEXT("ImpliedChild/AddedChild/Child") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA C_MiddleRoot, ImpliedParent, ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("ImpliedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA C_MiddleRoot, ImpliedParent, ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("ImpliedChild"), TEXT("ImpliedChild/AddedChild"),
				TEXT("ImpliedChild/AddedChild/Child") }));

		// Case: Requested path is a non-existent sibling path of an implied path that is not a stored
		// path.
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/C_MiddleRoot/MiddlePathExceptItDoesNotExist"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA MiddlePathExceptItDoesNotExist, ImpliedParent, ImpliedChildren, Recursive"),
			bExists == false && Children.IsEmpty());

		// Case: Requested path is an implied path that is not a stored child in the tree - it is an in-between dir in a
		// relpath - and it has an added child
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/D_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::None);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA D_MiddleRoot, !ImpliedParent, !ImpliedChildren, !Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/D_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA D_MiddleRoot, !ImpliedParent, !ImpliedChildren, Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/D_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedChildren);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA D_MiddleRoot, !ImpliedParent, ImpliedChildren, !Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/D_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA D_MiddleRoot, !ImpliedParent, ImpliedChildren, Recursive"),
			bExists == false && Children.IsEmpty());
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/D_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedParent);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA D_MiddleRoot, ImpliedParent, !ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/D_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA D_MiddleRoot, ImpliedParent, !ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild"), TEXT("AddedChild/Child") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/D_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA D_MiddleRoot, ImpliedParent, ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/D_MiddleRoot/MiddlePath"), Children,
			EDirectoryTreeGetFlags::ImpliedParent |
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA D_MiddleRoot, ImpliedParent, ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild"), TEXT("AddedChild/Child") }));

		// Case: Requested path is an added path and it has an added child and an implied child
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/E_AddedRoot"), Children,
			EDirectoryTreeGetFlags::None);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA E_AddedRoot, !ImpliedParent, !ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild"), TEXT("ImpliedChild/AddedChild")}));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/E_AddedRoot"), Children,
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA E_AddedRoot, !ImpliedParent, !ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild"), TEXT("ImpliedChild/AddedChild"),
				TEXT("ImpliedChild/AddedChild/ImpliedChild/AddedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/E_AddedRoot"), Children,
			EDirectoryTreeGetFlags::ImpliedChildren);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA E_AddedRoot, !ImpliedParent, ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild"), TEXT("ImpliedChild")}));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/E_AddedRoot"), Children,
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA E_MiddleRoot, !ImpliedParent, ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("AddedChild"), TEXT("ImpliedChild"),
				TEXT("ImpliedChild/AddedChild"), TEXT("ImpliedChild/AddedChild/ImpliedChild"),
				TEXT("ImpliedChild/AddedChild/ImpliedChild/AddedChild") }));

		// Case: Requested path is an added path and it has only an implied child
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/F_AddedRoot"), Children,
			EDirectoryTreeGetFlags::None);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA F_AddedRoot, !ImpliedParent, !ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("ImpliedChild/AddedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/F_AddedRoot"), Children,
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA F_AddedRoot, !ImpliedParent, !ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("ImpliedChild/AddedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/F_AddedRoot"), Children,
			EDirectoryTreeGetFlags::ImpliedChildren);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA F_AddedRoot, !ImpliedParent, ImpliedChildren, !Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("ImpliedChild") }));
		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Stem/F_AddedRoot"), Children,
			EDirectoryTreeGetFlags::ImpliedChildren |
			EDirectoryTreeGetFlags::Recursive);
		CHECK_MESSAGE(TEXT("GetChildrenComplexA F_AddedRoot, !ImpliedParent, ImpliedChildren, Recursive"),
			bExists == true && UnorderedEquals(Children, { TEXT("ImpliedChild"),
				TEXT("ImpliedChild/AddedChild") }));

		// Case: Requesting !ImpliedChildren and !Recursive on a path with an implied child, should report
		// the added path children of the Implied child
		Tree.Empty();
		Tree.FindOrAdd(TEXTVIEW("/Root/Implied1/Added1")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Root/Implied1/Added2")).Value = 1;
		Tree.FindOrAdd(TEXTVIEW("/Root/Implied2/Added")).Value = 1;

		Children.Reset();
		bExists = Tree.TryGetChildren(TEXTVIEW("/Root"), Children,
			EDirectoryTreeGetFlags::ImpliedParent);
		CHECK_MESSAGE(TEXT("!ImpliedChildren, !Recursive, and direct child is implied."),
			bExists == true && UnorderedEquals(Children, { TEXT("Implied1/Added1"), TEXT("Implied1/Added2"), TEXT("Implied2/Added") }));
	}
}
#endif // WITH_TESTS