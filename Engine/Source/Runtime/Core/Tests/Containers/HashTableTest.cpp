// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/HashTable.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FHashTableTestMove, "System::Core::Containers::HashTable::Move", "[ApplicationContextMask][SmokeFilter]")
{
    // Hash values by address and store in hash table
    int32 Values[] = { 1, 2, 3, 4, 5 };
    FHashTable A(4, 16);
    for (int32 i=0; i < UE_ARRAY_COUNT(Values); ++i)
    {
        A.Add(GetTypeHash(&Values[i]), i);
    }

    FHashTable B = MoveTemp(A);
    CHECK(A.GetIndexSize() == 0);
    CHECK(A.GetHashSize() == 0);
    CHECK(B.GetIndexSize() == 16);
    CHECK(B.GetHashSize() == 4);

    auto Find = [&Values](const FHashTable& Table, int32 TargetIndex)
    {
        for (int32 Index = Table.First(GetTypeHash(&Values[TargetIndex])); Table.IsValid(Index); Index = Table.Next(Index))
        {
            if (Index == TargetIndex)
            {
                return true;
            }
        }
        return false;
    };
    for (int32 i=0; i < UE_ARRAY_COUNT(Values); ++i)
    {
        CHECK_FALSE(Find(A, i));
        CHECK(Find(B, i));
    }
}

TEST_CASE_NAMED(FHashTableTestGrow, "System::Core::Containers::HashTable::Grow", "[ApplicationContextMask][SmokeFilter]")
{
    // Hash values by address and store in hash table
    int32 Values[] = { 1, 2, 3, 4, 5 };
    FHashTable Table(4, 16);
    for (int32 i=0; i < UE_ARRAY_COUNT(Values); ++i)
    {
        Table.Add(GetTypeHash(&Values[i]), i);
    }

    auto Find = [&Values](const FHashTable& Table, int32 TargetIndex)
    {
        for (int32 Index = Table.First(GetTypeHash(&Values[TargetIndex])); Table.IsValid(Index); Index = Table.Next(Index))
        {
            if (Index == TargetIndex)
            {
                return true;
            }
        }
        return false;
    };
    for (int32 i=0; i < UE_ARRAY_COUNT(Values); ++i)
    {
        CHECK(Find(Table, i));
    }

    Table.Resize(128);
    for (int32 i=0; i < UE_ARRAY_COUNT(Values); ++i)
    {
        CHECK(Find(Table, i));
    }
}

#endif // WITH_TESTS
