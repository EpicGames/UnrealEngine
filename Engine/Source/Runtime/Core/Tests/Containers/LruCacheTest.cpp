// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/LruCache.h"
#include "Tests/TestHarnessAdapter.h"

namespace Test
{
	template<typename KeyType, typename ValueType>
	void TestCompareCache(const TLruCache<KeyType,ValueType>& CacheA, const TLruCache<KeyType,ValueType>& CacheB)
	{
		CHECK(CacheA.GetLeastRecentKey() == CacheB.GetLeastRecentKey());

		TArray<KeyType> KeysA;
		CacheA.GetKeys(KeysA);

		TArray<KeyType> KeysB;
		CacheB.GetKeys(KeysB);
		CHECK(KeysA == KeysB);

		for (auto Key : KeysA)
		{
			const ValueType* ValuePtr = CacheA.Find(Key);
			const ValueType* ValueCopyPtr = CacheB.Find(Key);
			CHECK(ValuePtr != ValueCopyPtr);
			CHECK(*ValuePtr == *ValueCopyPtr);
		}
	};
}

TEST_CASE_NAMED(FLruCacheTest, "System::Core::Containers::LruCache", "[Core][Containers][LruCache]")
{
	SECTION("Basic")
	{
		TLruCache<int32, FString> Cache(5);
		Cache.Add(0, TEXT("A"));
		Cache.Add(1, TEXT("B"));
		Cache.Add(2, TEXT("C"));
		Cache.Add(3, TEXT("D"));
		Cache.Add(4, TEXT("E"));

		// Validate 0 is least recent key
		int32 LeastRecentKey = Cache.GetLeastRecentKey();
		CHECK(LeastRecentKey == 0);
			
		// Validate value matches inserted value
		CHECK(*Cache.Find(LeastRecentKey) == TEXT("A"));
		
		// Move LeastRecentKey to most recent and compare with inserted value
		CHECK(*Cache.FindAndTouch(LeastRecentKey) == TEXT("A"));

		// Validate 1 is least recent key
		LeastRecentKey = Cache.GetLeastRecentKey();
		CHECK(LeastRecentKey == 1);

		// Validate value matches inserted value
		CHECK(*Cache.Find(LeastRecentKey) == TEXT("B"));

		// Add new value (which should discard LeastRecentKey(1)
		Cache.Add(5, TEXT("F"));

		// Validate discard
		CHECK(Cache.Find(LeastRecentKey) == nullptr);

		// Validate 2 is least recent key
		LeastRecentKey = Cache.GetLeastRecentKey();
		CHECK(LeastRecentKey == 2);

		// Validate value matches inserted value
		CHECK(*Cache.Find(LeastRecentKey) == TEXT("C"));

		// Remove LeastRecent entry(2)
		Cache.RemoveLeastRecent();

		// Validate discard
		CHECK(Cache.Find(LeastRecentKey) == nullptr);

		// Validate 3 is least recent key
		LeastRecentKey = Cache.GetLeastRecentKey();
		CHECK(LeastRecentKey == 3);

		// Validate value matches inserted value
		CHECK(*Cache.Find(LeastRecentKey) == TEXT("D"));

		// Validate size
		CHECK(Cache.Num() == 4);
	}

	SECTION("SetMaxNumElements")
	{
		TLruCache<int32, FString> Cache(5);
		Cache.Add(0, TEXT("A"));
		Cache.Add(1, TEXT("B"));
		Cache.Add(2, TEXT("C"));
		Cache.Add(3, TEXT("D"));
		Cache.Add(4, TEXT("E"));

		TLruCache<int32, FString> CacheCopy(Cache);

		CacheCopy.SetMaxNumElements(10);

		Test::TestCompareCache(Cache, CacheCopy);

		Cache.SetMaxNumElements(3);
		CacheCopy.SetMaxNumElements(3);

		CHECK(Cache.Num() == 3);
		CHECK(CacheCopy.Num() == 3);

		Test::TestCompareCache(Cache, CacheCopy);
	}

	SECTION("Copy")
	{
		TLruCache<int32, FString> Cache(5);
		Cache.Add(0, TEXT("A"));
		Cache.Add(1, TEXT("B"));
		Cache.Add(2, TEXT("C"));
		Cache.Add(3, TEXT("D"));
		Cache.Add(4, TEXT("E"));

		TLruCache<int32, FString> CacheCopy(Cache);
		Test::TestCompareCache(Cache, CacheCopy);
	}

	SECTION("CopyAssign")
	{
		TLruCache<int32, FString> Cache(5);
		Cache.Add(0, TEXT("A"));
		Cache.Add(1, TEXT("B"));
		Cache.Add(2, TEXT("C"));
		Cache.Add(3, TEXT("D"));
		Cache.Add(4, TEXT("E"));

		TLruCache<int32, FString> CacheCopy;
		CacheCopy = Cache;
		Test::TestCompareCache(Cache, CacheCopy);
	}

	SECTION("Move")
	{
		TLruCache<int32, FString> Cache(5);
		Cache.Add(0, TEXT("A"));
		Cache.Add(1, TEXT("B"));
		Cache.Add(2, TEXT("C"));
		Cache.Add(3, TEXT("D"));
		Cache.Add(4, TEXT("E"));

		TLruCache<int32, FString> CacheCopy(Cache);
		TLruCache<int32, FString> MovedCache(MoveTemp(Cache));
		CHECK(Cache.IsEmpty());
		Test::TestCompareCache(CacheCopy, MovedCache);
	}

	SECTION("MoveAssign")
	{
		TLruCache<int32, FString> Cache(5);
		Cache.Add(0, TEXT("A"));
		Cache.Add(1, TEXT("B"));
		Cache.Add(2, TEXT("C"));
		Cache.Add(3, TEXT("D"));
		Cache.Add(4, TEXT("E"));

		TLruCache<int32, FString> CacheCopy(Cache);
		TLruCache<int32, FString> MovedCache;
		MovedCache = MoveTemp(Cache);
		CHECK(Cache.IsEmpty());
		Test::TestCompareCache(CacheCopy, MovedCache);
	}
	
}

#endif // WITH_TESTS