// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/SparseSet.h"
#include "Containers/CompactSet.h"

#include "Containers/Map.h"

#include "Tests/TestHarnessAdapter.h"

struct FSparseMapTestResolver
{
	template<typename InKeyType, typename InValueType, typename Allocator = FDefaultSparseSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<InKeyType, InValueType, false>>
	using MapType = TSparseMap<InKeyType, InValueType, Allocator, KeyFuncs>;

	template<typename InKeyType, typename InValueType, typename Allocator = FDefaultSparseSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<InKeyType, InValueType, true>>
	using MultiMapType = TSparseMultiMap<InKeyType, InValueType, Allocator, KeyFuncs>;
};

struct FCompactMapTestResolver
{
	template<typename InKeyType, typename InValueType, typename Allocator = FDefaultCompactSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<InKeyType, InValueType, false>>
	using MapType = TCompactMap<InKeyType, InValueType, Allocator, KeyFuncs>;

	template<typename InKeyType, typename InValueType, typename Allocator = FDefaultCompactSetAllocator, typename KeyFuncs = TDefaultMapHashableKeyFuncs<InKeyType, InValueType, true>>
	using MultiMapType = TCompactMultiMap<InKeyType, InValueType, Allocator, KeyFuncs>;
};

template<typename MapResolver = FSparseMapTestResolver>
struct TMapTestHelper
{
	template<typename Key, typename Value>
	using MapType = typename MapResolver::template MapType<Key, Value>;

	template<typename Key, typename Value>
	using MultiMapType = typename MapResolver::template MultiMapType<Key, Value>;

	TMapTestHelper()
	{
		SECTION("Default Constructor")
		{
			MapType<int, int> Map;
			CHECK(Map.Num() == 0);

			MultiMapType<int, int> MultiMap;
			CHECK(MultiMap.Num() == 0);
		}

		SECTION("Iterator Test")
		{
			MapType<int, int> MapUnderTest;

			MapUnderTest.Add(5, 55);
			MapUnderTest.Add(1, 11);
			MapUnderTest.Add(3, 33);
			MapUnderTest.Add(4, 44);

			int VisitedItemsCount = 0;
			for (auto It = MapUnderTest.CreateIterator(); It; ++It)
			{
				++VisitedItemsCount;
				if (It->Key == 1)
				{
					It.RemoveCurrent();
				}
			}

			CHECK(MapUnderTest.Num() == 3);
	
			CHECK(VisitedItemsCount == 4);
	
			CHECK(MapUnderTest.Find(5) != nullptr);
			CHECK(MapUnderTest.Find(1) == nullptr);
			CHECK(MapUnderTest.Find(3) != nullptr);
			CHECK(MapUnderTest.Find(4) != nullptr);
		}

		SECTION("Multi Map Test")
		{
			const int32 Keys = 100;
			const int32 Values = 10;

			MultiMapType<int, int> MultiMap;

			for (int32 Value = 0; Value < Values; ++Value)
			{
				for (int32 Key = 0; Key < Keys; ++Key)
				{
					MultiMap.Add(Key, (Key << 16) + Value);
				}
			}

			CHECK(MultiMap.Num() == Keys * Values);

			{
				int32 ValueCount = 0;
				for (auto KeyIter = MultiMap.CreateKeyIterator(3); KeyIter; ++KeyIter)
				{
					++ValueCount;
				}

				CHECK(ValueCount == Values);
			}

			for (int32 Key = 0; Key < Keys; ++Key)
			{
				CHECK(MultiMap.Remove(Key) == Values);
				CHECK(MultiMap.Num() == Values * (Keys - (Key + 1)));
			}
		}

		SECTION("Very Specific broken sequence for multi map remove")
		{
			MultiMapType<int, int> Map;
			{
				Map.Add(70, 1);
				Map.Add(8, 3);
				Map.Add(88, 4);
				Map.Add(30, 6);
				Map.Add(103, 9);
				Map.Add(6, 10);
				Map.Add(0, 12);
				Map.Add(62, 15);
				Map.Add(78, 24);
				Map.Add(70, 30);
				Map.Add(53, 40);
				Map.Add(53, 41);
				Map.Add(72, 44);
				Map.Add(72, 47);
				Map.Add(8, 48);
				Map.Add(8, 49);
				Map.Add(70, 54);
				Map.Add(70, 55);
				Map.Add(53, 56);
				Map.Add(53, 57);
				Map.Add(78, 59);
				Map.Add(93, 64);
				Map.Add(93, 65);
				Map.Add(28, 67);
				Map.Add(28, 68);
				Map.Add(28, 72);
				Map.Add(28, 73);
				Map.Add(30, 74);
				Map.Add(30, 75);
				Map.Add(93, 76);
				Map.Add(93, 77);
				Map.Add(88, 78);
				Map.Add(88, 79);
				Map.Add(93, 80);
				Map.Add(93, 81);
				Map.Add(90, 82);
				Map.Add(90, 83);
				Map.Add(66, 84);
				Map.Add(2, 86);
				Map.Add(64, 91);
				Map.Add(2, 94);
				Map.Add(2, 106);
				Map.Add(2, 107);
				Map.Add(66, 108);
				Map.Add(64, 111);
				Map.Add(103, 117);
				Map.Add(104, 118);
				Map.Add(60, 120);
				Map.Add(60, 121);
				Map.Add(62, 123);
				Map.Add(93, 132);
				Map.Add(90, 134);
				Map.Add(52, 138);
				Map.Add(52, 139);
				Map.Add(52, 140);
				Map.Add(52, 141);
				Map.Add(52, 142);
				Map.Add(52, 143);
				Map.Add(52, 144);
				Map.Add(52, 145);
				Map.Add(52, 146);
				Map.Add(52, 147);
				Map.Add(52, 148);
				Map.Add(52, 149);
				Map.Add(52, 150);
				Map.Add(52, 151);
				Map.Add(52, 152);
				Map.Add(52, 153);
				Map.Add(52, 156);
				Map.Add(102, 158);
				Map.Add(8, 160);
				Map.Add(8, 162);
				Map.Add(68, 165);
				Map.Add(53, 172);
				Map.Add(54, 174);
				Map.Add(54, 175);
				Map.Add(54, 177);
				Map.Add(102, 178);
				Map.Add(102, 179);
				Map.Add(53, 181);
				Map.Add(53, 182);
				Map.Add(53, 183);
				Map.Add(52, 184);
				Map.Add(52, 185);
				Map.Add(68, 186);
				Map.Add(68, 187);
				Map.Add(54, 188);
				Map.Add(54, 189);
				Map.Add(54, 190);
				Map.Add(54, 191);
				Map.Add(53, 192);
				Map.Add(53, 193);
				Map.Add(53, 194);
				Map.Add(53, 195);
				Map.Add(8, 196);
				Map.Add(68, 198);
				Map.Add(30, 201);
				Map.Add(90, 203);
				Map.Add(6, 204);
				Map.Add(104, 207);
				Map.Add(60, 209);
				Map.Add(2, 210);
				Map.Add(52, 212);
				Map.Add(52, 213);
				Map.Add(52, 214);
				Map.Add(52, 215);
				Map.Add(52, 216);
				Map.Add(52, 217);
				Map.Add(52, 218);
				Map.Add(52, 219);
				Map.Add(52, 221);
				Map.Add(10, 222);
				Map.Add(10, 223);
				Map.Add(10, 224);
				Map.Add(10, 225);
				Map.Add(10, 227);
				Map.Add(54, 228);
				Map.Add(54, 229);
				Map.Add(53, 230);
				Map.Add(53, 231);
				Map.Add(53, 232);
				Map.Add(53, 233);
				Map.Add(54, 234);
				Map.Add(54, 235);
				Map.Add(10, 238);
				Map.Add(10, 239);
				Map.Add(53, 241);
				Map.Add(54, 242);
				Map.Add(54, 246);
				Map.Add(54, 247);
				Map.Add(54, 248);
				Map.Add(54, 249);
				Map.Add(53, 254);
				Map.Add(53, 256);
				Map.Add(53, 257);
				Map.Add(28, 258);
				Map.Add(28, 259);
				Map.Add(93, 262);
				Map.Add(88, 265);
				Map.Add(28, 266);
				Map.Add(28, 267);
				Map.Add(6, 281);
				Map.Add(60, 283);
				Map.Add(2, 284);
				Map.Add(2, 285);
				Map.Add(104, 288);
				Map.Add(104, 289);
				Map.Add(103, 291);
				Map.Add(5, 292);
				Map.Add(5, 293);
				Map.Add(6, 296);
				Map.Add(6, 297);
				Map.Add(0, 298);
				Map.Add(0, 299);
				Map.Add(5, 305);
				Map.Add(5, 306);
				Map.Add(103, 308);
				Map.Add(103, 309);
				Map.Add(104, 310);
				Map.Add(104, 311);
				Map.Add(3, 312);
				Map.Add(3, 315);
				Map.Add(2, 323);
				Map.Add(0, 324);
				Map.Add(0, 326);
				Map.Add(0, 327);
				Map.Add(0, 329);
				Map.Add(0, 331);
				Map.Add(2, 350);
				Map.Add(2, 351);
				Map.Add(60, 353);
				Map.Add(10, 355);
				Map.Add(102, 357);
				Map.Add(103, 359);
				Map.Add(6, 360);
				Map.Add(2, 362);
				Map.Add(2, 363);
				Map.Add(2, 364);
				Map.Add(2, 365);
				Map.Add(2, 366);
				Map.Add(2, 367);
				Map.Add(64, 369);
				Map.Add(64, 372);
				Map.Add(60, 376);
			}

			Map.Remove(70);
			Map.Add(120, 54);
			Map.Add(120, 30);
			Map.Remove(2);
			Map.Add(123, 366);
			Map.Add(123, 365);
			Map.Add(123, 364);
			Map.Add(123, 363);
			Map.Add(123, 362);
			Map.Add(123, 351);
			Map.Add(123, 350);
			Map.Add(123, 323);
			Map.Add(2, 285);
			Map.Add(123, 210);
			Map.Add(123, 107);
			Map.Add(123, 106);
			Map.Add(2, 94);
			Map.Add(123, 86);
			Map.Remove(120);
			Map.Remove(2);
			Map.Remove(123);
			Map.Add(129, 323);
			Map.Add(129, 350);
			Map.Add(129, 351);
			Map.Add(129, 362);
			Map.Add(129, 363);
			Map.Add(129, 364);
			Map.Add(129, 365);
			Map.Add(129, 366);
			Map.Remove(0);
			Map.Add(131, 326);
			Map.Add(131, 329);
			Map.Add(131, 331);
			Map.Add(131, 299);
			Map.Add(131, 298);
			Map.Add(131, 12);
			Map.Remove(129);
			Map.Add(132, 323);
			Map.Add(132, 350);
			Map.Add(132, 351);
			Map.Add(132, 364);
			Map.Remove(131);
			Map.Add(135, 326);
			Map.Add(135, 329);
			Map.Remove(132);
			Map.Add(132, 323);
			Map.Add(132, 350);
			Map.Remove(104);
			Map.Add(139, 311);
			Map.Add(138, 288);
			Map.Add(139, 207);
			Map.Add(138, 118);
			Map.Remove(132);
			Map.Remove(139);
			Map.Remove(138);
			Map.Remove(135);
			Map.Remove(6);
			Map.Add(149, 204);
			Map.Add(149, 10);
			Map.Remove(5);
			Map.Remove(149);
			Map.Remove(28);
			Map.Add(155, 266);
			Map.Add(28, 259);
			Map.Add(155, 258);
			Map.Add(28, 67);
			Map.Remove(88);
			Map.Add(88, 78);
			Map.Add(88, 4);
			Map.Remove(28);
			Map.Remove(88);
			Map.Remove(54);
			Map.Add(163, 248);
			Map.Add(162, 247);
			Map.Add(163, 246);
			Map.Add(162, 242);
			Map.Add(163, 235);
			Map.Add(163, 234);
			Map.Add(162, 229);
			Map.Add(162, 228);
			Map.Add(162, 191);
			Map.Add(162, 190);
			Map.Add(162, 177);
			Map.Add(162, 174);
			Map.Remove(10);
			Map.Add(10, 225);
			Map.Add(164, 355);
			Map.Add(164, 224);
			Map.Add(10, 223);
			Map.Remove(162);
			Map.Add(166, 229);
			Map.Add(166, 228);
			Map.Add(166, 242);
			Map.Add(166, 247);
			Map.Remove(164);
			Map.Remove(166);
			Map.Remove(163);
			Map.Remove(52);
			Map.Add(174, 219);
			Map.Add(174, 217);
			Map.Add(174, 216);
			Map.Add(174, 213);
			Map.Add(174, 212);
			Map.Add(174, 185);
			Map.Add(174, 184);
			Map.Add(174, 156);
			Map.Add(174, 153);
			Map.Add(174, 152);
			Map.Add(174, 151);
			Map.Add(174, 150);
			Map.Add(174, 149);
			Map.Add(174, 148);
			Map.Add(174, 147);
			Map.Add(174, 146);
			Map.Add(174, 145);
			Map.Add(174, 144);
			Map.Add(174, 143);
			Map.Add(174, 142);
			Map.Add(174, 141);
			Map.Add(174, 140);
			Map.Add(174, 139);
			Map.Add(174, 138);
			Map.Remove(60);
			Map.Add(177, 209);
			Map.Add(177, 353);
			Map.Add(177, 376);
			Map.Add(177, 121);
			Map.Remove(174);
			Map.Add(179, 143);
			Map.Add(174, 142);
			Map.Add(174, 141);
			Map.Add(174, 140);
			Map.Add(174, 139);
			Map.Add(179, 144);
			Map.Add(179, 145);
			Map.Add(179, 146);
			Map.Add(179, 147);
			Map.Add(174, 148);
			Map.Add(174, 149);
			Map.Add(174, 152);
			Map.Add(174, 153);
			Map.Add(179, 156);
			Map.Add(179, 184);
			Map.Add(179, 185);
			Map.Add(178, 212);
			Map.Add(178, 213);
			Map.Add(178, 216);
			Map.Add(178, 219);
			Map.Remove(102);
			Map.Add(181, 178);
			Map.Add(181, 158);
			Map.Remove(178);
			Map.Remove(179);
			Map.Add(184, 147);
			Map.Add(184, 146);
			Map.Add(184, 144);
			Map.Add(184, 143);
			Map.Remove(174);
			Map.Add(187, 149);
			Map.Add(187, 148);
			Map.Add(187, 141);
			Map.Add(187, 142);
			Map.Remove(184);
			Map.Remove(187);

			CHECK(Map.Contains(53));
			Map.Remove(53);
			CHECK(!Map.Contains(53));
		}
	}
};

TEST_CASE_NAMED(FSparseMapTest, "System::Core::Containers::SparseMap", "[Core][Containers][SparseMap]")
{
	TMapTestHelper<FSparseMapTestResolver>();
}

TEST_CASE_NAMED(FCompactMapTest, "System::Core::Containers::CompactMap", "[Core][Containers][CompactMap]")
{
	TMapTestHelper<FCompactMapTestResolver>();
}

#endif // WITH_TESTS
