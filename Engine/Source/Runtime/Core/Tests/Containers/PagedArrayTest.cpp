// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/PagedArray.h"
#include "CoreMinimal.h"
#include "Serialization/Formatters/BinaryArchiveFormatter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "TestUtils.h"
#include "TokenTest.h"

#include "Tests/TestHarnessAdapter.h"

namespace PagedArray
{
using TestArrayType = TPagedArray<int32Token, 128>;

//---------------------------------------------------------------------------------------------------------------------
// Forward declaration test
//---------------------------------------------------------------------------------------------------------------------

struct FSomeForwardDeclaredStruct;

struct FDummyTestStruct
{
	FDummyTestStruct();
	~FDummyTestStruct();
	TPagedArray<FSomeForwardDeclaredStruct> Data;
};

//---------------------------------------------------------------------------------------------------------------------
// Unit tests
//---------------------------------------------------------------------------------------------------------------------

TEST_CASE_NAMED(FPagedArrayReserveWithoutDataTest, "System::Core::Containers::TPagedArray::Reserve without data", "[SmokeFilter][Core][Containers][PagedArray]")
{
	int32Token::Reset();
	using ArrayType = TPagedArray<int32Token, 128>;
	ArrayType PagedArray;
	CHECK(!PagedArray.Max());
	CHECK(!PagedArray.Num());
	CHECK(!PagedArray.MaxPages());
	CHECK(!PagedArray.NumPages());
	CHECK(PagedArray.IsEmpty());
	PagedArray.Reserve(ArrayType::MaxPerPage());
	CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
	CHECK(!PagedArray.Num());
	CHECK(PagedArray.MaxPages() == 1);
	CHECK(!PagedArray.NumPages());
	CHECK(PagedArray.IsEmpty());
	CHECK(int32Token::EvenConstructionDestructionCalls(0));
}

TEST_CASE_NAMED(
	FPagedArrayReserveEmplaceSingleElementTest,
	"System::Core::Containers::TPagedArray::Reserve Emplace single element",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	int32Token::Reset();
	{
		using ArrayType = TPagedArray<int32Token, 128>;
		ArrayType PagedArray;
		CHECK(!PagedArray.Max());
		CHECK(!PagedArray.Num());

		PagedArray.Reserve(0);	// innocuous
		PagedArray.Reserve(1);
		PagedArray.Emplace(0);
		CHECK(PagedArray.Max() >= 1);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(PagedArray.Num() == 1);
		CHECK(PagedArray.MaxPages() == 1);
		CHECK(PagedArray.NumPages() == 1);

		PagedArray.Reserve(ArrayType::MaxPerPage());
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(PagedArray.Num() == 1);
		CHECK(PagedArray.MaxPages() == 1);
		CHECK(PagedArray.NumPages() == 1);

		PagedArray.Reserve(1 + ArrayType::MaxPerPage());
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		CHECK(PagedArray.Num() == 1);
		CHECK(PagedArray.MaxPages() == 2);
		CHECK(PagedArray.NumPages() == 1);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(1));
}

TEST_CASE_NAMED(
	FPagedArrayReserveEmplaceMultiplePagesTest,
	"System::Core::Containers::TPagedArray::Reserve Emplace multiple pages",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	int32Token::Reset();
	{
		using ArrayType = TPagedArray<int32Token, 128>;
		ArrayType PagedArray;
		CHECK(!PagedArray.Max());
		CHECK(!PagedArray.Num());

		PagedArray.Reserve(0);	// innocuous
		PagedArray.Reserve(1);
		PagedArray.Emplace(0);
		const int32Token* ElementAddress = &PagedArray[0];
	
		CHECK(PagedArray.Max() >= 1);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(PagedArray.Num() == 1);

		PagedArray.Reset();
		PagedArray.Reserve(ArrayType::MaxPerPage() + 1);
		PagedArray.Emplace(0);
		const int32Token* NewElementAddress = &PagedArray[0];

		CHECK(NewElementAddress == ElementAddress);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		CHECK(PagedArray.Num() == 1);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
}

TEST_CASE_NAMED(FPagedArrayAddTest, "System::Core::Containers::TPagedArray::Add", "[SmokeFilter][Core][Containers][PagedArray]")
{
	int32Token::Reset();
	{
		using ArrayType = TPagedArray<int32Token, 128>;
		ArrayType PagedArray;
		CHECK(!PagedArray.Max());
		CHECK(!PagedArray.Num());

		int32Token Temp(13);
		PagedArray.Add(Temp);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(PagedArray.Num() == 1);

		PagedArray.Add(3);
		CHECK(PagedArray.Num() == 2);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(4));
}

TEST_CASE_NAMED(
	FPagedArrayEmplaceMultipleElementsTest,
	"System::Core::Containers::TPagedArray::Emplace multiple elements",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	int32Token::Reset();
	{
		static_assert(sizeof(int32Token) < 16);
		using ArrayType = TPagedArray<int32Token, 16>;
		ArrayType PagedArray;
		CHECK(!PagedArray.Max());
		CHECK(!PagedArray.Num());

		const int32 MaxCount = 16 / sizeof(int32Token);
		int32 CheckValue = 0;
		while (CheckValue < MaxCount)
		{
			PagedArray.Emplace(CheckValue++);
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
			CHECK(PagedArray.Num() == CheckValue);
		}
		PagedArray.Emplace(CheckValue++);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		CHECK(PagedArray.Num() == CheckValue);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(5));
}

TEST_CASE_NAMED(
	FPagedArrayEmplaceVariantsTest,
	"System::Core::Containers::TPagedArray::Emplace variants",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		ArrayType PagedArray;
		int32Token* CheckAddress = nullptr;
		int32Token* ElementAddress = nullptr;
		int32 CheckIndex = INDEX_NONE;

		// These are compile-error checks: uncomment to test
		// PagedArray.Emplace_GetRef(); //nodiscard compile-error
		// PagedArray.Add_GetRef(); //nodiscard compile-error

		ElementAddress = &PagedArray.Emplace_GetRef(0);
		CheckAddress = &PagedArray[0];
		CHECK(ElementAddress == CheckAddress);
		CheckIndex = PagedArray.Emplace(13);
		CHECK(CheckIndex == PagedArray.Num() - 1);
		CHECK(PagedArray[CheckIndex] == 13);

		PagedArray.Reset(PagedArray.Max() * 3);
		ElementAddress = &PagedArray.Emplace_GetRef(0);
		CheckAddress = &PagedArray[0];
		CHECK(ElementAddress == CheckAddress);

		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			CheckIndex = PagedArray.Emplace(i + 1);
			CHECK(CheckIndex == PagedArray.Num() - 1);
			CHECK(PagedArray[CheckIndex] == i + 1);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() + 3));
}

TEST_CASE_NAMED(FPagedArrayResetTest, "System::Core::Containers::TPagedArray::Reset", "[SmokeFilter][Core][Containers][PagedArray]")
{
	int32Token::Reset();
	{
		using ArrayType = TPagedArray<int32Token, 128>;
		ArrayType PagedArray;
		PagedArray.Reset();	 // Should be innocuous
		CHECK(!PagedArray.Max());
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());

		PagedArray.Emplace(0);
		CHECK(PagedArray.Max());
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(PagedArray.Num() == 1);
		CHECK(!PagedArray.IsEmpty());

		PagedArray.Reset();
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());

		PagedArray.Reserve(ArrayType::MaxPerPage() * 6);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 6);
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());
		PagedArray.Reset();
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 6);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(1));
}

TEST_CASE_NAMED(
	FPagedArrayResetMultiplePageAddressCheckTest,
	"System::Core::Containers::TPagedArray::Reset multiple-page plus consistent address check",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	int32Token::Reset();
	{
		using ArrayType = TPagedArray<int32Token, 16>;
		ArrayType PagedArray = {0, 1, 2, 3, 4, 5, 6, 7, 8, 9};
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 3);

		TArray<const int32Token*> PtrAddresses;
		for (const int32Token& Value : PagedArray)
		{
			PtrAddresses.Emplace(&Value);
		}

		PagedArray.Reset();
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 3);

		// Validate same address after Reset plus re-adding the same elements
		PagedArray = {0, 1, 2, 3, 4, 5, 6, 7, 8, 9};
		int32 Index = 0;
		for (const int32Token& Value : PagedArray)
		{
			CHECK(PtrAddresses[Index++] == &Value);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(10 * 4));
}

TEST_CASE_NAMED(FPagedArrayEmptyTest, "System::Core::Containers::TPagedArray::Empty", "[SmokeFilter][Core][Containers][PagedArray]")
{
	int32Token::Reset();
	{
		using ArrayType = TPagedArray<int32Token, 128>;
		ArrayType PagedArray;
		PagedArray.Empty();	 // Should be innocuous
		CHECK(!PagedArray.Max());
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(0));
}

TEST_CASE_NAMED(
	FPagedArrayEmptyAfterSingleEmplaceTest,
	"System::Core::Containers::TPagedArray::Empty after single Emplace",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	int32Token::Reset();
	{
		using ArrayType = TPagedArray<int32Token, 128>;
		ArrayType PagedArray;
		PagedArray.Emplace(0);
		CHECK(PagedArray.Max());
		CHECK(PagedArray.Num() == 1);
		CHECK(!PagedArray.IsEmpty());
		PagedArray.Empty();
		CHECK(!PagedArray.Max());
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(1));
}

TEST_CASE_NAMED(
	FPagedArrayEmptyWithTargetCapacityTest,
	"System::Core::Containers::TPagedArray::Empty with target capacity",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;

	int32Token::Reset();
	{
		ArrayType PagedArray;
		PagedArray.Emplace(0);
		CHECK(PagedArray.Max());
		CHECK(PagedArray.Num() == 1);
		CHECK(!PagedArray.IsEmpty());
		
		PagedArray.Empty(ArrayType::MaxPerPage() + 1);
		CHECK(int32Token::EvenConstructionDestructionCalls(1));

		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());

		PagedArray.SetNum(ArrayType::MaxPerPage() * 3 + 1);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 4);
		
		PagedArray.Empty(ArrayType::MaxPerPage() + 1);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());

		PagedArray.Empty(ArrayType::MaxPerPage());
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());

		PagedArray.Empty(0);
		CHECK(!PagedArray.Max());
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 3 + 2));
}

TEST_CASE_NAMED(
	FPagedArrayEmptyVsResetWithTargetCapacityTest,
	"System::Core::Containers::TPagedArray::Empty vs Reset with target capacity",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;

	int32Token::Reset();
	{
		ArrayType PagedArray;
		
		// Requesting higher capacity through Reset or Empty have the same effect
		PagedArray.Emplace(0);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());

		PagedArray.Reset(ArrayType::MaxPerPage() + 1);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());

		PagedArray.Empty(ArrayType::MaxPerPage() + 1);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());

		// Requesting lower capacity through Reset or Empty DO NOT have the same effect: Empty shrinks
		PagedArray.SetNum(ArrayType::MaxPerPage() * 4);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 4);

		PagedArray.Reset(ArrayType::MaxPerPage() + 1);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 4);
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());

		PagedArray.Empty(ArrayType::MaxPerPage() + 1);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		CHECK(!PagedArray.Num());
		CHECK(PagedArray.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 4 + 1));
}

TEST_CASE_NAMED(FPagedArrayIsValidIndexTest, "System::Core::Containers::TPagedArray::IsValidIndex", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		ArrayType PagedArray;

		// Single-page
		PagedArray.Reserve(ArrayType::MaxPerPage());
		CHECK(!PagedArray.IsValidIndex(-1));
		CHECK(!PagedArray.IsValidIndex(0));
		CHECK(!PagedArray.IsValidIndex(ArrayType::MaxPerPage()));
		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			PagedArray.Emplace(i);
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		}
		for (int32 i = 0; i < PagedArray.Num(); ++i)
		{
			CHECK(PagedArray.IsValidIndex(i));
		}
		CHECK(!PagedArray.IsValidIndex(PagedArray.Num() + 1));

		// Multi-page
		PagedArray.Reserve(ArrayType::MaxPerPage() * 2);
		CHECK(!PagedArray.IsValidIndex(ArrayType::MaxPerPage() * 2));
		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			PagedArray.Emplace(i + ArrayType::MaxPerPage());
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		}
		for (int32 i = 0; i < PagedArray.Num(); ++i)
		{
			CHECK(PagedArray.IsValidIndex(i));
		}
		CHECK(!PagedArray.IsValidIndex(PagedArray.Num() + 1));
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 2));
}

TEST_CASE_NAMED(FPagedArrayIndexingTest, "System::Core::Containers::TPagedArray::Indexing", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		ArrayType PagedArray;
		// Single-page
		PagedArray.Reserve(ArrayType::MaxPerPage());
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			PagedArray.Emplace(i);
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		}
		for (int32 i = 0; i < PagedArray.Num(); ++i)
		{
			CHECK(PagedArray[i] == i);
		}
		// Multi-page
		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			PagedArray.Emplace(i + ArrayType::MaxPerPage());
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		}
		for (int32 i = 0; i < PagedArray.Num(); ++i)
		{
			CHECK(PagedArray[i] == i);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 2));
}

TEST_CASE_NAMED(FPagedArrayIterationTest, "System::Core::Containers::TPagedArray::Iteration", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		ArrayType PagedArray;
		// Single-page
		PagedArray.Reserve(ArrayType::MaxPerPage());
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			PagedArray.Emplace(i);
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		}
		int32 CheckValue = 0;
		for (const int32Token& Value : PagedArray)
		{
			CHECK(Value == CheckValue++);
		}
		// Multi-page
		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			PagedArray.Emplace(i + ArrayType::MaxPerPage());
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		}
		CheckValue = 0;
		for (const int32Token& Value : PagedArray)
		{
			CHECK(Value == CheckValue++);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 2));
}

TEST_CASE_NAMED(FPagedArrayGetLastTest, "System::Core::Containers::TPagedArray::Last", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		ArrayType PagedArray;
		for (int32 i = 0; i < ArrayType::MaxPerPage() * 2; ++i)
		{
			PagedArray.Emplace(i);
			CHECK(PagedArray.Last() == i);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 2));
}

TEST_CASE_NAMED(FPagedArraySetNumTest, "System::Core::Containers::TPagedArray::SetNum", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		ArrayType PagedArray;

		// Zero: should be innocuous
		PagedArray.SetNum(0);
		CHECK(!PagedArray.NumPages());
		CHECK(PagedArray.IsEmpty());

		// Single non-full page
		PagedArray.SetNum(ArrayType::MaxPerPage() - 1);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() - 1);
		CHECK(PagedArray.NumPages() == 1);
		CHECK(!PagedArray.IsEmpty());

		// Single full page
		PagedArray.SetNum(ArrayType::MaxPerPage());
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage());
		CHECK(PagedArray.NumPages() == 1);
		CHECK(!PagedArray.IsEmpty());

		// Full page plus 1
		PagedArray.SetNum(ArrayType::MaxPerPage() + 1);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() + 1);
		CHECK(PagedArray.NumPages() == 2);
		CHECK(!PagedArray.IsEmpty());

		// Back to single full page
		PagedArray.SetNum(ArrayType::MaxPerPage());
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage());
		CHECK(PagedArray.NumPages() == 1);
		CHECK(!PagedArray.IsEmpty());

		// Back to single non-full page
		PagedArray.SetNum(ArrayType::MaxPerPage() - 1);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() - 1);
		CHECK(PagedArray.NumPages() == 1);
		CHECK(!PagedArray.IsEmpty());

		// Back to zero
		PagedArray.SetNum(0);
		CHECK(!PagedArray.NumPages());
		CHECK(PagedArray.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() + 1));
	int32Token::Reset();
	{
		ArrayType PagedArray;

		// Add multiple pages in one go (non-full page)
		PagedArray.SetNum(ArrayType::MaxPerPage() * 4 - 1);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() * 4 - 1);
		CHECK(PagedArray.NumPages() == 4);
		CHECK(!PagedArray.IsEmpty());

		// Add multiple pages in one go (full page)
		PagedArray.SetNum(ArrayType::MaxPerPage() * 6);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() * 6);
		CHECK(PagedArray.NumPages() == 6);
		CHECK(!PagedArray.IsEmpty());

		// Verify content
		int32 CheckValue = 0;
		for (int32Token& Value : PagedArray)
		{
			Value = CheckValue++;
		}

		// Remove multiple pages in one go (non-full page)
		PagedArray.SetNum(ArrayType::MaxPerPage() * 3 - 1);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() * 3 - 1);
		CHECK(PagedArray.NumPages() == 3);
		CHECK(!PagedArray.IsEmpty());
		CheckValue = 0;
		for (const int32Token& Value : PagedArray)
		{
			CHECK(Value == CheckValue++);
		}

		// Remove multiple pages in one go (full page)
		PagedArray.SetNum(ArrayType::MaxPerPage());
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage());
		CHECK(PagedArray.NumPages() == 1);
		CHECK(!PagedArray.IsEmpty());
		CheckValue = 0;
		for (const int32Token& Value : PagedArray)
		{
			CHECK(Value == CheckValue++);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 6 * 2));
}

TEST_CASE_NAMED(FPagedArraySetNumAfterResetTest, "System::Core::Containers::TPagedArray::SetNum after Reset", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	{
		ArrayType PagedArray;

		PagedArray.SetNum(ArrayType::MaxPerPage() * 3);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() * 3);
		CHECK(PagedArray.NumPages() == 3);

		// Reset and set same number of elements
		PagedArray.Reset();
		CHECK(PagedArray.MaxPages() == 3);
		CHECK(!PagedArray.NumPages());
		PagedArray.SetNum(ArrayType::MaxPerPage() * 3);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() * 3);
		CHECK(PagedArray.NumPages() == 3);

		// Verify content
		for (const int32Token& Value : PagedArray)
		{
			CHECK(Value == 0);
		}

		// Write content memory
		for (int32Token& Value : PagedArray)
		{
			Value = 13;
		}

		// Reset and set a greater number of elements
		PagedArray.Reset();
		CHECK(PagedArray.MaxPages() == 3);
		CHECK(!PagedArray.NumPages());
		PagedArray.SetNum(ArrayType::MaxPerPage() * 5);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() * 5);
		CHECK(PagedArray.NumPages() == 5);

		// Verify content
		for (const int32Token& Value : PagedArray)
		{
			CHECK(Value == 0);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * (3 * 3 + 5)));
}

TEST_CASE_NAMED(
	FPagedArraySetNumSmallPageSizeTest,
	"System::Core::Containers::TPagedArray::SetNum small page size",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	{
		ArrayType PagedArray;

		// Add multiple pages in one go (non-full page)
		PagedArray.SetNum(ArrayType::MaxPerPage() * 4 - 1);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() * 4 - 1);
		CHECK(PagedArray.NumPages() == 4);
		CHECK(!PagedArray.IsEmpty());

		// Verify content
		int32 CheckValue = 0;
		for (int32Token& Value : PagedArray)
		{
			Value = CheckValue++;
		}

		// Add multiple pages in one go (full page)
		PagedArray.SetNum(ArrayType::MaxPerPage() * 6);
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage() * 6);
		CHECK(PagedArray.NumPages() == 6);
		CHECK(!PagedArray.IsEmpty());

		CheckValue = 0;
		for (int32 i = 0; i < ArrayType::MaxPerPage() * 4 - 1; ++i)
		{
			CHECK(PagedArray[i] == CheckValue++);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 10 - 1));
}

TEST_CASE_NAMED(FPagedArraySetZeroTest, "System::Core::Containers::TPagedArray::SetZero", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	{
		ArrayType PagedArray;

		// Single page
		{
			// Write content memory
			PagedArray.SetNum(ArrayType::MaxPerPage() - 1);
			for (int32Token& Value : PagedArray)
			{
				Value = 13;
			}
			PagedArray.SetZero();

			// Verify content
			CHECK(PagedArray.Num() == ArrayType::MaxPerPage() - 1);
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == 0);
			}
		}
		// Multiple pages
		{
			// Write content memory
			PagedArray.SetNum(ArrayType::MaxPerPage() * 3 + 1);	 // This adds 2 pages + 2 elements
			for (int32Token& Value : PagedArray)
			{
				Value = 13;
			}
			PagedArray.SetZero();

			// Verify content
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == 0);
			}

			// Verify content
			CHECK(PagedArray.Num() == ArrayType::MaxPerPage() * 3 + 1);
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == 0);
			}
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 7 + 1));
}

TEST_CASE_NAMED(FPagedArrayRemoveAtSwapTest, "System::Core::Containers::TPagedArray::RemoveAtSwap", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		// Single page
		ArrayType PagedArray;
		PagedArray.SetNum(ArrayType::MaxPerPage());
		int32 CheckValue = 0;
		for (int32Token& Value : PagedArray)
		{
			Value = CheckValue++;
		}

		while (PagedArray.Num() > 1)
		{
			const int32 Count = PagedArray.Num();
			const int32 Index = FMath::RandRange(0, Count - 1);
			const int32Token Last = PagedArray.Last();
			PagedArray.RemoveAtSwap(Index);
			CHECK(Count == PagedArray.Num() + 1);
			if (Index < PagedArray.Num())
			{
				CHECK(PagedArray[Index] == Last);
			}
		}

		// Test page shrinking after removing last element in page
		PagedArray.RemoveAtSwap(0, EAllowShrinking::No);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		PagedArray.Emplace(0);
		PagedArray.RemoveAtSwap(0);
		CHECK(!PagedArray.Max());
	}
	// One full page for SetNum, another for assignment and another minus one for RemoveAtSwap.
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 3));
	int32Token::Reset();
	{
		// Multiple pages
		ArrayType PagedArray;
		PagedArray.SetNum(ArrayType::MaxPerPage() * 3 + 1);
		int32 CheckValue = 0;
		for (int32Token& Value : PagedArray)
		{
			Value = CheckValue++;
		}

		while (PagedArray.Num() > 1)
		{
			const int32 Count = PagedArray.Num();
			const int32 Index = FMath::RandRange(0, Count - 1);
			const int32Token Last = PagedArray.Last();
			PagedArray.RemoveAtSwap(Index);
			CHECK(Count == PagedArray.Num() + 1);
			if (Index < PagedArray.Num())
			{
				CHECK(PagedArray[Index] == Last);
			}
		}
		PagedArray.RemoveAtSwap(0);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 9 + 2));
}

TEST_CASE_NAMED(FPagedArrayPopTest, "System::Core::Containers::TPagedArray::Pop", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		// Single page
		ArrayType PagedArray;
		PagedArray.Emplace(0);
		PagedArray.Pop();
		CHECK(PagedArray.IsEmpty());

		PagedArray.SetNum(ArrayType::MaxPerPage());
		int32 CheckValue = 0;
		for (int32Token& Value : PagedArray)
		{
			Value = CheckValue++;
		}
		PagedArray.Emplace(0);
		PagedArray.Pop();
		CHECK(PagedArray.Num() == ArrayType::MaxPerPage());

		CheckValue = 0;
		for (int32Token& Value : PagedArray)
		{
			CHECK(Value == CheckValue++);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 2 + 2));
}

TEST_CASE_NAMED(FPagedArrayCopyTest, "System::Core::Containers::TPagedArray::Copy", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		ArrayType PagedArray;

		// Single-page
		PagedArray.Reserve(ArrayType::MaxPerPage());
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			PagedArray.Emplace(i);
		}
		ArrayType OtherArray(PagedArray);
		CHECK(OtherArray.Num() == PagedArray.Num());
		for (int32 i = 0; i < OtherArray.Num(); ++i)
		{
			CHECK(OtherArray[i] == i);
		}
		ArrayType AnotherArray;
		CHECK(!AnotherArray.Num());
		AnotherArray = PagedArray;
		CHECK(AnotherArray.Num() == PagedArray.Num());
		for (int32 i = 0; i < AnotherArray.Num(); ++i)
		{
			CHECK(AnotherArray[i] == i);
		}

		// Multi-page
		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			PagedArray.Emplace(i + ArrayType::MaxPerPage());
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		}
		ArrayType YetAnotherArray(PagedArray);
		CHECK(YetAnotherArray.Num() == PagedArray.Num());
		for (int32 i = 0; i < YetAnotherArray.Num(); ++i)
		{
			CHECK(YetAnotherArray[i] == i);
		}

		AnotherArray = PagedArray;
		CHECK(AnotherArray.Num() == PagedArray.Num());
		for (int32 i = 0; i < AnotherArray.Num(); ++i)
		{
			CHECK(AnotherArray[i] == i);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 8));
}

TEST_CASE_NAMED(FPagedArrayMoveTest, "System::Core::Containers::TPagedArray::Move", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		TestArrayType PagedArray;

		// Single-page
		PagedArray.Reserve(ArrayType::MaxPerPage());
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			PagedArray.Emplace(i);
		}
		ArrayType OtherArray(MoveTemp(PagedArray));
		CHECK(OtherArray.Num() == ArrayType::MaxPerPage());
		CHECK(PagedArray.IsEmpty());
		for (int32 i = 0; i < OtherArray.Num(); ++i)
		{
			CHECK(OtherArray[i] == i);
		}
		ArrayType AnotherArray;
		CHECK(PagedArray.IsEmpty());
		AnotherArray = MoveTemp(OtherArray);
		CHECK(OtherArray.IsEmpty());
		CHECK(AnotherArray.Num() == ArrayType::MaxPerPage());
		for (int32 i = 0; i < AnotherArray.Num(); ++i)
		{
			CHECK(AnotherArray[i] == i);
		}

		// Multi-page
		for (int32 i = 0; i < ArrayType::MaxPerPage() * 3; ++i)
		{
			PagedArray.Emplace(i);
		}
		ArrayType YetAnotherArray(MoveTemp(PagedArray));
		CHECK(YetAnotherArray.Num() == ArrayType::MaxPerPage() * 3);
		CHECK(PagedArray.IsEmpty());
		for (int32 i = 0; i < YetAnotherArray.Num(); ++i)
		{
			CHECK(YetAnotherArray[i] == i);
		}

		AnotherArray = MoveTemp(YetAnotherArray);
		CHECK(YetAnotherArray.IsEmpty());
		CHECK(AnotherArray.Num() == ArrayType::MaxPerPage() * 3);
		for (int32 i = 0; i < AnotherArray.Num(); ++i)
		{
			CHECK(AnotherArray[i] == i);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 4));
}

TEST_CASE_NAMED(
	FPagedArrayConstructFromStdInitializerListTest,
	"System::Core::Containers::TPagedArray::Construct from std initializer_list",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	{
		// Single-page
		{
			ArrayType PagedArray({0, 1, 2, 3});
			CHECK(PagedArray.Num() == 4);
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
			int32 CheckValue = 0;
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(PagedArray.Num() == CheckValue);
		}

		// Multiple-page
		{
			ArrayType PagedArray({0, 1, 2, 3, 4, 5, 6, 7, 8, 9});
			CHECK(PagedArray.Num() == 10);
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 3);
			int32 CheckValue = 0;
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(PagedArray.Num() == CheckValue);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls((4 + 10) * 2));
}

TEST_CASE_NAMED(
	FPagedArrayAssignFromStdInitializerListTest,
	"System::Core::Containers::TPagedArray::Assign from std initializer_list",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	{
		// Single-page
		{
			ArrayType PagedArray;
			CHECK(PagedArray.IsEmpty());
			PagedArray = {0, 1, 2, 3};
			CHECK(PagedArray.Num() == 4);
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
			int32 CheckValue = 0;
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(PagedArray.Num() == CheckValue);
		}

		// Multiple-page
		{
			ArrayType PagedArray;
			CHECK(PagedArray.IsEmpty());
			PagedArray = {0, 1, 2, 3, 4, 5, 6, 7, 8, 9};
			CHECK(PagedArray.Num() == 10);
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 3);
			int32 CheckValue = 0;
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(PagedArray.Num() == CheckValue);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls((4 + 10) * 2));
}

TEST_CASE_NAMED(FPagedArrayComparisonTest, "System::Core::Containers::TPagedArray::Comparison", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 128>;
	int32Token::Reset();
	{
		ArrayType PagedArray, OtherPagedArray;
		CHECK(PagedArray == OtherPagedArray);

		for (int32 i = 0; i < ArrayType::MaxPerPage(); ++i)
		{
			PagedArray.Emplace(i);
			OtherPagedArray.Emplace(i);
		}
		CHECK(PagedArray == OtherPagedArray);

		// Verify diffirent content
		PagedArray.Emplace(0);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		CHECK(OtherPagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(PagedArray != OtherPagedArray);

		// Verify same content with same page allocation
		PagedArray.Pop();
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(OtherPagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(PagedArray == OtherPagedArray);

		// Verify same content with same  different page allocation
		PagedArray.Emplace(0);
		PagedArray.Pop(EAllowShrinking::No);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		CHECK(OtherPagedArray.Max() == ArrayType::MaxPerPage());
		CHECK(PagedArray == OtherPagedArray);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 2 + 2));
}

TEST_CASE_NAMED(FPagedArrayAssignFromArrayTest, "System::Core::Containers::TPagedArray::Assign from TArray", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	{
		// Single-page
		{
			TArray<int32Token> SourceArray({0, 1, 2, 3});

			ArrayType PagedArray;
			CHECK(PagedArray.IsEmpty());
			PagedArray.Assign(TArray<int32Token>{});  // innocuous.
			PagedArray.Assign(SourceArray);
			CHECK(PagedArray.Num() == 4);
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage());
			int32 CheckValue = 0;
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(PagedArray.Num() == CheckValue);
		}

		// Multiple-page
		{
			const TArray<int32Token> SourceArray({0, 1, 2, 3, 4, 5, 6, 7, 8, 9});

			ArrayType PagedArray;
			CHECK(PagedArray.IsEmpty());
			PagedArray.Assign(SourceArray);
			CHECK(PagedArray.Num() == 10);
			CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 3);
			int32 CheckValue = 0;
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(PagedArray.Num() == CheckValue);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls((4 + 10) * 3));
}

TEST_CASE_NAMED(FPagedArrayCopyToArrayTest, "System::Core::Containers::TPagedArray::Copy to TArray", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	{
		ArrayType PagedArray;
		TArray<int32Token> DestinationArray;

		// Empty copy
		PagedArray.ToArray(DestinationArray);  // innocuos
		CHECK(PagedArray.IsEmpty());
		CHECK(DestinationArray.IsEmpty());

		// Single full page copy
		PagedArray = {0, 1, 2, 3};
		PagedArray.ToArray(DestinationArray);
		CHECK(DestinationArray.Num() == 4);
		int32 CheckValue = 0;
		for (const int32Token& Value : DestinationArray)
		{
			CHECK(Value == CheckValue++);
		}

		// Single full page copy from multiple allocated pages
		PagedArray.Reserve(20);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 5);
		PagedArray.ToArray(DestinationArray);
		CHECK(DestinationArray.Num() == 4);
		CheckValue = 0;
		for (const int32Token& Value : DestinationArray)
		{
			CHECK(Value == CheckValue++);
		}

		// Multiple page copy
		PagedArray = {0, 1, 2, 3, 4, 5, 6, 7, 8, 9};
		PagedArray.ToArray(DestinationArray);
		CHECK(DestinationArray.Num() == 10);
		CheckValue = 0;
		for (const int32Token& Value : DestinationArray)
		{
			CHECK(Value == CheckValue++);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(4 * 4 + 10 * 3));
}

TEST_CASE_NAMED(FPagedArrayMoveToArrayTest, "System::Core::Containers::TPagedArray::Move to TArray", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	{
		ArrayType PagedArray;
		TArray<int32Token> DestinationArray;

		// Empty copy
		MoveTemp(PagedArray).ToArray(DestinationArray);	 // innocuos
		CHECK(PagedArray.IsEmpty());
		CHECK(DestinationArray.IsEmpty());

		// Single full page copy
		PagedArray = {0, 1, 2, 3};
		MoveTemp(PagedArray).ToArray(DestinationArray);
		CHECK(DestinationArray.Num() == 4);
		int32 CheckValue = 0;
		for (const int32Token& Value : DestinationArray)
		{
			CHECK(Value == CheckValue++);
		}
		CHECK(PagedArray.IsEmpty());

		// Single full page copy from multiple allocated pages
		PagedArray = {0, 1, 2, 3};
		PagedArray.Reserve(20);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 5);
		MoveTemp(PagedArray).ToArray(DestinationArray);
		CHECK(DestinationArray.Num() == 4);
		CheckValue = 0;
		for (const int32Token& Value : DestinationArray)
		{
			CHECK(Value == CheckValue++);
		}
		CHECK(PagedArray.IsEmpty());

		// Multiple page copy
		PagedArray = {0, 1, 2, 3, 4, 5, 6, 7, 8, 9};
		MoveTemp(PagedArray).ToArray(DestinationArray);
		CHECK(DestinationArray.Num() == 10);
		CheckValue = 0;
		for (const int32Token& Value : DestinationArray)
		{
			CHECK(Value == CheckValue++);
		}
		CHECK(PagedArray.IsEmpty());
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(4 * 4 + 10 * 2));
}

TEST_CASE_NAMED(
	FPagedArrayAssignFromCompatiblePagedArrayTest,
	"System::Core::Containers::TPagedArray::Assign from compatible paged array",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	using OtherArrayType = TPagedArray<int32Token, 32>;
	int32Token::Reset();
	{
		// Single-page smaller to larger page size
		{
			ArrayType PagedArray({0, 1, 2, 3});
			OtherArrayType OtherArray;
			OtherArray.Assign(PagedArray);
			CHECK(PagedArray.NumPages() == 1);
			CHECK(OtherArray.NumPages() == 1);

			CHECK(OtherArray.Num() == 4);
			int32 CheckValue = 0;
			for (const int32Token& Value : OtherArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(OtherArray.Num() == CheckValue);
		}
		// Single-page larger to smaller page size
		{
			OtherArrayType OtherArray({0, 1, 2, 3, 4, 5, 6, 7});
			ArrayType PagedArray;
			PagedArray.Assign(OtherArray);
			CHECK(PagedArray.NumPages() == 2);
			CHECK(OtherArray.NumPages() == 1);

			CHECK(OtherArray.Num() == 8);
			int32 CheckValue = 0;
			for (const int32Token& Value : OtherArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(OtherArray.Num() == CheckValue);
		}
		// Multiple-page smaller to larger page size
		{
			ArrayType PagedArray({0, 1, 2, 3, 4, 5, 6, 7});
			OtherArrayType OtherArray;
			OtherArray.Assign(PagedArray);
			CHECK(PagedArray.NumPages() == 2);
			CHECK(OtherArray.NumPages() == 1);

			CHECK(OtherArray.Num() == 8);
			int32 CheckValue = 0;
			for (const int32Token& Value : OtherArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(OtherArray.Num() == CheckValue);
		}
		// Multiple-page larger to smaller page size
		{
			OtherArrayType OtherArray({0, 1, 2, 3, 4, 5, 6, 7, 8, 9});
			ArrayType PagedArray;
			PagedArray.Assign(OtherArray);
			CHECK(PagedArray.NumPages() == 3);
			CHECK(OtherArray.NumPages() == 2);

			CHECK(OtherArray.Num() == 10);
			int32 CheckValue = 0;
			for (const int32Token& Value : OtherArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(OtherArray.Num() == CheckValue);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls((10 + 8 + 8 + 4) * 3));
}

TEST_CASE_NAMED(
	FPagedArrayAppendFromCompatiblePagedArrayTest,
	"System::Core::Containers::TPagedArray::Append from compatible paged array",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	using OtherArrayType = TPagedArray<int32Token, 32>;
	int32Token::Reset();
	{
		// Single-page smaller to larger page size
		{
			ArrayType PagedArray;
			const ArrayType SourceArray({0, 1, 2, 3});
			const OtherArrayType OtherSourceArray({4, 5, 6, 7});
			CHECK(SourceArray.NumPages() == 1);
			CHECK(OtherSourceArray.NumPages() == 1);

			PagedArray.Append(SourceArray);
			CHECK(PagedArray.NumPages() == 1);
			CHECK(PagedArray.Num() == 4);

			PagedArray.Append(OtherSourceArray);
			CHECK(PagedArray.NumPages() == 2);
			CHECK(PagedArray.Num() == 8);

			// Validate content
			int32 CheckValue = 0;
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(PagedArray.Num() == CheckValue);
		}
		// Single-page larger to smaller page size
		{
			OtherArrayType OtherArray;
			const ArrayType SourceArray({0, 1, 2, 3});
			const OtherArrayType OtherSourceArray({4, 5, 6, 7});
			CHECK(SourceArray.NumPages() == 1);
			CHECK(OtherSourceArray.NumPages() == 1);

			OtherArray.Append(SourceArray);
			CHECK(OtherArray.NumPages() == 1);
			CHECK(OtherArray.Num() == 4);

			OtherArray.Append(OtherSourceArray);
			CHECK(OtherArray.NumPages() == 1);
			CHECK(OtherArray.Num() == 8);

			// Validate content
			int32 CheckValue = 0;
			for (const int32Token& Value : OtherArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(OtherArray.Num() == CheckValue);
		}
		// Multiple-page smaller to larger page size
		{
			ArrayType PagedArray;
			const ArrayType SourceArray({0, 1, 2, 3, 4, 5, 6});
			const OtherArrayType OtherSourceArray({7, 8, 9, 10, 11, 12, 13, 14, 15});
			CHECK(SourceArray.NumPages() == 2);
			CHECK(OtherSourceArray.NumPages() == 2);

			PagedArray.Append(SourceArray);
			CHECK(PagedArray.NumPages() == 2);
			CHECK(PagedArray.Num() == 7);

			PagedArray.Append(OtherSourceArray);
			CHECK(PagedArray.NumPages() == 4);
			CHECK(PagedArray.Num() == 16);

			// Validate content
			int32 CheckValue = 0;
			for (const int32Token& Value : PagedArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(PagedArray.Num() == CheckValue);
		}
		// Multiple-page larger to smaller page size
		{
			OtherArrayType OtherArray;
			const ArrayType SourceArray({0, 1, 2, 3, 4, 5});
			const OtherArrayType OtherSourceArray({6, 7, 8, 9, 10, 11, 12, 13, 14, 15});
			CHECK(SourceArray.NumPages() == 2);
			CHECK(OtherSourceArray.NumPages() == 2);

			OtherArray.Append(SourceArray);
			CHECK(OtherArray.NumPages() == 1);
			CHECK(OtherArray.Num() == 6);

			OtherArray.Append(OtherSourceArray);
			CHECK(OtherArray.NumPages() == 2);
			CHECK(OtherArray.Num() == 16);

			// Validate content
			int32 CheckValue = 0;
			for (const int32Token& Value : OtherArray)
			{
				CHECK(Value == CheckValue++);
			}
			CHECK(OtherArray.Num() == CheckValue);
		}
	}
	CHECK(int32Token::EvenConstructionDestructionCalls((8 + 16) * 6));
}

TEST_CASE_NAMED(FPagedArrayAppendMultipleTest, "System::Core::Containers::TPagedArray::Append multiple", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	{
		ArrayType PagedArray;
		PagedArray.Append({0, 1});
		CHECK(PagedArray.NumPages() == 1);
		CHECK(PagedArray.Num() == 2);

		PagedArray.Append({2, 3});
		CHECK(PagedArray.NumPages() == 1);
		CHECK(PagedArray.Num() == 4);

		for (int32 i = 4; i < 8; ++i)
		{
			PagedArray.Append({i});
			CHECK(PagedArray.NumPages() == 2);
			CHECK(PagedArray.Num() == i + 1);
		}

		// Validate content
		int32 CheckValue = 0;
		for (const int32Token& Value : PagedArray)
		{
			CHECK(Value == CheckValue++);
		}
		CHECK(PagedArray.Num() == CheckValue);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(8 * 2));
}

TEST_CASE_NAMED(FPagedArrayIteratorArithmeticTest, "System::Core::Containers::TPagedArray::Iterator arithmetic", "[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	{
		ArrayType PagedArray;

		// Single page
		PagedArray.Emplace(13);
		PagedArray.Emplace(42);
		PagedArray.Emplace(19);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage());

		auto It = PagedArray.begin();
		CHECK(**It == 13);
		CHECK(*It == int32Token(13));
		CHECK(It->Value == 13);
		auto It2 = It++;
		CHECK(It2 != It);
		CHECK(It2->Value == 13);
		CHECK(**It == 42);
		CHECK(*It == int32Token(42));
		CHECK(It->Value == 42);

		PagedArray.Emplace(73);
		PagedArray.Emplace(171);
		CHECK(PagedArray.Max() == ArrayType::MaxPerPage() * 2);
		It = PagedArray.begin();
		It++;
		It++;
		It++;
		It2 = It++;
		CHECK(It2 != It);
		CHECK(It2->Value == 73);
		CHECK(**It == 171);
		CHECK(*It == int32Token(171));
		CHECK(It->Value == 171);
	}
	// 5 from container, 3 manual instantiations
	CHECK(int32Token::EvenConstructionDestructionCalls(5 + 3));
}

TEST_CASE_NAMED(
	FPagedArraySerializeTest,
	"System::Core::Containers::TPagedArray::Serialize",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	// Container with no pages
	{
		ArrayType SourcePagedArray;
		ArrayType DestPagedArray;

		CHECK(SourcePagedArray.IsEmpty());
		CHECK(DestPagedArray.IsEmpty());

		TArray<uint8> Bytes;
		{
			FMemoryWriter MemoryWriter(Bytes);
			MemoryWriter << SourcePagedArray;
		}

		// Just storing the count - no elements
		CHECK(Bytes.Num() == sizeof(ArrayType::SizeType));

		{
			FMemoryReader MemoryReader(Bytes);
			MemoryReader << DestPagedArray;
		}

		CHECK(SourcePagedArray == DestPagedArray);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(0));
	int32Token::Reset();
	// Container with single page
	{
		ArrayType SourcePagedArray;
		ArrayType DestPagedArray;

		SourcePagedArray.SetNum(ArrayType::MaxPerPage());
		int32 CheckValue = 0;
		for (int32Token& Value : SourcePagedArray)
		{
			Value = CheckValue++;
		}

		CHECK(SourcePagedArray.Num() == ArrayType::MaxPerPage());
		CHECK(DestPagedArray.IsEmpty());

		TArray<uint8> Bytes;
		{
			FMemoryWriter MemoryWriter(Bytes);
			MemoryWriter << SourcePagedArray;
		}

		// Storing the count and the elements
		CHECK(Bytes.Num() == (sizeof(ArrayType::SizeType) + (sizeof(ArrayType::ElementType) * ArrayType::MaxPerPage())));

		{
			FMemoryReader MemoryReader(Bytes);
			MemoryReader << DestPagedArray;
		}

		CHECK(SourcePagedArray == DestPagedArray);
	}
	// We construct 3 times over - SetNum, assigning CheckValue and then reading into the dest array
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 3));
	int32Token::Reset();
	// Container with multiple pages
	{
		ArrayType SourcePagedArray;
		ArrayType DestPagedArray;

		SourcePagedArray.SetNum(ArrayType::MaxPerPage() * 3);
		int32 CheckValue = 0;
		for (int32Token& Value : SourcePagedArray)
		{
			Value = CheckValue++;
		}

		CHECK(SourcePagedArray.Num() == ArrayType::MaxPerPage() * 3);
		CHECK(DestPagedArray.IsEmpty());

		TArray<uint8> Bytes;
		{
			FMemoryWriter MemoryWriter(Bytes);
			MemoryWriter << SourcePagedArray;
		}

		// Storing the count and the elements
		CHECK(Bytes.Num() == (sizeof(ArrayType::SizeType) + (sizeof(ArrayType::ElementType) * ArrayType::MaxPerPage() * 3)));

		{
			FMemoryReader MemoryReader(Bytes);
			MemoryReader << DestPagedArray;
		}

		CHECK(SourcePagedArray == DestPagedArray);
	}
	// We construct 3 times over - SetNum, assigning CheckValue and then reading into the dest array
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 3 * 3));
}

TEST_CASE_NAMED(
	FPagedArrayStructuredSerializeTest,
	"System::Core::Containers::TPagedArray::Structured Serialize",
	"[SmokeFilter][Core][Containers][PagedArray]")
{
	using ArrayType = TPagedArray<int32Token, 16>;
	int32Token::Reset();
	// Container with no pages
	{
		ArrayType SourcePagedArray;
		ArrayType DestPagedArray;

		CHECK(SourcePagedArray.IsEmpty());
		CHECK(DestPagedArray.IsEmpty());

		TArray<uint8> Bytes;
		{
			FMemoryWriter MemoryWriter(Bytes);
			FBinaryArchiveFormatter BinaryFormatter(MemoryWriter);
			FStructuredArchive StructuredArchive(BinaryFormatter);
			FStructuredArchive::FSlot RootSlot = StructuredArchive.Open();
			RootSlot << SourcePagedArray;
			StructuredArchive.Close();
		}

		// Just storing the count - no elements
		CHECK(Bytes.Num() == sizeof(ArrayType::SizeType));

		{
			FMemoryReader MemoryReader(Bytes);
			FBinaryArchiveFormatter BinaryFormatter(MemoryReader);
			FStructuredArchive StructuredArchive(BinaryFormatter);
			FStructuredArchive::FSlot RootSlot = StructuredArchive.Open();
			RootSlot << DestPagedArray;
			StructuredArchive.Close();
		}

		CHECK(SourcePagedArray == DestPagedArray);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(0));
	int32Token::Reset();
	// Container with single page
	{
		ArrayType SourcePagedArray;
		ArrayType DestPagedArray;

		SourcePagedArray.SetNum(ArrayType::MaxPerPage());
		int32 CheckValue = 0;
		for (int32Token& Value : SourcePagedArray)
		{
			Value = CheckValue++;
		}

		CHECK(SourcePagedArray.Num() == ArrayType::MaxPerPage());
		CHECK(DestPagedArray.IsEmpty());

		TArray<uint8> Bytes;
		{
			FMemoryWriter MemoryWriter(Bytes);
			FBinaryArchiveFormatter BinaryFormatter(MemoryWriter);
			FStructuredArchive StructuredArchive(BinaryFormatter);
			FStructuredArchive::FSlot RootSlot = StructuredArchive.Open();
			RootSlot << SourcePagedArray;
			StructuredArchive.Close();
		}

		// Storing the count and the elements
		CHECK(Bytes.Num() == (sizeof(ArrayType::SizeType) + (sizeof(ArrayType::ElementType) * ArrayType::MaxPerPage())));

		{
			FMemoryReader MemoryReader(Bytes);
			FBinaryArchiveFormatter BinaryFormatter(MemoryReader);
			FStructuredArchive StructuredArchive(BinaryFormatter);
			FStructuredArchive::FSlot RootSlot = StructuredArchive.Open();
			RootSlot << DestPagedArray;
			StructuredArchive.Close();
		}

		CHECK(SourcePagedArray == DestPagedArray);
	}
	// We construct 3 times over - SetNum, assigning CheckValue and then reading into the dest array
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 3));
	int32Token::Reset();
	// Container with multiple pages
	{
		ArrayType SourcePagedArray;
		ArrayType DestPagedArray;

		SourcePagedArray.SetNum(ArrayType::MaxPerPage() * 3);
		int32 CheckValue = 0;
		for (int32Token& Value : SourcePagedArray)
		{
			Value = CheckValue++;
		}

		CHECK(SourcePagedArray.Num() == ArrayType::MaxPerPage() * 3);
		CHECK(DestPagedArray.IsEmpty());

		TArray<uint8> Bytes;
		{
			FMemoryWriter MemoryWriter(Bytes);
			FBinaryArchiveFormatter BinaryFormatter(MemoryWriter);
			FStructuredArchive StructuredArchive(BinaryFormatter);
			FStructuredArchive::FSlot RootSlot = StructuredArchive.Open();
			RootSlot << SourcePagedArray;
			StructuredArchive.Close();
		}

		// Storing the count and the elements
		CHECK(Bytes.Num() == (sizeof(ArrayType::SizeType) + (sizeof(ArrayType::ElementType) * ArrayType::MaxPerPage() * 3)));

		{
			FMemoryReader MemoryReader(Bytes);
			FBinaryArchiveFormatter BinaryFormatter(MemoryReader);
			FStructuredArchive StructuredArchive(BinaryFormatter);
			FStructuredArchive::FSlot RootSlot = StructuredArchive.Open();
			RootSlot << DestPagedArray;
			StructuredArchive.Close();
		}

		CHECK(SourcePagedArray == DestPagedArray);
	}
	// We construct 3 times over - SetNum, assigning CheckValue and then reading into the dest array
	CHECK(int32Token::EvenConstructionDestructionCalls(ArrayType::MaxPerPage() * 3 * 3));
}

}  // namespace PagedArray

#endif
