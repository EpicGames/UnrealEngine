// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/RingBuffer.h"

#if WITH_TESTS

#include "Containers/UnrealString.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::Containers::RingBuffer::Test
{
int32 NumCallsOfResizeAllocation = 0;
}

template <uint32 NumInlineElements>
class TCoreTRingBufferTestInlineAllocator :
	public TInlineAllocator<NumInlineElements, FDefaultAllocator>
{
public:
	using SuperAllocator = TInlineAllocator<NumInlineElements, FDefaultAllocator>;
	template <typename ElementType>
	using SuperForElementType = typename SuperAllocator::template ForElementType<ElementType>;
	using SizeType = typename SuperAllocator::SizeType;

	template<typename ElementType>
	class ForElementType : public SuperForElementType<ElementType>
	{
	public:
		// Allocator API that is used and we need to test
		void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement)
		{
			++UE::Containers::RingBuffer::Test::NumCallsOfResizeAllocation;
			SuperForElementType<ElementType>::ResizeAllocation(CurrentNum, NewMax, NumBytesPerElement);
		}

		// Allocator API unused by TRingBuffer
		// SizeType CalculateSlackReserve(SizeType NumElements, SIZE_T NumBytesPerElement) const
		// SizeType CalculateSlackShrink(SizeType NumElements, SizeType NumAllocatedElements, SIZE_T NumBytesPerElement) const
		// SizeType CalculateSlackGrow(SizeType NumElements, SizeType NumAllocatedElements, SIZE_T NumBytesPerElement) const

		// Allocator API that is used but we don't need to test
		// void MoveToEmpty(ForElementType& Other)
		// FORCEINLINE ElementType* GetAllocation() const
		// SIZE_T GetAllocatedSize(SizeType NumAllocatedElements, SIZE_T NumBytesPerElement) const
		// bool HasAllocation() const
		// SizeType GetInitialCapacity() const
#if UE_ENABLE_ARRAY_SLACK_TRACKING
		// void SlackTrackerLogNum(SizeType NewNumUsed)
#endif
	};
};

class FRingBufferTest
{
public:
	static bool IsIntegerRange(const TRingBuffer<uint32>& Queue, uint32 Start, uint32 End, bool bForward = true)
	{
		if (Queue.Num() != End - Start)
		{
			return false;
		}

		// Peek elements in queue at given offset, peek from back to front
		for (int32 It = 0; It < Queue.Num(); ++It)
		{
			uint32 QueueValue = bForward ? Queue[It] : Queue[Queue.Num() - 1 - It];
			if (QueueValue != Start + It)
			{
				return false;
			}
		}

		return true;
	}

	struct Counter
	{
		Counter(uint32 InValue = 0x12345)
			:Value(InValue)
		{
			++NumVoid;
		}
		Counter(const Counter& Other)
			:Value(Other.Value)
		{
			++NumCopy;
		}
		Counter(Counter&& Other)
			:Value(Other.Value)
		{
			++NumMove;
		}
		~Counter()
		{
			++NumDestruct;
		}
		Counter& operator=(const Counter& Other)
		{
			Value = Other.Value;
			++NumCopy;
			return *this;
		}
		Counter& operator=(Counter&& Other)
		{
			Value = Other.Value;
			++NumMove;
			return *this;
		}

		operator uint32() const
		{
			return Value;
		}

		bool operator==(const Counter& Other) const
		{
			return Value == Other.Value;
		}
		static int NumVoid;
		static int NumCopy;
		static int NumMove;
		static int NumDestruct;
		static void Clear()
		{
			NumVoid = NumCopy = NumMove = NumDestruct = 0;
		}

		uint32 Value;
	};

	FRingBufferTest() = default;

	bool Test()
	{
		// Test empty
		{
			TRingBuffer<uint32> Q(0);

			CHECK_MESSAGE(TEXT("Test empty - IsEmpty"), Q.IsEmpty() == true);
			CHECK_MESSAGE(TEXT("Test empty - Size"), Q.Num() == 0);
			CHECK_MESSAGE(TEXT("Test empty - Capacity"), Q.Max() == 0);
			CHECK_MESSAGE(TEXT("Test empty - Iterator"), Q.begin() == Q.end());
			CHECK_MESSAGE(TEXT("Test empty - ConvertPointerToIndex"), Q.ConvertPointerToIndex(nullptr) == INDEX_NONE);
			CHECK_MESSAGE(TEXT("Test empty - ConvertPointerToIndex"), Q.ConvertPointerToIndex(reinterpret_cast<uint32*>(this)) == INDEX_NONE);
			Q.Trim();
			CHECK_MESSAGE(TEXT("Test Trim From empty - Size"), Q.Num() == 0);
			CHECK_MESSAGE(TEXT("Test Trim From empty - Capacity"), Q.Max() == 0);
			Q.Reset();
			CHECK_MESSAGE(TEXT("Test Reset From empty - Size"), Q.Num() == 0);
			CHECK_MESSAGE(TEXT("Test Reset From empty - Capacity"), Q.Max() == 0);
			Q.Empty(0);
			CHECK_MESSAGE(TEXT("Test Empty From empty - Size"), Q.Num() == 0);
			CHECK_MESSAGE(TEXT("Test Empty From empty - Capacity"), Q.Max() == 0);
			Q.PopFront(0);
			Q.Pop(0);
			CHECK_MESSAGE(TEXT("Test Pop on empty - Size"), Q.Num() == 0);
			CHECK_MESSAGE(TEXT("Test Pop on empty - Capacity"), Q.Max() == 0);
			CHECK_MESSAGE(TEXT("Test empty - IsValidIndex"), Q.IsValidIndex(0) == false);


			const TRingBuffer<uint32> ConstQ(0);
			CHECK_MESSAGE(TEXT("Test const empty - IsEmpty"), ConstQ.IsEmpty() == true);
			CHECK_MESSAGE(TEXT("Test const empty - Size"), ConstQ.Num() == 0);
			CHECK_MESSAGE(TEXT("Test const empty - Capacity"), ConstQ.Max() == 0);
			CHECK_MESSAGE(TEXT("Test const empty - Iterator"), ConstQ.begin() == ConstQ.end());
			CHECK_MESSAGE(TEXT("Test const empty - ConvertPointerToIndex"), ConstQ.ConvertPointerToIndex(reinterpret_cast<uint32*>(this)) == INDEX_NONE);
		}

		// Test Adding a sequence of elements
		{
			const TRingBuffer<int32>::IndexType FirstSize = 8;

			TRingBuffer<int32> Q(0);

			CHECK_MESSAGE(TEXT("Test AddSequence - Capacity (Implementation Detail)"), Q.Max() == 0);
			Q.Emplace(0);
			CHECK_MESSAGE(TEXT("Test AddSequence - Size"), Q.Num() == 1);
			CHECK_MESSAGE(TEXT("Test AddSequence - Capacity (Implementation Detail)"), Q.Max() == 1);
			Q.Emplace(1);
			CHECK_MESSAGE(TEXT("Test AddSequence - Size"), Q.Num() == 2);
			CHECK_MESSAGE(TEXT("Implementation Detail - These tests expect that growing size will set capacity to successive powers of 2."), Q.Max() == 2);
			for (int32 It = 2; It < FirstSize; ++It)
			{
				Q.Emplace(It);
				CHECK_MESSAGE(TEXT("Test AddSequence - Size"), Q.Num() == It + 1);
				CHECK_MESSAGE(TEXT("Test AddSequence - Capacity (Implementation Detail)"),
					static_cast<uint32>(Q.Max()) == FMath::RoundUpToPowerOfTwo(It + 1));
			}

			for (int32 Index = 0; Index < FirstSize; ++Index)
			{
				CHECK_MESSAGE(TEXT("Test AddSequence - Expected values"), Q[Index] == Index);
				CHECK_MESSAGE(TEXT("Test AddSequence const- Expected values"), const_cast<TRingBuffer<int32>&>(Q)[Index] == Index);
			}

			const TRingBuffer<int32>::IndexType SecondSize = 13;
			for (int32 It = FirstSize; It < SecondSize; ++It)
			{
				Q.Emplace(It);
				CHECK_MESSAGE(TEXT("Test AddSequence non powerof2 - Size"), Q.Num() == It + 1);
				CHECK_MESSAGE(TEXT("Test AddSequence non powerof2 const - Capacity (Implementation Detail)"),
					static_cast<uint32>(Q.Max()) == FMath::RoundUpToPowerOfTwo(It + 1));
			}

			for (int32 Index = 0; Index < FirstSize; ++Index)
			{
				CHECK_MESSAGE(TEXT("Test AddSequence non powerof2 - Expected values"), Q[Index] == Index);
				CHECK_MESSAGE(TEXT("Test AddSequence non powerof2 const - Expected values"), const_cast<TRingBuffer<int32>&>(Q)[Index] == Index);
			}
		}

		// Test Add under/over Capacity
		{
			const TRingBuffer<int32>::IndexType FirstElementsToAdd = 3;
			const TRingBuffer<int32>::IndexType InitialCapacity = 8;
			const TRingBuffer<int32>::IndexType SecondElementsToAdd = 9;

			TRingBuffer<int32> Q(InitialCapacity);

			for (int32 It = 0; It < FirstElementsToAdd; ++It)
			{
				Q.Emplace(It);
			}

			CHECK_MESSAGE(TEXT("Test Add under Capacity - Size"), Q.Num() == FirstElementsToAdd);
			CHECK_MESSAGE(TEXT("Test Add under Capacity - Capacity"), Q.Max() == InitialCapacity);
			for (int32 Index = 0; Index < FirstElementsToAdd; ++Index)
			{
				CHECK_MESSAGE(TEXT("Test Add under Capacity - Expected values"), Q[Index] == Index);
				CHECK_MESSAGE(TEXT("Test Add under Capacity const - Expected values"), const_cast<TRingBuffer<int32>&>(Q)[Index] == Index);
			}

			for (int32 It = FirstElementsToAdd; It < SecondElementsToAdd; ++It)
			{
				Q.Emplace(It);
			}

			CHECK_MESSAGE(TEXT("Test Add over Capacity - Size"), Q.Num() == SecondElementsToAdd);
			CHECK_MESSAGE(TEXT("Test Add over Capacity - Capacity (Implementation Detail)"),
				static_cast<uint32>(Q.Max()) == FMath::RoundUpToPowerOfTwo(SecondElementsToAdd));
			for (int32 Index = 0; Index < SecondElementsToAdd; ++Index)
			{
				CHECK_MESSAGE(TEXT("Test Add over Capacity - Expected values"), Q[Index] == Index);
				CHECK_MESSAGE(TEXT("Test Add over Capacity const - Expected values"), const_cast<TRingBuffer<int32>&>(Q)[Index] == Index);
			}
		}

		// Test Last/First
		{
			TRingBuffer<uint32> Q({ 0,1,2,3 });
			CHECK_MESSAGE(TEXT("Test Last"), 3 == Q.Last());
			Q.Last() = 4;
			CHECK_MESSAGE(TEXT("Test Last const"), 4 == const_cast<TRingBuffer<uint32>&>(Q).Last());
			CHECK_MESSAGE(TEXT("Test First"), 0 == Q.First());
			Q.First() = 5;
			CHECK_MESSAGE(TEXT("Test First const"), 5 == const_cast<TRingBuffer<uint32>&>(Q).First());
		}

		// Test PopFrontValue/PopValue
		{
			TRingBuffer<Counter> Q({ 31,32,33 });
			Q.AddFront(30);

			Counter::Clear();
			Counter C(Q.PopFrontValue());
			CHECK_MESSAGE(TEXT("PopFrontValue - PoppedValue"), C.Value == 30);
			CHECK_MESSAGE(TEXT("PopFrontValue - ConstructorCounts"), (Counter::NumMove > 0 && Counter::NumCopy == 0) == true);
			CHECK_MESSAGE(TEXT("PopFrontValue - Remaining Values"), Q == TRingBuffer<Counter>({ 31,32,33 }));
			Counter::Clear();
			CHECK_MESSAGE(TEXT("PopFrontValue Inline - PoppedValue"), Q.PopFrontValue().Value == 31);
			CHECK_MESSAGE(TEXT("PopFrontValue Inline - ConstructorCounts"), (Counter::NumCopy == 0) == true);
			CHECK_MESSAGE(TEXT("PopFrontValue Inline - Remaining Values"), Q == TRingBuffer<Counter>({ 32,33 }));

			Counter::Clear();
			Counter D(Q.PopValue());
			CHECK_MESSAGE(TEXT("PopValue - PoppedValue"), D.Value == 33);
			CHECK_MESSAGE(TEXT("PopValue - ConstructorCounts"), (Counter::NumMove > 0 && Counter::NumCopy == 0) == true);
			CHECK_MESSAGE(TEXT("PopValue - Remaining Values"), Q == TRingBuffer<Counter>({ Counter(32) }));
			Counter::Clear();
			CHECK_MESSAGE(TEXT("PopValue Inline - PoppedValue"), Q.PopValue().Value == 32);
			CHECK_MESSAGE(TEXT("PopValue Inline - ConstructorCounts"), (Counter::NumCopy == 0) == true);
			CHECK_MESSAGE(TEXT("PopValue Inline - Remaining Values"), Q.IsEmpty() == true);
		}

		// Test Initializer_List
		{
			const TRingBuffer<int32>::IndexType InitializerSize = 9;
			TRingBuffer<int32> Q({ 0, 1, 2, 3, 4, 5, 6, 7, 8 });

			CHECK_MESSAGE(TEXT("Test Initializer_List - Size"), Q.Num() == InitializerSize);
			CHECK_MESSAGE(TEXT("Test Initializer_List - Capacity (Implementation Detail)"),
				static_cast<uint32>(Q.Max()) == FMath::RoundUpToPowerOfTwo(InitializerSize));
			for (int32 Index = 0; Index < InitializerSize; ++Index)
			{
				CHECK_MESSAGE(TEXT("Test Initializer_List - Expected values"), Q[Index] == Index);
			}
		}

		// Test RingBuffer's Copy Constructors et al
		{
			TRingBuffer<uint32> Original({ 0,1,2,3,4,5,6,7 });
			TRingBuffer<uint32> Copy(Original);
			CHECK_MESSAGE(TEXT("Copy Constructor"), Original == Copy);
			TRingBuffer<uint32> Moved(MoveTemp(Copy));
			CHECK_MESSAGE(TEXT("Move Constructor"), Original == Moved);
			CHECK_MESSAGE(TEXT("Move Constructor did in fact move"), Copy.Max() == 0);
			TRingBuffer<uint32> AssignCopy;
			AssignCopy = Original;
			CHECK_MESSAGE(TEXT("Copy Assignment"), Original == AssignCopy);
			TRingBuffer<uint32> AssignMove;
			AssignMove = MoveTemp(AssignCopy);
			CHECK_MESSAGE(TEXT("Move Assignment"), Original == AssignMove);
			CHECK_MESSAGE(TEXT("Move Assignment did in fact move"), AssignCopy.Max() == 0);
		}

		// Test Equality 
		{
			auto TestEquality = [this](const TCHAR* Message, bool ExpectedEqual, const TRingBuffer<int32>& A, const TRingBuffer<int32>& B)
			{
				CHECK_MESSAGE(*FString::Printf(TEXT("Test equality - %s - A == B"), Message), (A == B) == ExpectedEqual);
				CHECK_MESSAGE(*FString::Printf(TEXT("Test equality - %s - B == A"), Message), (B == A) == ExpectedEqual);
				CHECK_MESSAGE(*FString::Printf(TEXT("Test equality - %s - A != B"), Message), (A != B) == !ExpectedEqual);
				CHECK_MESSAGE(*FString::Printf(TEXT("Test equality - %s - B != A"), Message), (B != A) == !ExpectedEqual);
			};

			TestEquality(TEXT("empty"), true, TRingBuffer<int32>(0), TRingBuffer<int32>(0));
			TestEquality(TEXT("empty different capacities"), true, TRingBuffer<int32>(0), TRingBuffer<int32>(8));
			TestEquality(TEXT("equal nonempty powerof2"), true, TRingBuffer<int32>({ 0, 1, 2, 3 }), TRingBuffer<int32>({ 0, 1, 2, 3 }));
			TestEquality(TEXT("equal nonempty nonpowerof2"), true, TRingBuffer<int32>({ 0, 1, 2, 3, 4, 5 }), TRingBuffer<int32>({ 0, 1, 2, 3, 4, 5 }));
			{
				TRingBuffer<int32> QNum6Cap16(16);
				for (int32 Index = 0; Index < 6; ++Index)
				{
					QNum6Cap16.Add(Index);
				}
				TestEquality(TEXT("equal nonempty different capacities"), true, QNum6Cap16, TRingBuffer<int32>({ 0, 1, 2, 3, 4, 5 }));
			}

			TestEquality(TEXT("empty to nonempty"), false, TRingBuffer<int32>(0), TRingBuffer<int32>({ 0, 1, 2, 3, 4, 5 }));
			TestEquality(TEXT("smaller size to bigger size"), false, TRingBuffer<int32>({ 0, 1, 2 }), TRingBuffer<int32>({ 0, 1, 2, 3, 4, 5 }));
			TestEquality(TEXT("same size different elements"), false, TRingBuffer<int32>({ 0, 1, 2 }), TRingBuffer<int32>({ 0, 1, 3 }));
			TestEquality(TEXT("same elements different order"), false, TRingBuffer<int32>({ 0, 1, 2 }), TRingBuffer<int32>({ 0, 2, 1 }));

			for (int HasPow2 = 0; HasPow2 < 2; ++HasPow2)
			{
				const int Count = HasPow2 ? 8 : 7;
				TRingBuffer<int32> Q0Pop;
				TRingBuffer<int32> Q1PopFront;
				TRingBuffer<int32> Q2PopFront;
				TRingBuffer<int32> Q1Pop;
				TRingBuffer<int32> Q2Pop;
				TRingBuffer<int32> Q2PopFront3Pop;
				Q1PopFront.Add(47);
				Q2PopFront.Add(576);
				Q2PopFront.Add(-5);
				Q2PopFront3Pop.Add(84);
				Q2PopFront3Pop.Add(1000);
				for (int Index = 0; Index < Count; ++Index)
				{
					Q0Pop.Add(Index);
					Q1PopFront.Add(Index);
					Q2PopFront.Add(Index);
					Q1Pop.Add(Index);
					Q2Pop.Add(Index);
					Q2PopFront3Pop.Add(Index);
				}
				Q1PopFront.PopFront();
				Q2PopFront.PopFront();
				Q2PopFront.PopFront();
				Q1Pop.Add(-18);
				Q1Pop.Pop();
				Q2Pop.Add(105);
				Q2Pop.Add(219);
				Q2Pop.Pop();
				Q2Pop.Pop();
				Q2PopFront3Pop.Add(456);
				Q2PopFront3Pop.Add(654);
				Q2PopFront3Pop.Add(8888888);
				Q2PopFront3Pop.PopFront();
				Q2PopFront3Pop.Pop();
				Q2PopFront3Pop.PopFront();
				Q2PopFront3Pop.Pop();
				Q2PopFront3Pop.Pop();

				const TCHAR* Names[] =
				{
					TEXT("Q0Pop"),
					TEXT("Q1PopFront"),
					TEXT("Q2PopFront"),
					TEXT("Q1Pop"),
					TEXT("Q2Pop"),
					TEXT("Q2PopFront3Pop"),
				};
				TRingBuffer<int32>* Pops[] =
				{
					&Q0Pop,
					&Q1PopFront,
					&Q2PopFront,
					&Q1Pop,
					&Q2Pop,
					&Q2PopFront3Pop
				};


				auto TestThesePops = [this, HasPow2, &TestEquality, &Names, &Pops](int TrialA, int TrialB)
				{
					TestEquality(*FString::Printf(TEXT("%s - %s - %s"), Names[TrialA], Names[TrialB], (HasPow2 ? TEXT("powerof2") : TEXT("nonpowerof2"))), true, *Pops[TrialA], *Pops[TrialB]);
				};

				for (int TrialA = 0; TrialA < UE_ARRAY_COUNT(Names); ++TrialA)
				{
					for (int TrialB = TrialA /* test each against itself as well */; TrialB < UE_ARRAY_COUNT(Names); ++TrialB)
					{
						TestThesePops(TrialA, TrialB);
					}
				}
			}
		}

		// Test Add and pop all
		for (int Direction = 0; Direction < 2; ++Direction)
		{
			bool bIsAddBack = Direction == 0;
			auto GetMessage = [&bIsAddBack](const TCHAR* Message)
			{
				return FString::Printf(TEXT("Test %s (%s)"), Message, (bIsAddBack ? TEXT("AddBack") : TEXT("AddFront")));
			};

			// Test Mixed Adds and Pops
			{
				const TRingBuffer<uint32>::IndexType ElementsToAdd = 256;
				const TRingBuffer<uint32>::IndexType ElementPopMod = 16;
				const TRingBuffer<uint32>::IndexType ExpectedSize = 256 - ElementPopMod;
				const TRingBuffer<uint32>::IndexType ExpectedCapacity = 256;

				TRingBuffer<uint32> Q(4);

				uint32 ExpectedPoppedValue = 0;
				for (uint32 It = 0; It < 256; ++It)
				{
					if (bIsAddBack)
					{
						Q.Add(It);
						CHECK_MESSAGE(*GetMessage(TEXT("Add and pop - Add")), It == Q[Q.Num() - 1]);
					}
					else
					{
						Q.AddFront(It);
						CHECK_MESSAGE(*GetMessage(TEXT("Add and pop - Add")), It == Q[0]);
					}

					if (It % ElementPopMod == 0)
					{
						uint32 PoppedValue;
						if (bIsAddBack)
						{
							PoppedValue = Q[0];
							Q.PopFront();
						}
						else
						{
							PoppedValue = Q[Q.Num() - 1];
							Q.Pop();
						}
						CHECK_MESSAGE(*GetMessage(TEXT("Add and pop - Pop")), ExpectedPoppedValue == PoppedValue);
						++ExpectedPoppedValue;
					}
				}

				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop - Size")), Q.Num() == ExpectedSize);
				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop - Capacity")), Q.Max() == ExpectedCapacity);
				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop - IntegerRange")),
					IsIntegerRange(Q, ExpectedPoppedValue, ExpectedPoppedValue + ExpectedSize, bIsAddBack) == true);
			}


			// Popping down to empty
			{
				const TRingBuffer<uint32>::IndexType ElementsToAdd = 256;

				TRingBuffer<uint32> Q(ElementsToAdd);

				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop all - IsEmpty before")), Q.IsEmpty() == true);
				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop all - Size before")), Q.Num() == 0);

				for (TRingBuffer<int32>::IndexType It = 0; It < ElementsToAdd; ++It)
				{
					if (bIsAddBack)
					{
						Q.Add(It);
					}
					else
					{
						Q.AddFront(It);
					}
				}

				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop all - Size")), Q.Num() == ElementsToAdd);
				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop all - Capacity")), Q.Max() == ElementsToAdd);
				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop all - Expected")), IsIntegerRange(Q, 0, ElementsToAdd, bIsAddBack) == true);

				for (TRingBuffer<int32>::IndexType It = 0; It < ElementsToAdd; ++It)
				{
					if (bIsAddBack)
					{
						Q.PopFront();
					}
					else
					{
						Q.Pop();
					}
				}

				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop all - IsEmpty after")), Q.IsEmpty() == true);
				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop all - Size after")), Q.Num() == 0);
				CHECK_MESSAGE(*GetMessage(TEXT("Add and pop all - Capacity after")), Q.Max() == ElementsToAdd);
			}

			// Test index wrap
			{
				for (int32 Offset : {-12, -8, -5, -1, 0, 2, 7, 8, 15})
				{
					const TRingBuffer<uint32>::IndexType ElementsToAdd = 256;
					const TRingBuffer<uint32>::IndexType ElementPopMod = 16;
					const TRingBuffer<uint32>::IndexType ExpectedSize = 256 - ElementPopMod;
					const TRingBuffer<uint32>::IndexType ExpectedCapacity = 256;

					TRingBuffer<uint32> Q(8);

					// Set front and afterback to an arbitrary offset
					// Note that AfterBack is always exactly equal to Front + Num()
					Q.Front = Offset;
					Q.AfterBack = Q.Front;

					CHECK_MESSAGE(*GetMessage(TEXT("index wrap - IsEmpty before")), Q.IsEmpty() == true);
					CHECK_MESSAGE(*GetMessage(TEXT("index wrap - Size before")), Q.Num() == 0);

					for (TRingBuffer<uint32>::IndexType It = 0; It < ElementsToAdd; ++It)
					{
						if (bIsAddBack)
						{
							Q.Add(It);
						}
						else
						{
							Q.AddFront(It);
						}
					}

					CHECK_MESSAGE(*GetMessage(TEXT("index wrap - Size")), Q.Num() == ElementsToAdd);
					CHECK_MESSAGE(*GetMessage(TEXT("index wrap - Capacity")), Q.Max() == ElementsToAdd);
					CHECK_MESSAGE(*GetMessage(TEXT("index wrap - Expected")), IsIntegerRange(Q, 0, ElementsToAdd, bIsAddBack) == true);

					for (TRingBuffer<int32>::IndexType It = 0; It < ElementsToAdd; ++It)
					{
						if (bIsAddBack)
						{
							Q.PopFront();
						}
						else
						{
							Q.Pop();
						}
					}

					CHECK_MESSAGE(*GetMessage(TEXT("index wrap - IsEmpty after")), Q.IsEmpty() == true);
					CHECK_MESSAGE(*GetMessage(TEXT("index wrap - Size after")), Q.Num() == 0);
					CHECK_MESSAGE(*GetMessage(TEXT("index wrap - Capacity after")), Q.Max() == ElementsToAdd);
				}
			}
		}

		// Test Trim
		{
			const TRingBuffer<int32>::IndexType ElementsToAdd = 9;
			const TRingBuffer<int32>::IndexType ElementsToPop = 5;
			const TRingBuffer<int32>::IndexType ExpectedCapacity = 16;
			const TRingBuffer<int32>::IndexType ExpectedCapacityAfterTrim = 4;

			TRingBuffer<uint32> Q(0);

			for (TRingBuffer<int32>::IndexType It = 0; It < ElementsToAdd; ++It)
			{
				Q.Add(It);
			}

			CHECK_MESSAGE(TEXT("Test Trim - Size"), Q.Num() == ElementsToAdd);
			CHECK_MESSAGE(TEXT("Test Trim - Capacity"), Q.Max() == ExpectedCapacity);
			CHECK_MESSAGE(TEXT("Test Trim - Expected"), IsIntegerRange(Q, 0, ElementsToAdd) == true);

			for (TRingBuffer<int32>::IndexType It = 0; It < ElementsToPop; ++It)
			{
				Q.PopFront();
			}

			Q.Trim();

			CHECK_MESSAGE(TEXT("Test Trim - Size"), Q.Num() == ElementsToAdd - ElementsToPop);
			CHECK_MESSAGE(TEXT("Test Trim - Capacity"), Q.Max() == ExpectedCapacityAfterTrim);
			CHECK_MESSAGE(TEXT("Test Trim - Expected"), IsIntegerRange(Q, ElementsToPop, ElementsToAdd) == true);
		}

		// Test First and Last acting as two stacks
		{
			TRingBuffer<uint32> Q;

			const uint32 ElementsToAdd = 64;
			const uint32 ElementPopMod = 5;

			for (uint32 It = 0; It < ElementsToAdd; ++It)
			{
				Q.Add(It);
				CHECK_MESSAGE(TEXT("Test TwoStacks - AddBack"), Q.Last() == It);
				Q.AddFront(It);
				CHECK_MESSAGE(TEXT("Test TwoStacks - AddFront"), Q.First() == It);
				if (It % ElementPopMod == 0)
				{
					uint32 AddValue = 0xfefefefe;
					Q.Add(AddValue);
					CHECK_MESSAGE(TEXT("Test TwoStacks - Sporadic Pop"), Q.Last() == AddValue);
					Q.Pop();
					Q.AddFront(AddValue);
					CHECK_MESSAGE(TEXT("Test TwoStacks - Sporadic PopFront"), Q.First() == AddValue);
					Q.PopFront();
				}
			}

			CHECK_MESSAGE(TEXT("Test TwoStacks - MiddleSize"), Q.Num() == ElementsToAdd * 2);
			for (uint32 It = 0; It < ElementsToAdd * 2; ++It)
			{
				CHECK_MESSAGE(*FString::Printf(TEXT("TwoStacks - Middle value %d"), It),
					Q[It] == (It < ElementsToAdd ? ElementsToAdd - 1 - It : It - ElementsToAdd));
			}

			for (uint32 It = 0; It < ElementsToAdd; ++It)
			{
				CHECK_MESSAGE(TEXT("Test TwoStacks - Final Pop"), Q.Last() == ElementsToAdd - 1 - It);
				Q.Pop();
				CHECK_MESSAGE(TEXT("Test TwoStacks - Final PopFront"), Q.First() == ElementsToAdd - 1 - It);
				Q.PopFront();
			}

			CHECK_MESSAGE(TEXT("Test TwoStacks - FinalSize"), Q.Num() == 0);
		}

		// Test adding into space that has been cleared from popping on the other side
		{
			for (int Direction = 0; Direction < 2; ++Direction)
			{
				bool bIsAddBack = Direction == 0;
				auto GetMessage = [bIsAddBack](const TCHAR* Message)
				{
					return FString::Printf(TEXT("Test AddIntoPop - %s (%s)"), Message, (bIsAddBack ? TEXT("AddBack") : TEXT("AddFront")));
				};
				TRingBuffer<uint32> Q({ 0,1,2,3,4,5,6,7 });
				TRingBuffer<int32>::IndexType InitialSize = 8;
				CHECK_MESSAGE(*GetMessage(TEXT("InitialSize")), InitialSize == Q.Num());
				CHECK_MESSAGE(*GetMessage(TEXT("InitialCapacity (Implementation Detail)")), InitialSize == Q.Max());

				if (bIsAddBack)
				{
					Q.Pop();
				}
				else
				{
					Q.PopFront();
				}
				CHECK_MESSAGE(*GetMessage(TEXT("PoppedSize")), InitialSize - 1 == Q.Num());
				CHECK_MESSAGE(*GetMessage(TEXT("PoppedCapacity")), InitialSize == Q.Max());

				if (bIsAddBack)
				{
					Q.AddFront(8);
				}
				else
				{
					Q.Add(8);
				}
				CHECK_MESSAGE(*GetMessage(TEXT("AddedSize")), InitialSize == Q.Num());
				CHECK_MESSAGE(*GetMessage(TEXT("AddedCapacity")), InitialSize == Q.Max());
				if (bIsAddBack)
				{
					CHECK_MESSAGE(*GetMessage(TEXT("AddedValues")), Q == TRingBuffer<uint32>({ 8,0,1,2,3,4,5,6 }));
				}
				else
				{
					CHECK_MESSAGE(*GetMessage(TEXT("AddedValues")), Q == TRingBuffer<uint32>({ 1,2,3,4,5,6,7,8 }));
				}

				if (bIsAddBack)
				{
					Q.AddFront(9);
				}
				else
				{
					Q.Add(9);
				}
				CHECK_MESSAGE(*GetMessage(TEXT("Second AddedSize")), InitialSize + 1 == Q.Num());
				CHECK_MESSAGE(*GetMessage(TEXT("Second AddedCapacity")), static_cast<uint32>(FMath::RoundUpToPowerOfTwo(InitialSize + 1)) == Q.Max());
				if (bIsAddBack)
				{
					CHECK_MESSAGE(*GetMessage(TEXT("Second AddedValues")), Q == TRingBuffer<uint32>({ 9,8,0,1,2,3,4,5,6 }));
				}
				else
				{
					CHECK_MESSAGE(*GetMessage(TEXT("Second AddedValues")), Q == TRingBuffer<uint32>({ 1,2,3,4,5,6,7,8,9 }));
				}
			}
		}

		// Test Empty to a capacity
		{
			TRingBuffer<uint32> Q(16);
			CHECK_MESSAGE(TEXT("Test EmptyToCapacity - InitialCapacity"), 16 == Q.Max());
			Q.Empty(8);
			CHECK_MESSAGE(TEXT("Test EmptyToCapacity - Lower"), 8 == Q.Max());
			Q.Empty(32);
			CHECK_MESSAGE(TEXT("Test EmptyToCapacity - Higher"), 32 == Q.Max());
		}

		// Test Different Add constructors
		{
			auto Clear = []()
			{
				Counter::Clear();
			};
			auto TestCounts = [this](const TCHAR* Message, int32 NumVoid, int32 NumCopy, int32 NumMove, int32 NumDestruct)
			{
				CHECK_MESSAGE(Message, (NumVoid == Counter::NumVoid && NumCopy == Counter::NumCopy && NumMove == Counter::NumMove && NumDestruct == Counter::NumDestruct) == true);
			};

			Clear();
			{
				TRingBuffer<Counter> QEmpty(4);
				QEmpty.Reserve(8);
				QEmpty.Empty();
				TRingBuffer<Counter> QEmpty2(4);
			}
			TestCounts(TEXT("Test Add Constructors - Unallocated elements call no constructors/destructors"), 0, 0, 0, 0);
			{
				TRingBuffer<Counter> QEmpty(4);
				QEmpty.Emplace();
				QEmpty.Pop();
				Clear();
			}
			TestCounts(TEXT("Test Add Constructors - Already removed element calls no destructors"), 0, 0, 0, 0);


			uint32 MarkerValue = 0x54321;
			Counter CounterA(MarkerValue);

			TRingBuffer<Counter> Q(4);
			Clear();
			for (int Direction = 0; Direction < 2; ++Direction)
			{
				bool bAddBack = Direction == 0;
				auto TestDirCounts = [this, bAddBack, &TestCounts, &Q, &Clear, MarkerValue](const TCHAR* Message, int32 NumVoid, int32 NumCopy, int32 NumMove, int32 NumDestruct, bool bWasInitialized = true)
				{
					const TCHAR* DirectionText = bAddBack ? TEXT("Back") : TEXT("Front");
					bool bElementExists = Q.Num() == 1;
					CHECK_MESSAGE(*FString::Printf(TEXT("Test Add Constructors - %s%s ElementExists"), Message, DirectionText), bElementExists == true);
					if (bWasInitialized && bElementExists)
					{
						CHECK_MESSAGE(*FString::Printf(TEXT("Test Add Constructors - %s%s ValueEquals"), Message, DirectionText),
							(Q.First().Value == MarkerValue) == true);
					}
					Q.PopFront();
					TestCounts(*FString::Printf(TEXT("Test Add Constructors - %s%s CountsEqual"), Message, DirectionText), NumVoid, NumCopy, NumMove, NumDestruct);
					Clear();
				};

				if (bAddBack) Q.Add(CounterA); else Q.AddFront(CounterA);
				TestDirCounts(TEXT("Copy Add"), 0, 1, 0, 1);
				if (bAddBack) (void)Q.Add_GetRef(CounterA); else (void)Q.AddFront_GetRef(CounterA);
				TestDirCounts(TEXT("Copy GetRef Add"), 0, 1, 0, 1);
				if (bAddBack) Q.Add(MoveTemp(CounterA)); else Q.AddFront(MoveTemp(CounterA));
				TestDirCounts(TEXT("Move Add"), 0, 0, 1, 1);
				if (bAddBack) (void)Q.Add_GetRef(MoveTemp(CounterA)); else (void)Q.AddFront_GetRef(MoveTemp(CounterA));
				TestDirCounts(TEXT("Move GetRef Add"), 0, 0, 1, 1);
				if (bAddBack) Q.Emplace(MarkerValue); else Q.EmplaceFront(MarkerValue);
				TestDirCounts(TEXT("Emplace"), 1, 0, 0, 1);
				if (bAddBack) (void)Q.Emplace_GetRef(MarkerValue); else (void)Q.EmplaceFront_GetRef(MarkerValue);
				TestDirCounts(TEXT("GetRef Emplace"), 1, 0, 0, 1);
				if (bAddBack) Q.AddUninitialized(); else Q.AddFrontUninitialized();
				TestDirCounts(TEXT("Uninitialized Add"), 0, 0, 0, 1, false);
				if (bAddBack) (void)Q.AddUninitialized_GetRef(); else (void)Q.AddFrontUninitialized_GetRef();
				TestDirCounts(TEXT("Uninitialized GetRef Add"), 0, 0, 0, 1, false);
			}
		}

		// Test MoveAppendRange
		{
			constexpr uint32 MarkerValue = 0x54321;
			constexpr int32 SourceNum = 4;
			Counter SourceRange[SourceNum];
			auto Clear = [MarkerValue, &SourceRange]()
			{
				int32 Index = 0;
				for (Counter& Element : SourceRange)
				{
					Element = Counter(MarkerValue + Index++);
				}
				Counter::Clear();
			};
			auto TestResults = [this, MarkerValue, SourceNum](const TCHAR* HeaderMessage, TRingBuffer<Counter>& Q, TArrayView<Counter> Prefix,
				int32 NumVoid, int32 NumCopy, int32 NumMove, int32 NumDestruct)
			{
				if (!(NumVoid == Counter::NumVoid && NumCopy == Counter::NumCopy && NumMove == Counter::NumMove && NumDestruct == Counter::NumDestruct))
				{
					CHECK_MESSAGE(*FString::Printf(TEXT("%s: Constructor/destructor callcounts do not match"), HeaderMessage), false);
				}

				int32 SourceStart = Prefix.Num();
				int32 ExpectedNum = SourceNum + SourceStart;
				if (Q.Num() != ExpectedNum)
				{
					CHECK_MESSAGE(*FString::Printf(TEXT("%s: Length does not match"), HeaderMessage), false);
				}
				else
				{
					for (int32 Index = 0; Index < ExpectedNum; ++Index)
					{
						if ((Index < SourceStart && Q[Index] != Prefix[Index]) ||
							(Index >= SourceStart && (uint32)Q[Index] != MarkerValue + Index - SourceStart))
						{
							CHECK_MESSAGE(*FString::Printf(TEXT("%s: Element does not match"), HeaderMessage), false);
							break;
						}
					}
				}
			};

			{
				Clear();
				TRingBuffer<Counter> Q;
				Q.MoveAppendRange(SourceRange, SourceNum);
				TestResults(TEXT("MoveAppendRange from empty"),
					Q, TArrayView<Counter>(), 0, 0, SourceNum, 0);
			}

			{
				Clear();
				TRingBuffer<Counter> Q;
				Q.Reserve(SourceNum*2);
				Q.Front = Q.IndexMask + 1 - SourceNum/2;
				Q.AfterBack = Q.Front;
				Q.MoveAppendRange(SourceRange, SourceNum);
				TestResults(TEXT("MoveAppendRange from empty with wraparound"),
					Q, TArrayView<Counter>(), 0, 0, SourceNum, 0);
			}

			{
				Clear();
				TRingBuffer<Counter> Q;
				Q.Reserve(SourceNum*2);
				Q.Front = Q.IndexMask + 1 - SourceNum;
				Q.AfterBack = Q.Front;
				Q.MoveAppendRange(SourceRange, SourceNum);
				TestResults(TEXT("MoveAppendRange from empty with AfterBack + OtherNum at end"),
					Q, TArrayView<Counter>(), 0, 0, SourceNum, 0);
			}

			{
				TRingBuffer<Counter> Q;
				Counter Prefix[] = { 1,2,3 };
				Q.Reserve(UE_ARRAY_COUNT(Prefix) + SourceNum);
				for (Counter& C : Prefix)
				{
					Q.Add(C);
				}
				Clear();
				Q.MoveAppendRange(SourceRange, SourceNum);
				TestResults(TEXT("MoveAppendRange with existing elements"),
					Q, Prefix, 0, 0, SourceNum, 0);
			}

			{
				TRingBuffer<Counter> Q;
				Counter Prefix[] = { 1,2,3 };
				Q.Reserve(SourceNum * 2 + UE_ARRAY_COUNT(Prefix));
				Q.Front = Q.IndexMask + 1 - SourceNum / 2 - UE_ARRAY_COUNT(Prefix);
				Q.AfterBack = Q.Front;
				for (Counter& C : Prefix)
				{
					Q.Add(C);
				}
				Clear();
				Q.MoveAppendRange(SourceRange, SourceNum);
				TestResults(TEXT("MoveAppendRange with existing elements and wraparound"),
					Q, Prefix, 0, 0, SourceNum, 0);
			}

			{
				TRingBuffer<Counter> Q;
				Counter Prefix[] = { 1,2,3 };
				Q.Reserve(UE_ARRAY_COUNT(Prefix) + SourceNum);
				Q.Front = Q.IndexMask;
				Q.AfterBack = Q.Front;
				for (Counter& C : Prefix)
				{
					Q.Add(C);
				}
				Clear();
				Q.MoveAppendRange(SourceRange, SourceNum);
				TestResults(TEXT("MoveAppendRange with existing elements and AfterBackMasked < FrontMasked"),
					Q, Prefix, 0, 0, SourceNum, 0);
			}
		}

		TestShiftIndex<uint32>();
		TestShiftIndex<Counter>();

		// Test RemoveAt
		{
			{
				TRingBuffer<uint32> Q{ 0,1,2,3,4,5,6,7 };
				Q.RemoveAt(2);
				CHECK_MESSAGE(TEXT("Test RemoveAt Front Closest"), TRingBuffer<uint32>({ 0,1,3,4,5,6,7 }) == Q);
			}
			{
				TRingBuffer<uint32> Q{ 0,1,2,3,4,5,6,7 };
				Q.RemoveAt(5);
				CHECK_MESSAGE(TEXT("Test RemoveAt Back Closest"), TRingBuffer<uint32>({ 0,1,2,3,4,6,7 }) == Q);
			}
			{
				TRingBuffer<uint32> Q{ 0,1,2,3,4,5,6,7 };
				int32 Offset = 4;
				Q.Front += Offset;
				Q.AfterBack += Offset;
				//Now equal to: TRingBuffer<uint32> Q{ 4,5,6,7,0,1,2,3 };
				Q.RemoveAt(2);
				CHECK_MESSAGE(TEXT("Test RemoveAt Front Closest With Offset"), TRingBuffer<uint32>({ 4,5,7,0,1,2,3 }) == Q);
			}
			{
				TRingBuffer<uint32> Q{ 0,1,2,3,4,5,6,7 };
				int32 Offset = 4;
				Q.Front += Offset;
				Q.AfterBack += Offset;
				//Now equal to: TRingBuffer<uint32> Q{ 4,5,6,7,0,1,2,3 };
				Q.RemoveAt(5);
				CHECK_MESSAGE(TEXT("Test RemoveAt Back Closest With Offset"), TRingBuffer<uint32>({ 4,5,6,7,0,2,3 }) == Q);
			}
		}

		// Test Iteration
		{
			{
				TRingBuffer<uint32> Q{ 0,1,2,3,4,5,6,7 };
				uint32 Counter = 0;
				for (uint32 Value : Q)
				{
					CHECK_MESSAGE(TEXT("Test Iteration - Value"), Counter++ == Value);
				}
				CHECK_MESSAGE(TEXT("Test Iteration - Num"), Counter == 8);
			}
			{
				TRingBuffer<uint32> Q{ 4,5,6,7,0,1,2,3 };
				int32 Offset = 4;
				Q.Front += Offset;
				Q.AfterBack += Offset;
				// Now equal to 0,1,2,3,4,5,6,7
				uint32 Counter = 0;
				for (uint32 Value : Q)
				{
					CHECK_MESSAGE(TEXT("Test Iteration with Offset - Value"), Counter++ == Value);
				}
				CHECK_MESSAGE(TEXT("Test Iteration with Offset  - Num"), Counter == 8);
			}
		}

		// Test ConvertPointerToIndex
		{
			{
				TRingBuffer<uint32> Q{ 4,5,6,7,0,1,2,3 };
				int32 Offset = 4;
				Q.Front += Offset;
				Q.AfterBack += Offset;
				// Now equal to 0,1,2,3,4,5,6,7
				CHECK_MESSAGE(TEXT("Test ConvertPointerToIndex - before array"), Q.ConvertPointerToIndex(&Q[0] - 100) == INDEX_NONE);
				CHECK_MESSAGE(TEXT("Test ConvertPointerToIndex - after array"), Q.ConvertPointerToIndex(&Q[0] + 100) == INDEX_NONE);
				for (int32 It = 0; It < 8; ++It)
				{
					CHECK_MESSAGE(TEXT("Test ConvertPointerToIndex - Values"), Q.ConvertPointerToIndex(&Q[It]) == It);
				}
			}

			{
				TRingBuffer<uint32> Q(16);
				for (int32 It = 7; It >= 0; --It)
				{
					Q.AddFront(It);
				}
				Q.Pop();
				// 8 Invalids, followed by 0,1,2,3,4,5,6, followed by Invalid
				for (int32 It = 0; It < 7; ++It)
				{
					CHECK_MESSAGE(TEXT("Test ConvertPointerToIndex - Cap - Values"), Q.ConvertPointerToIndex(&Q[It]) == It);
				}
				CHECK_MESSAGE(TEXT("Test ConvertPointerToIndex - Cap - After End"), Q.ConvertPointerToIndex(&Q[6] + 1) == INDEX_NONE);
				CHECK_MESSAGE(TEXT("Test ConvertPointerToIndex - Cap - Before Start"), Q.ConvertPointerToIndex(&Q[0] - 1) == INDEX_NONE);
			}
		}

		// Test that setting Front to its maximum value and then popping the maximum number of elements does not break the contract that Front < capacity in StorageModulo space
		{
			TRingBuffer<uint32> Q(8);
			Q.AddFront(0);
			for (uint32 It = 1; It < 8; ++It)
			{
				Q.Add(It);
			}
			CHECK_MESSAGE(TEXT("Test Front<Capacity - Setup"), ((Q.Front & Q.IndexMask) == Q.IndexMask && Q.Num() == Q.Max()) == true);
			Q.PopFront(8);
			CHECK_MESSAGE(TEXT("Test Front<Capacity - Contract is true"), (static_cast<uint32>(Q.Front) < static_cast<uint32>(Q.Max())) == true);
		}

		// Test IsValidIndex
		{
			TRingBuffer<uint32> Q({ 0,1,2,3,4 });
			for (int32 It = 0; It < Q.Num(); ++It)
			{
				CHECK_MESSAGE(TEXT("IsValidIndex - InRange"), Q.IsValidIndex(It) == true);
			}
			CHECK_MESSAGE(TEXT("IsValidIndex - Negative"), Q.IsValidIndex(-1) == false);
			CHECK_MESSAGE(TEXT("IsValidIndex - Num()"), Q.IsValidIndex(Q.Num() + 1) == false);
			CHECK_MESSAGE(TEXT("IsValidIndex - Capacity"), Q.IsValidIndex(Q.Max()) == false);
			CHECK_MESSAGE(TEXT("IsValidIndex - Capacity + 1"), Q.IsValidIndex(Q.Max() + 1) == false);
		}

		// Test Compact
		{
			{
				TRingBuffer<uint32> QEmpty;
				CHECK_MESSAGE(TEXT("Compact - Empty zero capacity"), QEmpty.Compact().Num() == 0);
				QEmpty.Add(1);
				QEmpty.PopFront();
				CHECK_MESSAGE(TEXT("Compact - Empty non-zero capacity"), QEmpty.Compact().Num() == 0);
			}
			{
				TArrayView<uint32> View;
				TRingBuffer<uint32> Q(8);
				Q.AddFront(37);
				View = Q.Compact();
				CHECK_MESSAGE(TEXT("Compact - Front at end"), ArrayViewsEqual(View, TArrayView<const uint32>({ 37 })) == true);
			}
			{
				TArrayView<uint32> View;
				TRingBuffer<uint32> Q(8);
				for (TRingBuffer<uint32>::IndexType It = 0; It < 6; ++It)
				{
					Q.Add(It);
				}
				Q.PopFront();
				TRingBuffer<uint32>::StorageModuloType SavedFront = Q.Front;
				CHECK_MESSAGE(TEXT("Compact - Front in middle - setup"), (SavedFront > 0) == true);
				View = Q.Compact();
				CHECK_MESSAGE(TEXT("Compact - Front in middle - values"), ArrayViewsEqual(View, TArrayView<const uint32>({ 1,2,3,4,5 })) == true);
				CHECK_MESSAGE(TEXT("Compact - Front in middle - no reallocate"), (Q.Front == SavedFront) == true);
			}
			{
				TArrayView<uint32> View;
				TRingBuffer<uint32> Q(8);
				for (TRingBuffer<uint32>::IndexType It = 1; It < 8; ++It)
				{
					Q.Add(It);
				}
				Q.AddFront(0);
				CHECK_MESSAGE(TEXT("Compact - Full array front at end - setup"), ((Q.Front & Q.IndexMask) == 7) == true);
				View = Q.Compact();
				CHECK_MESSAGE(TEXT("Compact - Full array front at end - values"),
					ArrayViewsEqual(View, TArrayView<const uint32>({ 0,1,2,3,4,5,6,7 })) == true);
				CHECK_MESSAGE(TEXT("Compact - Full array front at end - reallocated"), (Q.Front == 0) == true);
			}
			{
				TArrayView<uint32> View;
				TRingBuffer<uint32> Q(8);
				for (TRingBuffer<uint32>::IndexType It = 0; It < 8; ++It)
				{
					Q.Add(It);
				}
				uint32* SavedData = Q.GetStorage();
				CHECK_MESSAGE(TEXT("Compact - Full array front at start - setup"), (Q.Front == 0) == true);
				View = Q.Compact();
				CHECK_MESSAGE(TEXT("Compact - Full array front at start - values"),
					ArrayViewsEqual(View, TArrayView<const uint32>({ 0,1,2,3,4,5,6,7 })) == true);
				CHECK_MESSAGE(TEXT("Compact - Full array front at start - no reallocate"), (Q.GetStorage() == SavedData) == true);
			}
		}

		// Test Remove
		{
			Counter Value;
			{
				TRingBuffer<Counter> Q;
				Value.Value = 2;
				Counter::Clear();
				CHECK_MESSAGE(TEXT("Remove - empty"), Q.Remove(Value) == 0);
				CHECK_MESSAGE(TEXT("Remove - empty - destructor count"), Counter::NumDestruct == 0);
			}
			{
				TRingBuffer<Counter> Q({ 0,1,2,3,4 });
				Value.Value = 5;
				Counter::Clear();
				CHECK_MESSAGE(TEXT("Remove - no hits"), Q.Remove(Value) == 0);
				CHECK_MESSAGE(TEXT("Remove - no hits - destructor count"), Counter::NumDestruct == 0);
				Q.Add(5);
				CHECK_MESSAGE(TEXT("Remove - no hits - values"), (Q == TRingBuffer<Counter>({ 0,1,2,3,4,5 })) == true);
			}
			{
				TRingBuffer<Counter> Q({ 1,2,3,4 });
				Q.AddFront(0);
				Value.Value = 0;
				Counter::Clear();
				CHECK_MESSAGE(TEXT("Remove - one element at front - num"), Q.Remove(Value) == 1);
				CHECK_MESSAGE(TEXT("Remove - one element at front - destructor count"), Counter::NumDestruct == 5);
				Q.Add(5);
				CHECK_MESSAGE(TEXT("Remove - one element at front - values"), (Q == TRingBuffer<Counter>({ 1,2,3,4,5 })) == true);
			}
			{
				TRingBuffer<Counter> Q({ 0,1,2,3,4 });
				Value.Value = 2;
				Counter::Clear();
				CHECK_MESSAGE(TEXT("Remove - one element in mid - num"), Q.Remove(Value) == 1);
				CHECK_MESSAGE(TEXT("Remove - one element in mid - destructor count"), Counter::NumDestruct == 3);
				Q.Add(5);
				CHECK_MESSAGE(TEXT("Remove - one element in mid - values"), (Q == TRingBuffer<Counter>({ 0,1,3,4,5 })) == true);
			}
			{
				TRingBuffer<Counter> Q({ 1,2,3,4 });
				Q.AddFront(0);
				Value.Value = 2;
				Counter::Clear();
				CHECK_MESSAGE(TEXT("Remove - one element in mid - front at end"), Q.Remove(Value) == 1);
				CHECK_MESSAGE(TEXT("Remove - one element in mid - front at end - destructor count"), Counter::NumDestruct == 3);
				Q.Add(5);
				CHECK_MESSAGE(TEXT("Remove - one element in mid - front at end - values"), (Q == TRingBuffer<Counter>({ 0,1,3,4,5 })) == true);
			}
			{
				TRingBuffer<Counter> Q({ 0,1,2,3,4 });
				Value.Value = 4;
				Counter::Clear();
				CHECK_MESSAGE(TEXT("Remove - one element - element at end - num"), Q.Remove(Value) == 1);
				CHECK_MESSAGE(TEXT("Remove - one element - element at end - destructor count"), Counter::NumDestruct == 1);
				Q.Add(5);
				CHECK_MESSAGE(TEXT("Remove - one element - element at end - values"), (Q == TRingBuffer<Counter>({ 0,1,2,3,5 })) == true);
			}
			{
				TRingBuffer<Counter> Q({ 1,2,3,4 });
				Q.AddFront(4);
				Value.Value = 4;
				Counter::Clear();
				CHECK_MESSAGE(TEXT("Remove - one element at front one at end - num"), Q.Remove(Value) == 2);
				CHECK_MESSAGE(TEXT("Remove - one element at front one at end - destructor count"), Counter::NumDestruct == 5);
				Q.Add(5);
				CHECK_MESSAGE(TEXT("Remove - one element at front one at end - values"), (Q == TRingBuffer<Counter>({ 1,2,3,5 })) == true);
			}
			{
				TRingBuffer<Counter> Q({ 1,2,3,4 });
				Q.AddFront(1);
				Value.Value = 1;
				Counter::Clear();
				CHECK_MESSAGE(TEXT("Remove - two elements - front at end - num"), Q.Remove(Value) == 2);
				CHECK_MESSAGE(TEXT("Remove - two elements - front at end - destructor count"), Counter::NumDestruct == 5);
				Q.Add(5);
				CHECK_MESSAGE(TEXT("Remove - two elements - front at end - values"), (Q == TRingBuffer<Counter>({ 2,3,4,5 })) == true);
			}
		}

		// Test use of a specified allocator
		{
			using namespace UE::Containers::RingBuffer::Test;
			{
				NumCallsOfResizeAllocation = 0;
				TRingBuffer<int32, TCoreTRingBufferTestInlineAllocator<10>> Q;
				for (int32 AddIndex = 0; AddIndex < 20; ++AddIndex)
				{
					Q.Add(AddIndex);
					for (int32 TestIndex = 0; TestIndex < Q.Num(); ++TestIndex)
					{
						CHECK_MESSAGE(TEXT("Specified Allocator - Expected values present with inline allocator"),
							Q[TestIndex] == TestIndex);
					}
				}
				CHECK_MESSAGE(TEXT("Specified Allocator - ResizeAllocation called"), NumCallsOfResizeAllocation > 0);
			}
		}

		// Test some Reallocate cases
		{
			struct FLargeElement
			{
				FLargeElement(int32 InValue)
					: Value(InValue)
				{
				}
				int32 Value;
				// To be a proper test, this buffer must be larger than InlineBufferSize in Reallocate
				uint8 Buffer[16384];
			};
			{
				TRingBuffer<FLargeElement> Q;
				constexpr int32 Count = 10;
				for (int32 AddIndex = 0; AddIndex < Count; ++AddIndex)
				{
					Q.Emplace(AddIndex);
				}
				for (int32 TestIndex = 0; TestIndex < Count; ++TestIndex)
				{
					CHECK_MESSAGE(TEXT("Reallocate FLargeElement - Expected values present"), Q[TestIndex].Value == TestIndex);
				}
			}
			{
				// To be a proper test, sizeof(int32)*Count must be larger than InlineBufferSize in Reallocate 
				constexpr int32 Count = 10000;
				TRingBuffer<FLargeElement> Q;
				for (int32 AddIndex = 0; AddIndex < Count; ++AddIndex)
				{
					Q.Emplace(AddIndex);
				}
				for (int32 TestIndex = 0; TestIndex < Count; ++TestIndex)
				{
					CHECK_MESSAGE(TEXT("Reallocate large count - Expected values present"), Q[TestIndex].Value == TestIndex);
				}
			}
		}

		return true;
	}

	template <typename T>
	void TestShiftIndex()
	{
		// Test shifts at specific points
		{
			{
				TRingBuffer<T> Q{ 0,1,2,3,4,5,6,7 };
				Q.ShiftIndexToFront(5);
				CHECK_MESSAGE(TEXT("ShiftIndexToFront"), TRingBuffer<T>({ 5,0,1,2,3,4,6,7 }) == Q);
				Q.ShiftIndexToBack(3);
				CHECK_MESSAGE(TEXT("ShiftIndexToBack"), TRingBuffer<T>({ 5,0,1,3,4,6,7,2 }) == Q);
			}

			{
				TRingBuffer<T> Q{ 0,1,2,3,4,5,6,7 };
				int32 Offset = 4;
				Q.Front += Offset;
				Q.AfterBack += Offset;
				//Now equal to: TRingBuffer<uint32> Q{ 4,5,6,7,0,1,2,3 };
				Q.ShiftIndexToFront(5);
				CHECK_MESSAGE(TEXT("ShiftIndexToFront With Offset"), TRingBuffer<T>({ 1,4,5,6,7,0,2,3 }) == Q);
				Q.ShiftIndexToBack(3);
				CHECK_MESSAGE(TEXT("ShiftIndexToBack With Offset"), TRingBuffer<T>({ 1,4,5,7,0,2,3,6 }) == Q);
			}

			{
				TRingBuffer<T> Q{ 0,1,2,3,4,5,6,7,8 };
				CHECK_MESSAGE(TEXT("ShiftIndexToFront Cap - Capacity"), Q.Max() == 16);
				Q.ShiftIndexToFront(5);
				CHECK_MESSAGE(TEXT("ShiftIndexToFront Cap"), TRingBuffer<T>({ 5,0,1,2,3,4,6,7,8 }) == Q);
				Q.ShiftIndexToBack(3);
				CHECK_MESSAGE(TEXT("ShiftIndexToBack Cap"), TRingBuffer<T>({ 5,0,1,3,4,6,7,8,2 }) == Q);
			}

			{
				TRingBuffer<T> Q(16);
				for (int32 It = 7; It >= 0; --It)
				{
					Q.AddFront(It);
				}
				Q.Pop();
				// 8 Invalids, followed by 0,1,2,3,4,5,6, followed by Invalid
				Q.ShiftIndexToFront(5);
				CHECK_MESSAGE(TEXT("ShiftIndexToFront Cap With Offset"), TRingBuffer<T>({ 5,0,1,2,3,4,6 }) == Q);
				Q.ShiftIndexToBack(3);
				CHECK_MESSAGE(TEXT("ShiftIndexToBack Cap With Offset"), TRingBuffer<T>({ 5,0,1,3,4,6,2 }) == Q);
			}

			{
				TRingBuffer<T> Q(16);
				for (int32 It = 7; It >= 0; --It)
				{
					Q.AddFront(It);
				}
				Q.Add(8);
				// 8, (AfterBack), followed by 7 Invalids, followed by (Start) 0,1,2,3,4,5,6,7
				Q.ShiftIndexToFront(8);
				CHECK_MESSAGE(TEXT("ShiftIndexToFront Cap With Wrapped"), TRingBuffer<T>({ 8,0,1,2,3,4,5,6,7 }) == Q);
				Q.ShiftIndexToBack(0);
				CHECK_MESSAGE(TEXT("ShiftIndexToBack Cap With Wrapped"), TRingBuffer<T>({ 0,1,2,3,4,5,6,7,8 }) == Q);
			}
		}

		// Test ShiftIndex of each possible index
		{
			int32 Count = 8;
			for (int32 It = 0; It < Count; ++It)
			{
				TRingBuffer<T> Q({ 0,1,2,3,4,5,6,7 });
				Q.ShiftIndexToBack(It);
				int32 CheckIndex = 0;
				for (; CheckIndex < It; ++CheckIndex)
				{
					CHECK_MESSAGE(*FString::Printf(TEXT("ShiftIndexToBack Entire Array Values (%d,%d)"), It, CheckIndex), CheckIndex == (int32)Q[CheckIndex]);
				}
				for (; CheckIndex < Count - 1; ++CheckIndex)
				{
					CHECK_MESSAGE(*FString::Printf(TEXT("ShiftIndexToBack Entire Array Values (%d,%d)"), It, CheckIndex), CheckIndex + 1 == (int32)Q[CheckIndex]);
				}
				CHECK_MESSAGE(*FString::Printf(TEXT("ShiftIndexToBack Entire Array Values (%d,%d)"), It, Count - 1), It == (int32)Q[Count - 1]);
			}
			for (int32 It = 0; It < Count; ++It)
			{
				TRingBuffer<T> Q({ 0,1,2,3,4,5,6,7 });
				Q.ShiftIndexToFront(It);

				CHECK_MESSAGE(*FString::Printf(TEXT("ShiftIndexToFront Entire Array Values (%d,%d)"), It, 0), It == (int32)Q[0]);
				int32 CheckIndex = 1;
				for (; CheckIndex <= It; ++CheckIndex)
				{
					CHECK_MESSAGE(*FString::Printf(TEXT("ShiftIndexToFront Entire Array Values (%d,%d)"), It, CheckIndex), CheckIndex - 1 == (int32)Q[CheckIndex]);
				}
				for (; CheckIndex < Count; ++CheckIndex)
				{
					CHECK_MESSAGE(*FString::Printf(TEXT("ShiftIndexToFront Entire Array Values (%d,%d)"), It, CheckIndex), CheckIndex == (int32)Q[CheckIndex]);
				}
			}
		}
	}

	template <typename T, typename U>
	static bool ArrayViewsEqual(const TArrayView<T>& A, const TArrayView<U>& B)
	{
		int32 Num = A.Num();
		if (Num != B.Num())
		{
			return false;
		}
		for (int It = 0; It < Num; ++It)
		{
			if (A[It] != B[It])
			{
				return false;
			}
		}
		return true;
	}
};

int FRingBufferTest::Counter::NumVoid = 0;
int FRingBufferTest::Counter::NumCopy = 0;
int FRingBufferTest::Counter::NumMove = 0;
int FRingBufferTest::Counter::NumDestruct = 0;

TEST_CASE_NAMED(FRingBufferTestSubClass, "System::Core::Containers::RingBuffer", "[ApplicationContextMask][EngineFilter]")
{
	FRingBufferTest instance;
	instance.Test();
}

#endif // #if WITH_TESTS