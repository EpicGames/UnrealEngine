// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Misc/AutomationTest.h"

#include "Experimental/Containers/RobinHoodHashTable.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FRobinHoodHashTableTest, "System::Core::Containers::RobinHoodShashTable::Basic", "[ApplicationContextMask][SmokeFilter]")
{
	Experimental::TRobinHoodHashMap<int32, int32> Table;
	static constexpr int32 N = 1999; // Some reasonable prime number of elements

	// Insert some unique values into the table
	for (int32 Index = 0; Index < N; ++Index)
	{
		bool bAlreadyExists = false;
		Table.FindOrAdd(Index, Index, bAlreadyExists);
		CHECK_MESSAGE(TEXT("bAlreadyExists"), bAlreadyExists == false);
	}

	// Confirm that everything was added
	for (int32 Index = 0; Index < N; ++Index)
	{
		bool bAlreadyExists = false;
		Table.FindOrAdd(Index, Index, bAlreadyExists);
		CHECK_MESSAGE(TEXT("bAlreadyExists"), bAlreadyExists == true);
	}

	// Check that everything can be accessed via const Find()
	{
		const Experimental::TRobinHoodHashMap<int32, int32>& TableConst = Table;
		for (int32 Index = 0; Index < N; ++Index)
		{
			bool bAlreadyExists = false;
			const int32* FoundPtr = TableConst.Find(Index);
			CHECK_MESSAGE(TEXT("FoundPtr"), FoundPtr != nullptr);
			CHECK_MESSAGE(TEXT("*FoundPtr"), ((FoundPtr != nullptr) && (*FoundPtr == Index)));
		}
	}
}

#endif // WITH_TESTS
