// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Algo/IndexOf.h"
#include "Algo/RandomShuffle.h"
#include "Containers/CompactSet.h"
#include "Containers/ScriptCompactSet.h"
#include "Containers/SparseSet.h"
#include "Containers/ScriptSparseSet.h"
#include "Containers/Map.h"
#include "Tests/TestHarnessAdapter.h"

#define WITH_SET_BENCHMARKS 0 && WITH_LOW_LEVEL_TESTS

#if WITH_SET_BENCHMARKS
#include <catch2/benchmark/catch_benchmark.hpp>
#endif

struct FSparseSetTestResolver
{
	template<typename InElementType, typename KeyFuncs = DefaultKeyFuncs<InElementType>, typename Allocator = FDefaultSparseSetAllocator>
	using SetType = TSparseSet<InElementType, KeyFuncs, Allocator>;

	template <typename Type, int32 Size>
	using FixedSetType = TSparseSet<Type, DefaultKeyFuncs<Type>, TFixedSparseSetAllocator<Size>>;
};

struct FCompactSetTestResolver
{
	template<typename InElementType, typename KeyFuncs = DefaultKeyFuncs<InElementType>, typename Allocator = FDefaultCompactSetAllocator>
	using SetType = TCompactSet<InElementType, KeyFuncs, Allocator>;

	template <typename Type, int32 Size>
	using FixedSetType = TCompactSet<Type, DefaultKeyFuncs<Type>, TFixedCompactSetAllocator<Size>>;
};

template<typename Type, typename SetType>
void SetTestExists(const Type& Key, const SetType& Set)
{
	const Type* Value = Set.Find(Key);
	CHECK(Value);
	if (Value)
	{
		CHECK(*Value == Key);
	}
}

struct FSetTestDestructableData
{
	FSetTestDestructableData()
	{
		++Count;
	}
	FSetTestDestructableData(uint32 InValue)
	: Value(InValue)
	{
		++Count;
	}

	FSetTestDestructableData(const FSetTestDestructableData& Other) : FSetTestDestructableData(Other.Value)
	{
	}

	FSetTestDestructableData(FSetTestDestructableData&& Other) : FSetTestDestructableData(Other.Value)
	{
	}

	~FSetTestDestructableData()
	{
		--Count;
	}

	FSetTestDestructableData& operator=(const FSetTestDestructableData&) = default;
	FSetTestDestructableData& operator=(FSetTestDestructableData&&) = default;
	
	bool operator==(const FSetTestDestructableData& Other) const
	{
		return Value == Other.Value;
	}

	int32 Value = 0;

	static inline int32 Count;
};

struct FSetTestNonCopyableTestData : public FSetTestDestructableData
{
	FSetTestNonCopyableTestData() = default;
	FSetTestNonCopyableTestData(uint32 InValue) : FSetTestDestructableData(InValue)
	{
	}

	// Not using noncopyable helpers since we're explicitly want to catch incorrect copy operations
	FSetTestNonCopyableTestData(const FSetTestNonCopyableTestData&) = delete;
	FSetTestNonCopyableTestData& operator=(const FSetTestNonCopyableTestData&) = delete;

	// But allow for proper use of move operations
	FSetTestNonCopyableTestData(FSetTestNonCopyableTestData&&) = default;
	FSetTestNonCopyableTestData& operator=(FSetTestNonCopyableTestData&&) = default;
};

uint32 GetTypeHash(const FSetTestDestructableData& A)
{
	return A.Value;
}

template<typename SetResolver = FCompactSetTestResolver>
struct TSetTestHelper
{
	template <typename Type>
	using SetType = typename SetResolver::template SetType<Type>;

	template<typename T>
	using DuplicateKeyFunc = DefaultKeyFuncs<T, true>;

	template <typename Type>
	using DuplicateSetType = typename SetResolver::template SetType<Type, DuplicateKeyFunc<Type>>;

	template <typename Type, int32 Size>
	using FixedSetType = typename SetResolver::template FixedSetType<Type, Size>;

	TSetTestHelper()
	{
		SECTION("Default Constructor")
		{
			SetType<int32> Set;
			CHECK(Set.Num() == 0);
		}

		SECTION("Basic Tests")
		{
			FString StringTestItems[] = { TEXT("Zero"), TEXT("One"), TEXT("Two"), TEXT("Three") };
			SetType<FString> Set( { StringTestItems[0], StringTestItems[1], StringTestItems[2], StringTestItems[3] });
			CHECK(Set.Num() == 4);

			for (int32 Index = 0; Index < UE_ARRAY_COUNT(StringTestItems); ++Index)
			{
				CHECK(Set[FSetElementId::FromInteger(Index)] == StringTestItems[Index]);
				SetTestExists(StringTestItems[Index], Set);
			}

			CHECK(Set.Find( { TEXT("Invalid") }) == nullptr);
			CHECK(Set.Find({}) == nullptr);
		}

		SECTION("NonCopyable Tests")
		{
			SetType<FSetTestNonCopyableTestData> Set;
			for (uint32 Index = 0; Index < 10; ++Index)
			{
				Set.Emplace(Index);

				FSetTestNonCopyableTestData Tmp { Index };
				Set.Emplace(MoveTemp(Tmp));
			}
		}

		SECTION("Destruction Tests")
		{
			constexpr int32 TestCount = 10;

			SetType<FSetTestDestructableData> Set;
			for (uint32 Index = 0; Index < TestCount; ++Index)
			{
				Set.Emplace(Index);
			}

			CHECK(FSetTestDestructableData::Count == TestCount);

			SetType<FSetTestDestructableData> SetCopy = Set;

			CHECK(FSetTestDestructableData::Count == TestCount * 2);

			SetType<FSetTestDestructableData> SetMove = MoveTemp(Set);

			CHECK(FSetTestDestructableData::Count == TestCount * 2);

			for (uint32 Index = 0; Index < TestCount; ++Index)
			{
				Set.Emplace(Index);
			}

			CHECK(FSetTestDestructableData::Count == TestCount * 3);

			SetCopy = Set;

			CHECK(FSetTestDestructableData::Count == TestCount * 3);

			SetMove = MoveTemp(Set);

			CHECK(FSetTestDestructableData::Count == TestCount * 2);

			SetCopy.Reset();

			CHECK(FSetTestDestructableData::Count == TestCount);
		}

		SECTION("Advanced Tests")
		{
			int32 IntTestItems[256];
			int32 IntExtraTestItem[16];

			{
				int32 IntInit = 1;
				for (int32& Int : IntTestItems)
				{
					Int = HashCombine(IntInit++, 0);
				}
				for (int32& Int : IntExtraTestItem)
				{
					Int = HashCombine(IntInit++, 0);
				}
			}

			// SetAllTestItems and SetAllTestItemsReverse are the same, but different order
			SetType<int32> SetAllTestItems(IntTestItems);
			CHECK(SetAllTestItems.Num() == UE_ARRAY_COUNT(IntTestItems)); // Ensure we didn't have any duplicates

			SetType<int32> SetAllTestItemsReverse;

			// Test behavior of same set of data but in a different order
			{
				for (int32 Item : ReverseIterate(IntTestItems))
				{
					SetAllTestItemsReverse.Add(Item);
				}

				for (int32 TestItem : IntTestItems)
				{
					SetTestExists(TestItem, SetAllTestItems);
					SetTestExists(TestItem, SetAllTestItemsReverse);
				}

				CHECK(SetAllTestItems.Includes(SetAllTestItemsReverse));
				CHECK(SetAllTestItemsReverse.Includes(SetAllTestItems));
				CHECK(SetAllTestItems.Difference(SetAllTestItemsReverse).IsEmpty());

				CHECK(SetAllTestItems.Union(SetAllTestItemsReverse).Num() == SetAllTestItems.Num());

				CHECK(SetAllTestItems.Union(SetAllTestItemsReverse).Difference(SetAllTestItemsReverse).IsEmpty());
				CHECK(SetAllTestItems.Intersect(SetAllTestItemsReverse).Difference(SetAllTestItemsReverse).IsEmpty());
			}

			// Contains the first half of whole set of items
			SetType<int32> SetFirstHalfTestItems;
			// Contains the second half of whole set of items
			SetType<int32> SetSecondHalfTestItems;

			// Test behavior of subsets of the same data
			{
				for (int32 Index = 0; Index < UE_ARRAY_COUNT(IntTestItems) / 2; ++Index)
				{
					SetFirstHalfTestItems.Add(IntTestItems[Index]);
				}
				for (int32 Index = UE_ARRAY_COUNT(IntTestItems) / 2; Index < UE_ARRAY_COUNT(IntTestItems); ++Index)
				{
					SetSecondHalfTestItems.Add(IntTestItems[Index]);
				}

				CHECK(SetAllTestItems.Includes(SetFirstHalfTestItems));
				CHECK(SetAllTestItems.Includes(SetSecondHalfTestItems));
				CHECK(!SetFirstHalfTestItems.Includes(SetAllTestItems));
				CHECK(!SetSecondHalfTestItems.Includes(SetAllTestItems));

				CHECK(SetAllTestItems.Difference(SetFirstHalfTestItems).Num() == SetAllTestItems.Num() - SetFirstHalfTestItems.Num());
				CHECK(SetAllTestItems.Difference(SetSecondHalfTestItems).Num() == SetAllTestItems.Num() - SetSecondHalfTestItems.Num());
				CHECK(SetAllTestItems.Intersect(SetFirstHalfTestItems).Difference(SetFirstHalfTestItems).IsEmpty());
				CHECK(SetAllTestItems.Difference(SetFirstHalfTestItems).Difference(SetAllTestItemsReverse.Difference(SetFirstHalfTestItems)).IsEmpty());

				CHECK(SetAllTestItems.Includes(SetFirstHalfTestItems));

				CHECK(SetFirstHalfTestItems.Difference(SetSecondHalfTestItems).Num() == SetFirstHalfTestItems.Num());
				CHECK(SetSecondHalfTestItems.Difference(SetFirstHalfTestItems).Num() == SetSecondHalfTestItems.Num());
				CHECK(SetFirstHalfTestItems.SymmetricDifference(SetSecondHalfTestItems).Num() == UE_ARRAY_COUNT(IntTestItems));
				CHECK(SetSecondHalfTestItems.SymmetricDifference(SetFirstHalfTestItems).Num() == UE_ARRAY_COUNT(IntTestItems));
				CHECK(SetFirstHalfTestItems.SymmetricDifference(SetAllTestItems).Num() == SetFirstHalfTestItems.Num());
				CHECK(SetSecondHalfTestItems.SymmetricDifference(SetAllTestItemsReverse).Num() == SetSecondHalfTestItems.Num());
				CHECK(SetFirstHalfTestItems.Intersect(SetSecondHalfTestItems).IsEmpty());
			}

			// Test behavior of disinct sets of data
			{
				// More items that are not contained in any other set
				SetType<int32> SetExtraItems(IntExtraTestItem);
				CHECK(SetExtraItems.Num() == UE_ARRAY_COUNT(IntExtraTestItem)); // Ensure we didn't have any duplicates
				CHECK(SetExtraItems.Union(SetAllTestItems).Num() == SetExtraItems.Num() + SetAllTestItems.Num()); // Ensure we didn't have any duplicates between the two sets
				CHECK(SetExtraItems.SymmetricDifference(SetAllTestItems).Num() == SetExtraItems.Num() + SetAllTestItems.Num());

				SetType<int32> SetExtraItemsWithFirstHalf = SetExtraItems.Union(SetFirstHalfTestItems);
				SetType<int32> SetExtraItemsWithSecondHalf = SetExtraItems.Union(SetSecondHalfTestItems);

				CHECK(SetExtraItemsWithFirstHalf.Includes(SetExtraItems));
				CHECK(SetExtraItemsWithFirstHalf.Includes(SetFirstHalfTestItems));
				CHECK(SetExtraItemsWithSecondHalf.Includes(SetExtraItems));
				CHECK(SetExtraItemsWithSecondHalf.Includes(SetSecondHalfTestItems));

				CHECK(SetExtraItemsWithFirstHalf.Intersect(SetExtraItemsWithSecondHalf).Num() == SetExtraItems.Num());

				CHECK(SetExtraItemsWithFirstHalf.SymmetricDifference(SetAllTestItems).Num() == SetExtraItems.Num() + SetAllTestItems.Num() - SetFirstHalfTestItems.Num());
				CHECK(SetExtraItemsWithFirstHalf.SymmetricDifference(SetExtraItemsWithSecondHalf).Num() == SetAllTestItems.Num());

				CHECK(SetExtraItems.Union(SetFirstHalfTestItems).SymmetricDifference(SetSecondHalfTestItems).Num() == SetExtraItems.Num() + SetAllTestItems.Num());
				CHECK(SetExtraItemsWithFirstHalf.SymmetricDifference(SetExtraItemsWithSecondHalf).Intersect(SetExtraItems).IsEmpty());
			}
		}

		SECTION("Copy/Move")
		{
			FString StringTestItems[] = { TEXT("Zero"), TEXT("One"), TEXT("Two"), TEXT("Three") };
			SetType<FString> SetAllTestItems( { StringTestItems[0], StringTestItems[1], StringTestItems[2], StringTestItems[3] });
			SetType<FString> SetAllTestItemsReverse(MoveTemp(SetAllTestItems));

			CHECK(SetAllTestItems.IsEmpty());
			CHECK(SetAllTestItems.Difference(SetAllTestItemsReverse).IsEmpty());
		}

		SECTION("Growth Tests")
		{
			SetType<int32> Set;

			int32 ValuesToAdd[66000]; // include at least one hash size growth
			int32 IndexToAdd = 0;
			for (int32& Value : ValuesToAdd)
			{
				IndexToAdd += 1 + FMath::RandHelper(4);
				Value = IndexToAdd;
			}
			Algo::RandomShuffle(ValuesToAdd);

			for (int32 Value : ValuesToAdd)
			{
				CHECK(Set.Contains(Value) == false);
				Set.Add(Value);
				CHECK(Set.Contains(Value) == true);
			}

			for (int32 Value : ValuesToAdd)
			{
				CHECK(Set.Contains(Value) == true);
			}
		}

		SECTION("Clear Tests")
		{
			SetType<int32> Set;

			const uint32 PartialElements = 4;
			// + 1 so it's immune to < vs <=, we want to make sure to test both unhash and memset scenarios for both set types
			const uint32 MaxElements = (PartialElements + 1) * FMath::Max(UE_SET_UNHASH_THRESHOLD_STORED_KEY, UE_SET_UNHASH_THRESHOLD_CALCULATE_KEY);
			Set.Empty(MaxElements);
			CHECK(Set.Max() >= MaxElements);
			const uint32 CalculatedMaxElements = Set.Max();

			// There's not really much to test here, just run multiple passes of reusing the set to make sure things don't explode
			for (uint32 LoopIndex = 0; LoopIndex < 4; ++LoopIndex)
			{
				const int32 Seed = MaxElements * (LoopIndex & 1);

				for (uint32 PartialIndex = 0; PartialIndex < PartialElements; ++PartialIndex)
				{
					Set.Add(Seed + PartialIndex);
					CHECK(Set.Contains(Seed + PartialIndex));
				}

				CHECK(Set.Num() == PartialElements);
				Set.Reset();
				CHECK(Set.Num() == 0);
				CHECK(Set.Max() == CalculatedMaxElements);

				for (uint32 FullIndex = 0; FullIndex < MaxElements; ++FullIndex)
				{
					Set.Add(Seed + FullIndex);
					CHECK(Set.Contains(Seed + FullIndex));
				}

				CHECK(Set.Num() == MaxElements);
				Set.Reset();
				CHECK(Set.Num() == 0);
				CHECK(Set.Max() == CalculatedMaxElements);
			}
		}

		SECTION("Iteration")
		{
			int32 IntTestItems[256];

			{
				int32 IntInit = 1;
				for (int32& Int : IntTestItems)
				{
					Int = HashCombine(IntInit++, 0);
				}
			}

			SetType<int32> Set(IntTestItems);

			for (int32 Index = 0; Index < Set.Num(); ++Index)
			{
				CHECK(IntTestItems[Index] == Set[FSetElementId::FromInteger(Index)]);
			}

			int32 RemoveIndex = 0;
			for (auto Iter = Set.CreateIterator(); Iter; ++Iter, ++RemoveIndex)
			{
				if (RemoveIndex % 2)
				{
					Iter.RemoveCurrent();
				}
			}

			CHECK(Set.Num() == UE_ARRAY_COUNT(IntTestItems) / 2);
		}

		SECTION("Fixed Allocator")
		{
			FixedSetType<int32, 64> Set;

			for (int32 Index = 0; Index < 64; ++Index)
			{
				Set.Add(Index);
			}
		}

		// ensure that duplicate entries overrides existing elements and doesn't cause reordering
		SECTION("Duplicate Entry Behavior")
		{
			SetType<int32> Set;

			Set.Add(0);
			Set.Add(1);
			Set.Add(2);
			Set.Add(3);
			Set.Add(2);

			for (int32 Index = 0; Index < Set.Num(); ++Index)
			{
				CHECK(Set[FSetElementId::FromInteger(Index)] == Index);
			}
		}

		SECTION("Duplicate Support")
		{
			FString StringTestItems[] = { TEXT("Zero"), TEXT("One"), TEXT("Two"), TEXT("Three") };
			DuplicateSetType<FString> Set;

			for (int32 Index = 0; Index < 32; ++Index)
			{
				for (const FString& TestItem : StringTestItems)
				{
					Set.Add(TestItem);
				}
			}

			CHECK(Set.Num() == 32 * 4);

			{
				int32 KeyIteratorIndex = 0;
				for (typename DuplicateSetType<FString>::TConstKeyIterator KeyIter(Set, StringTestItems[1]); KeyIter; ++KeyIter, ++KeyIteratorIndex)
				{
					CHECK(KeyIter.GetId().AsInteger() % 4 == 1);
					CHECK(*KeyIter == StringTestItems[1]);
				}

				CHECK(KeyIteratorIndex == 32);
			}

			Set.Sort([&StringTestItems](const FString& Lhs, const FString& Rhs)
				{
					return Algo::IndexOf(StringTestItems, Lhs) < Algo::IndexOf(StringTestItems, Rhs);
				});

			{
				int32 KeyIteratorIndex = 0;
				for (typename DuplicateSetType<FString>::TConstKeyIterator KeyIter(Set, StringTestItems[1]); KeyIter; ++KeyIter, ++KeyIteratorIndex)
				{
					CHECK((KeyIter.GetId().AsInteger() >= 32 && KeyIter.GetId().AsInteger() < 64));
					CHECK(*KeyIter == StringTestItems[1]);
				}
				CHECK(KeyIteratorIndex == 32);
			}

			// Remove half of items
			{
				int32 KeyIteratorIndex = 0;
				for (typename DuplicateSetType<FString>::TKeyIterator KeyIter(Set, StringTestItems[1]); KeyIter; ++KeyIter, ++KeyIteratorIndex)
				{
					if ((KeyIteratorIndex % 2) == 0)
					{
						// Sorting breaks here
						KeyIter.RemoveCurrent();
					}
				}
				CHECK(KeyIteratorIndex == 32);
			}

			// Check remaining
			{
				int32 KeyIteratorIndex = 0;
				for (typename DuplicateSetType<FString>::TConstKeyIterator KeyIter(Set, StringTestItems[1]); KeyIter; ++KeyIter, ++KeyIteratorIndex)
				{
					// Items will still fall in the old range since they remained in place while items from the end filled the gap
					CHECK((KeyIter.GetId().AsInteger() >= 32 && KeyIter.GetId().AsInteger() < 64));
					CHECK(*KeyIter == StringTestItems[1]);
				}
				CHECK(KeyIteratorIndex == 16);
			}

			CHECK(Set.Remove(StringTestItems[1]) == 16);
			CHECK(Set.Num() == 32 * 3);
			CHECK(Set.Find(StringTestItems[1]) == nullptr);
		}

		SECTION("Optional Support")
		{
			TOptional<SetType<int32>> OptionalSet;

			const SetType<int32>* OptionalSetInternals = (const SetType<int32>*)&OptionalSet;
			const FIntrusiveUnsetOptionalState& OptionalCheck = *(FIntrusiveUnsetOptionalState*)&OptionalSet; // hack to access this type for internal comparison
			CHECK(OptionalSetInternals->Num() == 0);
			CHECK(*OptionalSetInternals == OptionalCheck);

			OptionalSet.Emplace();
			OptionalSet.GetValue().Add(1);
			OptionalSet.GetValue().Add(2);
			CHECK(OptionalSet.GetValue().Num() == 2);
			CHECK(!(*OptionalSetInternals == OptionalCheck));

			OptionalSet.Reset();
			CHECK(OptionalSetInternals->Num() == 0);
			CHECK(*OptionalSetInternals == OptionalCheck);
		}

#if WITH_SET_BENCHMARKS
		SECTION("Benchmark")
		{
			BENCHMARK("Add (With Alloc)")
			{
				SetType<int32> Set;

				for (int32 Index = 0; Index < 10000; ++Index)
				{
					Set.Add(Index * 1023);
				}
			};

			BENCHMARK("Add (Without Alloc)")
			{
				SetType<int32> Set;
				Set.Reserve(1000000);

				for (int32 Index = 0; Index < 100000; ++Index)
				{
					Set.Add(Index * 1023);
				}
			};

			BENCHMARK_ADVANCED("Remove (Forward)")(Catch::Benchmark::Chronometer meter)
			{
				SetType<int32> Set;
				Set.Reserve(10000);

				for (int32 Index = 0; Index < 10000; ++Index)
				{
					Set.Add(Index * 1023);
				}

				meter.measure([&]
					{
						for (int32 Index = 0; Index < 10000; ++Index)
						{
							Set.Remove(Index * 1023);
						}
					});
			};

			BENCHMARK_ADVANCED("Remove (Reverse)")(Catch::Benchmark::Chronometer meter)
			{
				SetType<int32> Set;
				Set.Reserve(10000);

				for (int32 Index = 0; Index < 10000; ++Index)
				{
					Set.Add(Index * 1023);
				}

				meter.measure([&]
					{
						for (int32 Index = 10000 - 1; Index >= 0; --Index)
						{
							Set.Remove(Index * 1023);
						}
					});
			};

			BENCHMARK_ADVANCED("Lookup")(Catch::Benchmark::Chronometer meter)
			{
				SetType<int32> Set;
				Set.Reserve(10000);

				for (int32 Index = 0; Index < 10000; ++Index)
				{
					Set.Add(Index * 1023);
				}

				meter.measure([&]
					{
						int32 Lookups = 0;
						for (int32 Index = 10000 - 1; Index >= 0; --Index)
						{
							Lookups += *Set.Find(Index * 1023) == Index * 1023;
						}
						return Lookups;
					});
			};

			BENCHMARK_ADVANCED("Lookup Small")(Catch::Benchmark::Chronometer meter)
			{
				SetType<int32> Set;
				Set.Reserve(128);

				for (int32 Index = 0; Index < 128; ++Index)
				{
					Set.Add(Index * 1023);
				}

				meter.measure([&]
					{
						int32 Lookups = 0;
						for (int32 Index = 0; Index < 10000; ++Index)
						{
							Lookups += *Set.Find((Index % 128) * 1023) == (Index % 128) * 1023;
						}
						return Lookups;
					});
			};
		}
#endif
	}

	~TSetTestHelper()
	{
		CHECK(FSetTestDestructableData::Count == 0);
	}
};

TEST_CASE_NAMED(FCompactSetTest, "System::Core::Containers::CompactSet", "[Core][Containers][CompactSet]")
{
	TSetTestHelper<FCompactSetTestResolver>();

	SECTION("Allocation Size")
	{
		static const int32 SizeToTest = 139;
		static_assert(TCompactSet<int32>::GetTotalMemoryRequiredInBytes(SizeToTest) == UE::Core::Private::CompactSetAllocatorHelpers::CalculateRequiredBytes<SizeToTest, sizeof(int32)>());
		TCompactSet<int32> Container;
		for (int32 Index = 0; Index < SizeToTest; ++Index)
		{
			Container.Add(Index);
		}

		// Memory allocator may have additional room in block allocated, so the set will fill up that space which can lead to increased memory usage
		CHECK(Container.GetAllocatedSize() >= TCompactSet<int32>::GetTotalMemoryRequiredInBytes(SizeToTest));
	}

	// Todo: Move into TSetTestHelper when Stable moved to TSparseSet
	SECTION("Stability Support")
	{
		TArray<FString> StringTestItems = { TEXT("Zero"), TEXT("One"), TEXT("Two"), TEXT("Three") };
		TSetTestHelper<FCompactSetTestResolver>::DuplicateSetType<FString> Set(StringTestItems);

		Set.RemoveStable(TEXT("One"));
		StringTestItems.RemoveAt(1);

		{
			auto Itr = Set.CreateConstIterator();
			for (const FString& TestString : StringTestItems)
			{
				CHECK(TestString == *Itr);
				++Itr;
			}
		}

		Set.RemoveStable(FSetElementId::FromInteger(0));
		StringTestItems.RemoveAt(0);

		{
			auto Itr = Set.CreateConstIterator();
			for (const FString& TestString : StringTestItems)
			{
				CHECK(TestString == *Itr);
				++Itr;
			}
		}
	}
}

TEST_CASE_NAMED(FSparseSetTest, "System::Core::Containers::SparseSet", "[Core][Containers][CompactSet]")
{
	TSetTestHelper<FSparseSetTestResolver>();
}

TEST_CASE_NAMED(FScriptCompactSetTest, "System::Core::Containers::ScriptCompactSet", "[Core][Containers][CompactSet]")
{
	SECTION("Basic Tests")
	{
		FScriptCompactSet Set;
		FScriptCompactSetLayout Layout = { sizeof(FString), alignof(FString) };

		FString StringTestItems[] = { TEXT("Zero"), TEXT("One"), TEXT("Two"), TEXT("Three") };
		auto GetKeyHash = [](const void* Data) { return GetTypeHash( * (const FString*)Data); };
		auto EqualityFn = [](const void* Lhs, const void* Rhs) { return * (const FString*)Lhs == * (const FString*)Rhs; };
		auto DestructFn = [](void* Data) { ((FString*)Data)->~FString(); };

		for (const FString& TestItem : StringTestItems)
		{
			Set.Add(&TestItem, Layout, GetKeyHash, EqualityFn, [&TestItem](void* Data) { new (Data)FString(TestItem); }, DestructFn);
		}

		CHECK(Set.Num() == 4);

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(StringTestItems); ++Index)
		{
			CHECK(EqualityFn(Set.GetData(Index, Layout), &StringTestItems[Index]));

			int32 FindIndex = Set.FindIndex(&StringTestItems[Index], Layout, GetKeyHash, EqualityFn);
			CHECK(FindIndex == Index);
		}

		FString Invalid;
		CHECK(Set.FindIndex(&Invalid, Layout, GetKeyHash, EqualityFn) == INDEX_NONE);
		Invalid = TEXT("Invalid");
		CHECK(Set.FindIndex(&Invalid, Layout, GetKeyHash, EqualityFn) == INDEX_NONE);

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(StringTestItems); ++Index)
		{
			DestructFn(Set.GetData(Index, Layout));
		}
	}

	SECTION("Advanced Tests")
	{
		int32 IntTestItems[256];

		{
			int32 IntInit = 1;
			for (int32& Int : IntTestItems)
			{
				Int = HashCombine(IntInit++, 0);
			}
		}

		FScriptCompactSet Set;
		FScriptCompactSetLayout Layout = { sizeof(int32), alignof(int32) };

		auto GetKeyHash = [](const void* Data) { return * (const int32 *)Data; };
		auto EqualityFn = [](const void* Lhs, const void* Rhs) { return * (const int32 *)Lhs == * (const int32 *)Rhs; };
		auto DestructFn = [](void* Data) { };

		for (int32 TestItem : IntTestItems)
		{
			*(int32 *)Set.GetData(Set.AddUninitialized(Layout), Layout) = TestItem;
		}
		Set.Rehash(Layout, GetKeyHash);

		for (int32 TestItem : IntTestItems)
		{
			CHECK(Set.FindIndex(&TestItem, Layout, GetKeyHash, EqualityFn) != INDEX_NONE);
		}

		for (int32 Index = 0; Index < Set.Num(); ++Index)
		{
			if (Index % 2)
			{
				Set.RemoveAt(Index, Layout, GetKeyHash, DestructFn);
				*(int32 *)Set.GetData(Index, Layout) = *(int32 *)Set.GetData(Set.Num() - 1, Layout);
				CHECK(Set.FindIndex(&IntTestItems[Index], Layout, GetKeyHash, EqualityFn) == INDEX_NONE);
			}
			else
			{
				CHECK(Set.FindIndex(&IntTestItems[Index], Layout, GetKeyHash, EqualityFn) != INDEX_NONE);
			}
		}
	}
}

#endif // WITH_TESTS
