// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/SharedString.h"

#include "Tests/TestHarnessAdapter.h"

namespace UE
{

TEST_CASE_NAMED(FSharedStringTest, "System::Core::String::SharedString", "[Core][String][SmokeFilter]")
{
	SECTION("Empty")
	{
		const FSharedString String;
		CHECK(String.IsEmpty());
		CHECK(String.Len() == 0);
		CHECK(**String == TEXT('\0'));
		CHECK(String == FSharedString::Empty);
	}

	SECTION("Construct")
	{
		const FStringView View = TEXTVIEW("ABC");
		const FSharedString String = View;
		CHECK(!String.IsEmpty());
		CHECK(String.Len() == View.Len());
		CHECK(**String == View[0]);
		CHECK(*String != View.GetData());
		CHECK(String == View);
		CHECK_FALSE(String == FSharedString::Empty);
	}

	SECTION("Copy/Move")
	{
		const FStringView View = TEXTVIEW("ABC");
		FSharedString String = View;
		const TCHAR* StringData = *String;

		FSharedString StringCopy = String;
		CHECK(*StringCopy == StringData);

		String.Reset();
		CHECK(String.IsEmpty());
		CHECK(StringCopy == View);

		String = StringCopy;
		CHECK(*String == StringData);

		String = MoveTemp(StringCopy);
		CHECK(*String == StringData);
		CHECK(StringCopy.IsEmpty());

		StringCopy = MoveTemp(String);
		CHECK(*StringCopy == StringData);
		CHECK(String.IsEmpty());
	}
}

} // UE

#endif // WITH_TESTS
