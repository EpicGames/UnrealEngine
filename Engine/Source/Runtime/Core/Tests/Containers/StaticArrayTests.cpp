// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/StaticArray.h"
#include "Templates/UnrealTemplate.h"

// Currently only doing compile time tests to verify constexpr functionality.
namespace Test
{
	struct FNonTrivialStruct
	{
		constexpr FNonTrivialStruct() = default;
		constexpr ~FNonTrivialStruct() = default;

		constexpr FNonTrivialStruct(int32 InValue) noexcept
			: Value(InValue)
		{

		}

		constexpr FNonTrivialStruct(const FNonTrivialStruct& Other) noexcept
		{
			Value = Other.Value;
		}

		constexpr FNonTrivialStruct(FNonTrivialStruct&& Other) noexcept
		{
			Swap(Value, Other.Value);
		}

		constexpr FNonTrivialStruct& operator=(const FNonTrivialStruct& Other) noexcept
		{
			if (this != &Other)
			{
				Value = Other.Value;
			}
			return *this;
		}

		constexpr FNonTrivialStruct& operator=(FNonTrivialStruct&& Other) noexcept
		{
			if (this != &Other)
			{
				Swap(Value, Other.Value);
			}
			return *this;
		}

		constexpr bool operator==(const FNonTrivialStruct& Other) const = default;
		constexpr bool operator!=(const FNonTrivialStruct& Other) const = default;

		constexpr int32 GetValue() const
		{
			return Value;
		}

		int32 Value = 0;
	};

	static_assert(!std::is_trivial<FNonTrivialStruct>::value);

	using FTestType = FNonTrivialStruct;

	// Test creating an array and assigning elements
	template<uint32 Count>
	constexpr TStaticArray<FTestType, Count> CreateAscendingArray()
	{
		TStaticArray<FTestType, Count> Array;
		for (int32 Idx = 0; Idx < Array.Num(); ++Idx)
		{
			Array[Idx] = Idx;
		}
		return Array;
	}

	static_assert(CreateAscendingArray<4>().Num() == 4);
	static_assert(CreateAscendingArray<4>()[2].GetValue() == 2);

	// Test variadic constructor
	template<typename ElementType, typename... Elements>
	constexpr auto CreateStaticArray(Elements... InElements)
	{
		return TStaticArray<ElementType, sizeof...(InElements)>(InElements...);
	}

	static_assert(CreateStaticArray<FTestType>(1, 2, 3, 4)[2].GetValue() == 3);

	// Test In-place constructor
	static_assert(TStaticArray<FTestType, 4>(InPlace, 42)[2].GetValue() == 42);

	// Test copying
	template<uint32 Count>
	constexpr TStaticArray<FTestType, Count> CloneStaticArray(const TStaticArray<FTestType, Count>& InArray)
	{
		TStaticArray<FTestType, Count> OutArray = InArray;
		return OutArray;
	}

	static_assert(CloneStaticArray(CreateAscendingArray<4>())[2].GetValue() == 2);

	// Test moving
	template<uint32 Count>
	constexpr TStaticArray<FTestType, Count> StealStaticArray(TStaticArray<FTestType, Count>&& InArray)
	{
		TStaticArray<FTestType, Count> OutArray(MoveTemp(InArray));
		return OutArray;
	}

	static_assert(StealStaticArray(CreateAscendingArray<4>())[2].GetValue() == 2);

	template<uint32 Count>
	constexpr TStaticArray<FTestType, Count> StealStaticArrayViaMoveAssignment(TStaticArray<FTestType, Count>&& InArray)
	{
		TStaticArray<FTestType, Count> OutArray;
		OutArray = MoveTemp(InArray);
		return OutArray;
	}

	static_assert(StealStaticArray(CreateAscendingArray<4>())[2].GetValue() == 2);

	template<uint32 Count>
	constexpr TStaticArray<FTestType, Count> ReturnMovedFromArray(TStaticArray<FTestType, Count>&& InArray)
	{
		TStaticArray<FTestType, Count> OutArray = MoveTemp(InArray);
		return InArray;
	}

	// This is only safe as we know the moved-from state is valid.
	static_assert(ReturnMovedFromArray<4>(CreateAscendingArray<4>())[2] == 0);

	// Test MakeUniformStaticArray
	static_assert(MakeUniformStaticArray<FTestType, 8>(42).Num() == 8);
	static_assert(MakeUniformStaticArray<FTestType, 8>(42)[3].GetValue() == 42);

	// Test comparisons
	static_assert(CreateAscendingArray<4>() == CreateAscendingArray<4>()); //-V501
	static_assert(CreateAscendingArray<4>() != MakeUniformStaticArray<FTestType, 4>(10));

	// Test IsEmpty
	static_assert(TStaticArray<FTestType, 0>{}.IsEmpty());

	// Test Iteration
	template<uint32 Count>
	constexpr TStaticArray<FTestType, Count> AddOne(TStaticArray<FTestType, Count> InArray)
	{
		for (FTestType& Value : InArray)
		{
			++Value.Value;
		}
		return InArray;
	}

	static_assert(AddOne(MakeUniformStaticArray<FTestType, 4>(10)) == MakeUniformStaticArray<FTestType, 4>(11));

} // namespace Test

#endif // WITH_TESTS
