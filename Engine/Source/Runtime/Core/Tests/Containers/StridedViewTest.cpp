// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/StridedView.h"
#include "Misc/AutomationTest.h"


#include "Algo/Accumulate.h"
#include "Containers/Array.h"
#include "Math/Vector.h"

#include <type_traits>
#include "Tests/TestHarnessAdapter.h"

namespace UE
{
	namespace StridedViewTest
	{

		struct FMyStruct
		{
			uint32  SomeData = 123;
			FVector Position = FVector(5.0f, 6.0f, 7.0f);
			uint32  MoreData = 890;
		};

		struct FMyStructDerived : FMyStruct
		{
			FVector Normal = FVector(0.0f, 0.0f, 0.0f);
		};

		static void SetValue(TStridedView<FVector> Output, const FVector& Value)
		{
			for (FVector& It : Output)
			{
				It = Value;
			}
		}

		static FVector ComputeSum(TStridedView<const FVector> Values)
		{
			return Algo::Accumulate(Values, FVector(0.0f));
		}

		static FVector ComputeMean(TStridedView<const FVector> Values)
		{
			return ComputeSum(Values) / (float)Values.Num();
		}

		static FVector ComputeMeanPosition(TArrayView<const FMyStruct> Structs)
		{
			return ComputeMean(MakeStridedView(sizeof(FMyStruct), &Structs[0].Position, Structs.Num()));
		}

		static FVector ComputeMeanPositionStrided(TStridedView<const FMyStruct> Structs)
		{
			return ComputeMean(MakeStridedView(Structs.GetStride(), &Structs[0].Position, Structs.Num()));
		}

		static TStridedView<const FVector> TestCreateConstArrayView(const TArray<FVector>& VecsArray)
		{
			return MakeStridedView(VecsArray);
		}

		static_assert(std::is_convertible_v<TStridedView<FMyStruct>, TStridedView<FMyStruct>>);
		static_assert(std::is_convertible_v<TStridedView<FMyStruct>, TStridedView<const FMyStruct>>);
		static_assert(!std::is_convertible_v<TStridedView<const FMyStruct>, TStridedView<FMyStruct>>);
		static_assert(!std::is_convertible_v<TStridedView<FMyStructDerived>, TStridedView<FMyStruct>>);
		static_assert(!std::is_convertible_v<TStridedView<FMyStruct>, TStridedView<FMyStructDerived>>);
		static_assert(std::is_convertible_v<TStridedView<const FMyStruct*>, TStridedView<const FMyStruct*>>);
		static_assert(std::is_convertible_v<TStridedView<const FMyStruct*>, TStridedView<const FMyStruct* const>>);
		static_assert(!std::is_convertible_v<TStridedView<const FMyStruct* const>, TStridedView<const FMyStruct*>>);
		static_assert(!std::is_convertible_v<TStridedView<const FMyStructDerived*>, TStridedView<const FMyStruct*>>);
		static_assert(!std::is_convertible_v<TStridedView<const FMyStruct*>, TStridedView<const FMyStructDerived*>>);

	} // namespace StridedViewTest
} // namespace UE

TEST_CASE_NAMED(FStridedViewTest, "System::Core::Containers::StridedView::Misc", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::StridedViewTest;

	static constexpr int32 Num = 17;

	FMyStruct Structs[Num];

	// Initialize test data and validate that basic iteration works

	{
		TStridedView<FVector> PositionsView(sizeof(FMyStruct), &Structs[0].Position, Num);
		for (int32 i = 0; i < PositionsView.Num(); ++i)
		{
			PositionsView[i] = FVector((float)i);
		}

		for (int32 i = 0; i < Num; ++i)
		{
			CHECK_MESSAGE(TEXT("Structs[i].Position"), Structs[i].Position == FVector((float)i));
		}

		const FVector Multiplier = FVector(2.0f, 3.0f, 4.0f);
		for (FVector& It : PositionsView)
		{
			It *= Multiplier;
		}

		for (int32 i = 0; i < Num; ++i)
		{
			CHECK_MESSAGE(TEXT("Structs[i].Position"), (Structs[i].Position == (FVector((float)i) * Multiplier)));
		}
	}

	FVector ExpectedAvg = FVector(0.0f);
	for (int32 i = 0; i < Num; ++i)
	{
		ExpectedAvg += Structs[i].Position;
	}
	ExpectedAvg /= (float)Num;

	// Test various ways to construct and use the strided view

	{
		// Incorrect API usage: trying to pass a const strided view to a mutating function. Generates compile error (as expected).
		// TStridedView<const FVector> ConstView(sizeof(FMyStruct), &Structs[0].Position, Num);
		// SetValue(ConstView, FVector(123.0f));
	}

	{
		FVector Avg = ComputeMeanPosition(MakeArrayView(Structs));
		CHECK_MESSAGE(TEXT("ComputeMeanPosition(MakeArrayView(Structs))"), Avg == ExpectedAvg);
	}

	{
		FVector Avg = ComputeMeanPositionStrided(MakeStridedView(Structs));
		CHECK_MESSAGE(TEXT("ComputeMeanPositionStrided(MakeStridedView(Structs))"), Avg == ExpectedAvg);
	}

	{
		FVector Avg = ComputeMean(TStridedView<FVector>(sizeof(FMyStruct), &Structs[0].Position, Num));
		CHECK_MESSAGE(TEXT("ComputeMean(TStridedView<FVector>(sizeof(FMyStruct), &Structs[0].Position, Num))"), Avg == ExpectedAvg);
	}

	{
		FVector Avg = ComputeMean(TStridedView<const FVector>(sizeof(FMyStruct), &Structs[0].Position, Num));
		CHECK_MESSAGE(TEXT("ComputeMean(TStridedView<const FVector>(sizeof(FMyStruct), &Structs[0].Position, Num))"), Avg == ExpectedAvg);
	}

	{
		FVector Avg = ComputeMean(MakeStridedView(sizeof(FMyStruct), &Structs[0].Position, Num));
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(sizeof(FMyStruct), &Structs[0].Position, Num))"), Avg == ExpectedAvg);
	}

	{
		FVector Avg = ComputeMean(MakeStridedView(MakeArrayView(Structs), &FMyStruct::Position));
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(MakeArrayView(Structs), &FMyStruct::Position))"), Avg == ExpectedAvg);
	}

	{
		FVector Avg = ComputeMean(MakeStridedView(Structs, &FMyStruct::Position));
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(Structs, &FMyStruct::Position))"), Avg == ExpectedAvg);
	}

	{
		TArray<FMyStruct> StructsArray = TArray<FMyStruct>(Structs, Num);
		FVector Avg = ComputeMean(MakeStridedView(StructsArray, &FMyStruct::Position));
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(StructsArray, &FMyStruct::Position))"), Avg == ExpectedAvg);
	}

	// Test support for trivial strided views (stride = sizeof(element))

	FVector Vecs[Num];
	for (int32 i = 0; i < Num; ++i)
	{
		Vecs[i] = Structs[i].Position;
	}

	{
		FVector Avg = ComputeMean(MakeStridedView(Vecs));
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(Vecs))"), Avg == ExpectedAvg);
	}

	{
		TArrayView<const FVector> VecsView = Vecs;
		auto VecsStridedView = MakeStridedView(VecsView);
		static_assert(std::is_const_v<decltype(VecsStridedView)::ElementType>);
		FVector Avg = ComputeMean(VecsStridedView);
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(VecsView))"), Avg == ExpectedAvg);
	}

	{
		TArray<FVector> VecsArray = TArray<FVector>(Vecs, Num);
		auto VecsStridedView = TestCreateConstArrayView(VecsArray);
		static_assert(std::is_const_v<decltype(VecsStridedView)::ElementType>);
		FVector Avg = ComputeMean(VecsStridedView);
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(VecsArray))"), Avg == ExpectedAvg);
	}

	// Test smearing an element

	{
		FVector V = FVector(1.0f, 2.0f, 3.0f);
		FVector Avg = ComputeMean(TStridedView<FVector>(0, &V, Num));
		CHECK_MESSAGE(TEXT("ComputeMean(TStridedView<FVector>(0, &V, Num))"), Avg == V);

		FVector Sum = ComputeSum(TStridedView<FVector>(0, &V, Num));
		FVector ExpectedSum = V * (float)Num;
		CHECK_MESSAGE(TEXT("ComputeSum(TStridedView<FVector>(0, &V, Num))"), Sum == ExpectedSum);
	}

	// Test iterating through derived struct

	FMyStructDerived StructsDerived[Num];

	// Initialize derived struct data by copying positions from base and set normals to (1,2,3)*i

	{
		TStridedView<FVector> PositionsViewDerived(sizeof(FMyStructDerived), &StructsDerived[0].Position, Num);
		TStridedView<FVector> NormalsView(sizeof(FMyStructDerived), &StructsDerived[0].Normal, Num);
		TStridedView<FVector> PositionsView(sizeof(FMyStruct), &Structs[0].Position, Num);

		for (int32 i = 0; i < PositionsViewDerived.Num(); ++i)
		{
			PositionsViewDerived[i] = PositionsView[i];
			NormalsView[i] = FVector(1.0f, 2.0f, 3.0f) * (float)i;
		}

		for (int32 i = 0; i < PositionsViewDerived.Num(); ++i)
		{
			CHECK_MESSAGE(TEXT("StructsDerived[i].Position"), StructsDerived[i].Position == Structs[i].Position);
			CHECK_MESSAGE(TEXT("StructsDerived[i].Normal"), (StructsDerived[i].Normal == ( FVector(1.0f, 2.0f, 3.0f) * (float)i)));
		}
	}

	// Test different ways to construct a view of a derived struct

	{
		// Incorrect API usage: trying to pass a const strided view to a mutating function. Generates compile error (as expected).
		// TStridedView<const FVector> ConstView(sizeof(FMyStructDerived), &StructsDerived[0].Position, Num);
		// SetValue(ConstView, FVector(123.0f));
	}

	{
		FVector Avg = ComputeMean(TStridedView<FVector>(sizeof(FMyStructDerived), &StructsDerived[0].Position, Num));
		CHECK_MESSAGE(TEXT("ComputeMean(TStridedView<FVector>(sizeof(FMyStructDerived), &StructsDerived[0].Position, Num))"), Avg == ExpectedAvg);
	}

	{
		FVector Avg = ComputeMean(TStridedView<const FVector>(sizeof(FMyStructDerived), &StructsDerived[0].Position, Num));
		CHECK_MESSAGE(TEXT("ComputeMean(TStridedView<const FVector>(sizeof(FMyStructDerived), &StructsDerived[0].Position, Num))"), Avg == ExpectedAvg);
	}

	{
		FVector Avg = ComputeMean(MakeStridedView(sizeof(FMyStructDerived), &StructsDerived[0].Position, Num));
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(sizeof(FMyStructDerived), &StructsDerived[0].Position, Num))"), Avg == ExpectedAvg);
	}

	{
		// Incorrect API usage: trying to create view of derived class from base. Generates compile error (as expected).
		// TStridedView<FMyStructDerived> ViewDerivedError = MakeStridedView(Structs);

		TStridedView<FMyStructDerived> ViewDerived = MakeStridedView(StructsDerived);

		// Incorrect API usage : trying to create view of base class from derived. Generates compile error (as expected).
		// TStridedView<FMyStruct> ViewDerivedAsBase = MakeStridedView(StructsDerived);

		// Explicitly construct view of base from derived
		TStridedView<FMyStruct> ViewDerivedAsBase(sizeof(StructsDerived[0]), &static_cast<FMyStruct&>(StructsDerived[0]), Num);
		TStridedView<FMyStruct> ViewDerivedAsBase2 = MakeStridedViewOfBase<FMyStruct>(MakeArrayView(StructsDerived));

		CHECK_MESSAGE(TEXT("ViewDerived.GetStride()"), ViewDerived.GetStride() == (int32)sizeof(FMyStructDerived));
		CHECK_MESSAGE(TEXT("ViewDerivedAsBase.GetStride()"), ViewDerivedAsBase.GetStride() == (int32)sizeof(FMyStructDerived));
		CHECK_MESSAGE(TEXT("ViewDerivedAsBase2.GetStride()"), ViewDerivedAsBase2.GetStride() == (int32)sizeof(FMyStructDerived));

		// Incorrect API usage : implicit conversion of derived view to base. Generates compile error (as expected).
		// FVector Avg = ComputeMeanPositionStrided(ViewDerived);

		FVector AvgDerivedAsBase = ComputeMeanPositionStrided(ViewDerivedAsBase);
		CHECK_MESSAGE(TEXT("ComputeMeanPositionStrided(ViewDerivedAsBase)"), AvgDerivedAsBase == ExpectedAvg);

		FVector AvgDerivedAsBase2 = ComputeMeanPositionStrided(ViewDerivedAsBase2);
		CHECK_MESSAGE(TEXT("ComputeMeanPositionStrided(ViewDerivedAsBase2)"), AvgDerivedAsBase2 == ExpectedAvg);
	}

	{
		TArrayView<FMyStructDerived> DerivedArrayView = MakeArrayView(StructsDerived);
		TStridedView<FVector> View = MakeStridedView(DerivedArrayView, &FMyStructDerived::Position);
		CHECK_MESSAGE(TEXT("MakeStridedView(MakeArrayView(StructsDerived), &FMyStructDerived::Position).GetStride()"), View.GetStride() == (int32)sizeof(StructsDerived[0]));
		FVector Avg = ComputeMean(View);
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(MakeArrayView(StructsDerived), &FMyStructDerived::Position))"), Avg == ExpectedAvg);
	}

	{
		TArrayView<FMyStructDerived> DerivedArrayView = MakeArrayView(StructsDerived);
		TStridedView<FVector> View = MakeStridedView(DerivedArrayView, &FMyStruct::Position);
		CHECK_MESSAGE(TEXT("MakeStridedView(MakeArrayView(StructsDerived), &FMyStruct::Position).GetStride()"), View.GetStride() == (int32)sizeof(StructsDerived[0]));
		FVector Avg = ComputeMean(View);
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(MakeArrayView(StructsDerived), &FMyStruct::Position))"), Avg == ExpectedAvg);
	}

	{
		FVector Avg = ComputeMean(MakeStridedView(StructsDerived, &FMyStructDerived::Position));
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(StructsDerived, &FMyStructDerived::Position))"), Avg == ExpectedAvg);
	}

	{
		TArray<FMyStructDerived> StructsArrayDerived = TArray<FMyStructDerived>(StructsDerived, Num);
		FVector Avg = ComputeMean(MakeStridedView(StructsArrayDerived, &FMyStructDerived::Position));
		CHECK_MESSAGE(TEXT("ComputeMean(MakeStridedView(StructsArrayDerived, &FMyStructDerived::Position))"), Avg == ExpectedAvg);
	}

	// Test iterating through nested fields

	{
		TStridedView<FVector::FReal> ViewOfZs = MakeStridedView((int32)sizeof(Structs[0]), &Structs[0].Position.Z, Num);
		FVector::FReal ExpectedSum = 0.0;
		for (int32 i = 0; i < Num; ++i)
		{
			ExpectedSum += Structs[i].Position.Z;
		}
		FVector::FReal Sum = Algo::Accumulate(ViewOfZs, FVector::FReal(0));
		CHECK_MESSAGE(TEXT("Algo::Accumulate(MakeStridedView((int32)sizeof(Structs[0]), &Structs[0].Position.Z, Num), 0.0f)"), Sum == ExpectedSum);
	}

}


#endif // WITH_TESTS
