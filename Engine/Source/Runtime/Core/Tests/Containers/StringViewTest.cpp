// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/StringView.h"

#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"
#include "Tests/TestHarnessAdapter.h"

static_assert(std::is_same_v<typename FStringView::ElementType, TCHAR>, "FStringView must use TCHAR.");
static_assert(std::is_same_v<typename FAnsiStringView::ElementType, ANSICHAR>, "FAnsiStringView must use ANSICHAR.");
static_assert(std::is_same_v<typename FWideStringView::ElementType, WIDECHAR>, "FWideStringView must use WIDECHAR.");

static_assert(std::is_same_v<FStringView, TStringView<TCHAR>>, "FStringView must be the same as TStringView<TCHAR>.");
static_assert(std::is_same_v<FAnsiStringView, TStringView<ANSICHAR>>, "FAnsiStringView must be the same as TStringView<ANSICHAR>.");
static_assert(std::is_same_v<FWideStringView, TStringView<WIDECHAR>>, "FWideStringView must be the same as TStringView<WIDECHAR>.");

static_assert(TIsContiguousContainer<FStringView>::Value, "FStringView must be a contiguous container.");
static_assert(TIsContiguousContainer<FAnsiStringView>::Value, "FAnsiStringView must be a contiguous container.");
static_assert(TIsContiguousContainer<FWideStringView>::Value, "FWideStringView must be a contiguous container.");

namespace UE::String::Private::TestArgumentDependentLookup
{

struct FTestType
{
	using ElementType = TCHAR;
};

const TCHAR* GetData(const FTestType&) { return TEXT("ABC"); }
int32 GetNum(const FTestType&) { return 3; }
} // UE::String::Private::TestArgumentDependentLookup

template <> struct TIsContiguousContainer<UE::String::Private::TestArgumentDependentLookup::FTestType> { static constexpr bool Value = true; };

TEST_CASE_NAMED(FStringViewConstructorTest, "System::Core::String::StringView::Constructor", "[Core][String][SmokeFilter]")
{
	SECTION("Default View")
	{
		FStringView View;
		CHECK(View.Len() == 0);
		CHECK(View.IsEmpty());
	}

	SECTION("Empty View")
	{
		FStringView View(TEXT(""));
		CHECK(View.Len() == 0);
		CHECK(View.IsEmpty());
	}

	SECTION("Construct from nullptr")
	{
		FStringView View(nullptr);
		CHECK(View.Len() == 0);
		CHECK(View.IsEmpty());
	}

	SECTION("Construct from TCHAR Literal")
	{
		FStringView View(TEXT("Test Ctor"));
		CHECK(View.Len() == FCString::Strlen(TEXT("Test Ctor")));
		CHECK(FCString::Strncmp(View.GetData(), TEXT("Test Ctor"), View.Len()) == 0);
		CHECK_FALSE(View.IsEmpty());
	}

	SECTION("Construct from Partial TCHAR Literal")
	{
		FStringView View(TEXT("Test SubSection Ctor"), 4);
		CHECK(View.Len() == 4);
		CHECK(FCString::Strncmp(View.GetData(), TEXT("Test"), View.Len()) == 0);
		CHECK_FALSE(View.IsEmpty());
	}

	SECTION("Construct from FString")
	{
		FString String("String Object");
		FStringView View(String);

		CHECK(View.Len() == String.Len());
		CHECK(FCStringWide::Strncmp(View.GetData(), *String, View.Len()) == 0);
		CHECK_FALSE(View.IsEmpty());
	}

	SECTION("Construct from ANSICHAR Literal")
	{
		FAnsiStringView View("Test Ctor");
		CHECK(View.Len() == FCStringAnsi::Strlen("Test Ctor"));
		CHECK(FCStringAnsi::Strncmp(View.GetData(), "Test Ctor", View.Len()) == 0);
		CHECK_FALSE(View.IsEmpty());
	}

	SECTION("Construct from Partial ANSICHAR Literal")
	{
		FAnsiStringView View("Test SubSection Ctor", 4);
		CHECK(View.Len() == 4);
		CHECK(FCStringAnsi::Strncmp(View.GetData(), "Test", View.Len()) == 0);
		CHECK_FALSE(View.IsEmpty());
	}

	SECTION("Construct from StringView Literals")
	{
		FStringView View = TEXTVIEW("Test");
		FAnsiStringView ViewAnsi = ANSITEXTVIEW("Test");
		FWideStringView ViewWide = WIDETEXTVIEW("Test");
	}

	SECTION("Construct using Class Template Argument Deduction")
	{
		TStringView ViewAnsiLiteral((ANSICHAR*)"Test");
		TStringView ViewWideLiteral((WIDECHAR*)WIDETEXT("Test"));
		TStringView ViewUtf8Literal((UTF8CHAR*)UTF8TEXT("Test"));
		STATIC_CHECK(std::is_same_v<decltype(ViewAnsiLiteral)::ElementType, ANSICHAR>);
		STATIC_CHECK(std::is_same_v<decltype(ViewWideLiteral)::ElementType, WIDECHAR>);
		STATIC_CHECK(std::is_same_v<decltype(ViewUtf8Literal)::ElementType, UTF8CHAR>);

		TStringView ViewAnsiConstLiteral((const ANSICHAR*)"Test");
		TStringView ViewWideConstLiteral((const WIDECHAR*)WIDETEXT("Test"));
		TStringView ViewUtf8ConstLiteral((const UTF8CHAR*)UTF8TEXT("Test"));
		STATIC_CHECK(std::is_same_v<decltype(ViewAnsiConstLiteral)::ElementType, ANSICHAR>);
		STATIC_CHECK(std::is_same_v<decltype(ViewWideConstLiteral)::ElementType, WIDECHAR>);
		STATIC_CHECK(std::is_same_v<decltype(ViewUtf8ConstLiteral)::ElementType, UTF8CHAR>);

		auto WriteAnsiString = WriteToAnsiString<16>("Test");
 		auto WriteWideString = WriteToWideString<16>(WIDETEXT("Test"));
 		auto WriteUft8String = WriteToUtf8String<16>(UTF8TEXT("Test"));
		TStringView ViewAnsiStringBuilder(WriteAnsiString);
		TStringView ViewWideStringBuilder(WriteWideString);
		TStringView ViewUtf8StringBuilder(WriteUft8String);
		STATIC_CHECK(std::is_same_v<decltype(ViewAnsiStringBuilder)::ElementType, ANSICHAR>);
		STATIC_CHECK(std::is_same_v<decltype(ViewWideStringBuilder)::ElementType, WIDECHAR>);
		STATIC_CHECK(std::is_same_v<decltype(ViewUtf8StringBuilder)::ElementType, UTF8CHAR>);

		FString String("Test");
		TStringView ViewString(String);
		STATIC_CHECK(std::is_same_v<decltype(ViewString)::ElementType, WIDECHAR>);
	}

	SECTION("Verify that argument-dependent lookup is working for GetData and GetNum")
	{
		UE::String::Private::TestArgumentDependentLookup::FTestType Test;
		FStringView View(Test);
		CHECK(View.Equals(TEXTVIEW("ABC")));
	}
}

TEST_CASE_NAMED(FStringViewIteratorsTest, "System::Core::String::StringView::Iterators", "[Core][String][SmokeFilter]")
{
	SECTION("View Iterator")
	{
		const TCHAR* StringLiteralSrc = TEXT("Iterator!");
		FStringView View(StringLiteralSrc);
		for (TCHAR C : View)
		{
			CHECK(C == *StringLiteralSrc++);
		}
		CHECK(*StringLiteralSrc == TEXT('\0'));
	}

	SECTION("Partial View Iterator")
	{
		const TCHAR* StringLiteralSrc = TEXT("Iterator|with extras!");
		FStringView View(StringLiteralSrc, 8);
		for (TCHAR C : View)
		{
			CHECK(C == *StringLiteralSrc++);
		}
		CHECK(*StringLiteralSrc == TEXT('|'));
	}
}

TEST_CASE_NAMED(FStringViewEqualityTest, "System::Core::String::StringView::Equality", "[Core][String][SmokeFilter]")
{
	const ANSICHAR* AnsiStringLiteralSrc = "String To Test!";
	const ANSICHAR* AnsiStringLiteralLower = "string to test!";
	const ANSICHAR* AnsiStringLiteralUpper = "STRING TO TEST!";
	const TCHAR* WideStringLiteralSrc = TEXT("String To Test!");
	const TCHAR* WideStringLiteralLower = TEXT("string to test!");
	const TCHAR* WideStringLiteralUpper = TEXT("STRING TO TEST!");
	const TCHAR* WideStringLiteralShort = TEXT("String To");
	const TCHAR* WideStringLiteralLonger = TEXT("String To Test! Extended");

	FStringView WideView(WideStringLiteralSrc);

	SECTION("TCHAR Equals TCHAR Literal")
	{
		CHECK(WideView == WideStringLiteralSrc);
		CHECK_FALSE(WideView != WideStringLiteralSrc);
		CHECK(WideView == WideStringLiteralLower);
		CHECK_FALSE(WideView != WideStringLiteralLower);
		CHECK(WideView == WideStringLiteralUpper);
		CHECK_FALSE(WideView != WideStringLiteralUpper);
		CHECK_FALSE(WideView == WideStringLiteralShort);
		CHECK(WideView != WideStringLiteralShort);
		CHECK_FALSE(WideView == WideStringLiteralLonger);
		CHECK(WideView != WideStringLiteralLonger);

		CHECK(WideStringLiteralSrc == WideView);
		CHECK_FALSE(WideStringLiteralSrc != WideView);
		CHECK(WideStringLiteralLower == WideView);
		CHECK_FALSE(WideStringLiteralLower != WideView);
		CHECK(WideStringLiteralUpper == WideView);
		CHECK_FALSE(WideStringLiteralUpper != WideView);
		CHECK_FALSE(WideStringLiteralShort == WideView);
		CHECK(WideStringLiteralShort != WideView);
		CHECK_FALSE(WideStringLiteralLonger == WideView);
		CHECK(WideStringLiteralLonger != WideView);
	}

	FString WideStringSrc = WideStringLiteralSrc;
	FString WideStringLower = WideStringLiteralLower;
	FString WideStringUpper = WideStringLiteralUpper;
	FString WideStringShort = WideStringLiteralShort;
	FString WideStringLonger = WideStringLiteralLonger;

	SECTION("TCHAR Equals FString")
	{
		CHECK(WideView == WideStringSrc);
		CHECK_FALSE(WideView != WideStringSrc);
		CHECK(WideView == WideStringLower);
		CHECK_FALSE(WideView != WideStringLower);
		CHECK(WideView == WideStringUpper);
		CHECK_FALSE(WideView != WideStringUpper);
		CHECK_FALSE(WideView == WideStringShort);
		CHECK(WideView != WideStringShort);
		CHECK_FALSE(WideView == WideStringLonger);
		CHECK(WideView != WideStringLonger);

		CHECK(WideStringSrc == WideView);
		CHECK_FALSE(WideStringSrc != WideView);
		CHECK(WideStringLower == WideView);
		CHECK_FALSE(WideStringLower != WideView);
		CHECK(WideStringUpper == WideView);
		CHECK_FALSE(WideStringUpper != WideView);
		CHECK_FALSE(WideStringShort == WideView);
		CHECK(WideStringShort != WideView);
		CHECK_FALSE(WideStringLonger == WideView);
		CHECK(WideStringLonger != WideView);
	}

	SECTION("TCHAR Equals Identical")
	{
		FStringView IdenticalView(WideStringLiteralSrc);

		CHECK(WideView == IdenticalView);
		CHECK_FALSE(WideView != IdenticalView);
		CHECK(IdenticalView == WideView);
		CHECK_FALSE(IdenticalView != WideView);
	}

	SECTION("ANSI Equals ANSI NoTerminator")
	{
		FStringView ShortViewNoNull = WideView.Left(FStringView(WideStringLiteralShort).Len());

		CHECK(ShortViewNoNull == WideStringLiteralShort);
		CHECK_FALSE(ShortViewNoNull != WideStringLiteralShort);
		CHECK(WideStringLiteralShort == ShortViewNoNull);
		CHECK_FALSE(WideStringLiteralShort != ShortViewNoNull);
		CHECK_FALSE(ShortViewNoNull == WideStringLiteralSrc);
		CHECK(ShortViewNoNull != WideStringLiteralSrc);
		CHECK_FALSE(WideStringLiteralSrc == ShortViewNoNull);
		CHECK(WideStringLiteralSrc != ShortViewNoNull);

		CHECK(ShortViewNoNull == WideStringShort);
		CHECK_FALSE(ShortViewNoNull != WideStringShort);
		CHECK(WideStringShort == ShortViewNoNull);
		CHECK_FALSE(WideStringShort != ShortViewNoNull);
		CHECK_FALSE(ShortViewNoNull == WideStringSrc);
		CHECK(ShortViewNoNull != WideStringSrc);
		CHECK_FALSE(WideStringSrc == ShortViewNoNull);
		CHECK(WideStringSrc != ShortViewNoNull);
	}

	SECTION("TCHAR Equals TCHAR NoTerminator")
	{
		FStringView WideViewNoNull = FStringView(WideStringLiteralLonger).Left(WideView.Len());

		CHECK(WideViewNoNull == WideStringLiteralSrc);
		CHECK_FALSE(WideViewNoNull != WideStringLiteralSrc);
		CHECK(WideStringLiteralSrc == WideViewNoNull);
		CHECK_FALSE(WideStringLiteralSrc != WideViewNoNull);
		CHECK_FALSE(WideViewNoNull == WideStringLiteralLonger);
		CHECK(WideViewNoNull != WideStringLiteralLonger);
		CHECK_FALSE(WideStringLiteralLonger == WideViewNoNull);
		CHECK(WideStringLiteralLonger != WideViewNoNull);

		CHECK(WideViewNoNull == WideStringLiteralSrc);
		CHECK_FALSE(WideViewNoNull != WideStringLiteralSrc);
		CHECK(WideStringLiteralSrc == WideViewNoNull);
		CHECK_FALSE(WideStringLiteralSrc != WideViewNoNull);
		CHECK_FALSE(WideViewNoNull == WideStringLiteralLonger);
		CHECK(WideViewNoNull != WideStringLiteralLonger);
		CHECK_FALSE(WideStringLiteralLonger == WideViewNoNull);
		CHECK(WideStringLiteralLonger != WideViewNoNull);
	}

	SECTION("ANSICHAR Equals TCHAR")
	{
		FAnsiStringView AnsiView(AnsiStringLiteralSrc);
		FAnsiStringView AnsiViewLower(AnsiStringLiteralLower);
		FAnsiStringView AnsiViewUpper(AnsiStringLiteralUpper);

		CHECK(AnsiView.Equals(WideView));
		CHECK(WideView.Equals(AnsiView));
		CHECK_FALSE(AnsiViewLower.Equals(WideView, ESearchCase::CaseSensitive));
		CHECK(AnsiViewLower.Equals(WideView, ESearchCase::IgnoreCase));
		CHECK_FALSE(WideView.Equals(AnsiViewLower, ESearchCase::CaseSensitive));
		CHECK(WideView.Equals(AnsiViewLower, ESearchCase::IgnoreCase));
		CHECK_FALSE(AnsiViewUpper.Equals(WideView, ESearchCase::CaseSensitive));
		CHECK(AnsiViewUpper.Equals(WideView, ESearchCase::IgnoreCase));
		CHECK_FALSE(WideView.Equals(AnsiViewUpper, ESearchCase::CaseSensitive));
		CHECK(WideView.Equals(AnsiViewUpper, ESearchCase::IgnoreCase));

		CHECK(WideView.Equals(AnsiStringLiteralSrc));
		CHECK_FALSE(WideView.Equals(AnsiStringLiteralLower, ESearchCase::CaseSensitive));
		CHECK(WideView.Equals(AnsiStringLiteralLower, ESearchCase::IgnoreCase));
		CHECK_FALSE(WideView.Equals(AnsiStringLiteralUpper, ESearchCase::CaseSensitive));
		CHECK(WideView.Equals(AnsiStringLiteralUpper, ESearchCase::IgnoreCase));
		CHECK(AnsiView.Equals(WideStringLiteralSrc));
		CHECK_FALSE(AnsiViewLower.Equals(WideStringLiteralSrc, ESearchCase::CaseSensitive));
		CHECK(AnsiViewLower.Equals(WideStringLiteralSrc, ESearchCase::IgnoreCase));
		CHECK_FALSE(AnsiViewUpper.Equals(WideStringLiteralSrc, ESearchCase::CaseSensitive));
		CHECK(AnsiViewUpper.Equals(WideStringLiteralSrc, ESearchCase::IgnoreCase));
	}

	SECTION("Empty String Equality")
	{
		const TCHAR* EmptyLiteral = TEXT("");
		const TCHAR* NonEmptyLiteral = TEXT("ABC");
		FStringView EmptyView;
		FStringView NonEmptyView = TEXTVIEW("ABC");
		CHECK(EmptyView.Equals(EmptyLiteral));
		CHECK_FALSE(EmptyView.Equals(NonEmptyLiteral));
		CHECK_FALSE(NonEmptyView.Equals(EmptyLiteral));
		CHECK(EmptyView.Equals(EmptyView));
		CHECK_FALSE(EmptyView.Equals(NonEmptyView));
		CHECK_FALSE(NonEmptyView.Equals(EmptyView));
	}

	SECTION("Types Convertible to TStringView")
	{
		STATIC_CHECK(std::is_same_v<bool, decltype(FAnsiStringView().Equals(FString()))>);
		STATIC_CHECK(std::is_same_v<bool, decltype(FWideStringView().Equals(FString()))>);
		STATIC_CHECK(std::is_same_v<bool, decltype(FAnsiStringView().Equals(TAnsiStringBuilder<16>()))>);
		STATIC_CHECK(std::is_same_v<bool, decltype(FAnsiStringView().Equals(TWideStringBuilder<16>()))>);
		STATIC_CHECK(std::is_same_v<bool, decltype(FWideStringView().Equals(TAnsiStringBuilder<16>()))>);
		STATIC_CHECK(std::is_same_v<bool, decltype(FWideStringView().Equals(TWideStringBuilder<16>()))>);
	}
}

TEST_CASE_NAMED(FStringViewComparisonCaseSensitiveTest, "System::Core::String::StringView::ComparisonCaseSensitive", "[Core][String][SmokeFilter]")
{
	SECTION("Basic comparisons involving case")
	{
		const ANSICHAR* AnsiStringLiteralSrc = "String To Test!";
		const TCHAR* WideStringLiteralSrc = TEXT("String To Test!");
		const TCHAR* WideStringLiteralLower = TEXT("string to test!");
		const TCHAR* WideStringLiteralUpper = TEXT("STRING TO TEST!");

		FStringView WideView(WideStringLiteralSrc);

		CHECK(WideView.Compare(WideStringLiteralSrc, ESearchCase::CaseSensitive) == 0);
		CHECK_FALSE(WideView.Compare(WideStringLiteralLower, ESearchCase::CaseSensitive) > 0);
		CHECK_FALSE(WideView.Compare(WideStringLiteralUpper, ESearchCase::CaseSensitive) < 0);

		FStringView EmptyView(TEXT(""));
		CHECK(WideView.Compare(EmptyView, ESearchCase::CaseSensitive) > 0);

		FStringView IdenticalView(WideStringLiteralSrc);
		CHECK(WideView.Compare(IdenticalView, ESearchCase::CaseSensitive) == 0);

		FAnsiStringView AnsiView(AnsiStringLiteralSrc);
		CHECK(WideView.Compare(AnsiView, ESearchCase::CaseSensitive) == 0);
		CHECK(WideView.Compare(AnsiStringLiteralSrc, ESearchCase::CaseSensitive) == 0);
	}

	SECTION("Test comparisons of different lengths")
	{
		const ANSICHAR* AnsiStringLiteralUpper = "ABCDEF";
		const TCHAR* WideStringLiteralUpper = TEXT("ABCDEF");
		const TCHAR* WideStringLiteralLower = TEXT("abcdef");
		const TCHAR* WideStringLiteralLowerShort = TEXT("abc");

		const ANSICHAR* AnsiStringLiteralUpperFirst = "ABCdef";
		const TCHAR* WideStringLiteralUpperFirst = TEXT("ABCdef");
		const TCHAR* WideStringLiteralLowerFirst = TEXT("abcDEF");

		FStringView ViewLongUpper(WideStringLiteralUpper);
		FStringView ViewLongLower(WideStringLiteralLower);

		// Note that the characters after these views are in a different case, this will help catch over read issues
		FStringView ViewShortUpper(WideStringLiteralUpperFirst, 3);
		FStringView ViewShortLower(WideStringLiteralLowerFirst, 3);

		// Same length, different cases
		CHECK(ViewLongUpper.Compare(ViewLongLower, ESearchCase::CaseSensitive) < 0);
		CHECK(ViewLongLower.Compare(ViewLongUpper, ESearchCase::CaseSensitive) > 0);
		CHECK(ViewLongLower.Compare(AnsiStringLiteralUpper, ESearchCase::CaseSensitive) > 0);
		CHECK(ViewShortUpper.Compare(WideStringLiteralLowerShort, ESearchCase::CaseSensitive) < 0);

		// Same case, different lengths
		CHECK(ViewLongUpper.Compare(ViewShortUpper, ESearchCase::CaseSensitive) > 0);
		CHECK(ViewShortUpper.Compare(ViewLongUpper, ESearchCase::CaseSensitive) < 0);
		CHECK(ViewShortUpper.Compare(AnsiStringLiteralUpper, ESearchCase::CaseSensitive) < 0);
		CHECK(ViewLongLower.Compare(WideStringLiteralLowerShort, ESearchCase::CaseSensitive) > 0);

		// Different length, different cases
		CHECK(ViewLongUpper.Compare(ViewShortLower, ESearchCase::CaseSensitive) < 0);
		CHECK(ViewShortLower.Compare(ViewLongUpper, ESearchCase::CaseSensitive) > 0);
		CHECK(ViewShortLower.Compare(AnsiStringLiteralUpper, ESearchCase::CaseSensitive) > 0);
		CHECK(ViewLongUpper.Compare(WideStringLiteralLowerShort, ESearchCase::CaseSensitive) < 0);
	}

	SECTION("Test comparisons of empty strings")
	{
		const TCHAR* EmptyLiteral = TEXT("");
		const TCHAR* NonEmptyLiteral = TEXT("ABC");
		FStringView EmptyView;
		FStringView NonEmptyView = TEXTVIEW("ABC");
		CHECK(EmptyView.Compare(EmptyLiteral) == 0);
		CHECK(EmptyView.Compare(NonEmptyLiteral) < 0);
		CHECK(NonEmptyView.Compare(EmptyLiteral) > 0);
		CHECK(EmptyView.Compare(EmptyView) == 0);
		CHECK(EmptyView.Compare(NonEmptyView) < 0);
		CHECK(NonEmptyView.Compare(EmptyView) > 0);
	}

 	SECTION("Test types convertible to a string view")
	{
		STATIC_CHECK(std::is_same_v<int32, decltype(FAnsiStringView().Compare(FString()))>);
		STATIC_CHECK(std::is_same_v<int32, decltype(FWideStringView().Compare(FString()))>);
		STATIC_CHECK(std::is_same_v<int32, decltype(FAnsiStringView().Compare(TAnsiStringBuilder<16>()))>);
		STATIC_CHECK(std::is_same_v<int32, decltype(FAnsiStringView().Compare(TWideStringBuilder<16>()))>);
		STATIC_CHECK(std::is_same_v<int32, decltype(FWideStringView().Compare(TAnsiStringBuilder<16>()))>);
		STATIC_CHECK(std::is_same_v<int32, decltype(FWideStringView().Compare(TWideStringBuilder<16>()))>);
	}
}

TEST_CASE_NAMED(FStringViewComparisonCaseInsensitiveTest, "System::Core::String::StringView::ComparisonCaseInsensitive", "[Core][String][SmokeFilter]")
{
	SECTION("Basic comparisons involving case")
	{
		const ANSICHAR* AnsiStringLiteralSrc = "String To Test!";
		const TCHAR* WideStringLiteralSrc = TEXT("String To Test!");
		const TCHAR* WideStringLiteralLower = TEXT("string to test!");
		const TCHAR* WideStringLiteralUpper = TEXT("STRING TO TEST!");

		FStringView WideView(WideStringLiteralSrc);

		CHECK(WideView.Compare(WideStringLiteralSrc, ESearchCase::IgnoreCase) == 0);
		CHECK(WideView.Compare(WideStringLiteralLower, ESearchCase::IgnoreCase) == 0);
		CHECK(WideView.Compare(WideStringLiteralUpper, ESearchCase::IgnoreCase) == 0);

		FStringView EmptyView(TEXT(""));
		CHECK(WideView.Compare(EmptyView, ESearchCase::IgnoreCase) > 0);

		FStringView IdenticalView(WideStringLiteralSrc);
		CHECK(WideView.Compare(IdenticalView, ESearchCase::IgnoreCase) == 0);

		FAnsiStringView AnsiView(AnsiStringLiteralSrc);
		CHECK(WideView.Compare(AnsiView, ESearchCase::IgnoreCase) == 0);
		CHECK(WideView.Compare(AnsiStringLiteralSrc, ESearchCase::IgnoreCase) == 0);
	}

	SECTION("Test comparisons of different lengths")
	{
		const ANSICHAR* AnsiStringLiteralUpper = "ABCDEF";
		const TCHAR* WideStringLiteralUpper = TEXT("ABCDEF");
		const TCHAR* WideStringLiteralLower = TEXT("abcdef");
		const TCHAR* WideStringLiteralLowerShort = TEXT("abc");

		const ANSICHAR* AnsiStringLiteralUpperFirst = "ABCdef";
		const TCHAR* WideStringLiteralUpperFirst = TEXT("ABCdef");
		const TCHAR* WideStringLiteralLowerFirst = TEXT("abcDEF");

		FStringView ViewLongUpper(WideStringLiteralUpper);
		FStringView ViewLongLower(WideStringLiteralLower);

		// Note that the characters after these views are in a different case, this will help catch over read issues
		FStringView ViewShortUpper(WideStringLiteralUpperFirst, 3);
		FStringView ViewShortLower(WideStringLiteralLowerFirst, 3);

		// Same length, different cases
		CHECK(ViewLongUpper.Compare(ViewLongLower, ESearchCase::IgnoreCase) == 0);
		CHECK(ViewLongLower.Compare(ViewLongUpper, ESearchCase::IgnoreCase) == 0);
		CHECK(ViewLongLower.Compare(AnsiStringLiteralUpper, ESearchCase::IgnoreCase) == 0);
		CHECK(ViewShortUpper.Compare(WideStringLiteralLowerShort, ESearchCase::IgnoreCase) == 0);

		// Same case, different lengths
		CHECK(ViewLongUpper.Compare(ViewShortUpper, ESearchCase::IgnoreCase) > 0);
		CHECK(ViewShortUpper.Compare(ViewLongUpper, ESearchCase::IgnoreCase) < 0);
		CHECK(ViewShortUpper.Compare(AnsiStringLiteralUpper, ESearchCase::IgnoreCase) < 0);
		CHECK(ViewLongLower.Compare(WideStringLiteralLowerShort, ESearchCase::IgnoreCase) > 0);

		// Different length, different cases
		CHECK(ViewLongUpper.Compare(ViewShortLower, ESearchCase::IgnoreCase) > 0);
		CHECK(ViewShortLower.Compare(ViewLongUpper, ESearchCase::IgnoreCase) < 0);
		CHECK(ViewShortLower.Compare(AnsiStringLiteralUpper, ESearchCase::IgnoreCase) < 0);
		CHECK(ViewLongUpper.Compare(WideStringLiteralLowerShort, ESearchCase::IgnoreCase) > 0);
	}
}

TEST_CASE_NAMED(FStringViewArrayAccessorTest, "System::Core::String::StringView::ArrayAccessor", "[Core][String][SmokeFilter]")
{
	const TCHAR* SrcString = TEXT("String To Test");
	FStringView View(SrcString);

	for (int32 Index = 0; Index < View.Len(); ++Index)
	{
		CHECK(View[Index] == SrcString[Index]);
	}
}

TEST_CASE_NAMED(FStringViewModifiersTest, "System::Core::String::StringView::Modifiers", "[Core][String][SmokeFilter]")
{
	const TCHAR* FullText = TEXT("PrefixSuffix");
	const TCHAR* Prefix = TEXT("Prefix");
	const TCHAR* Suffix = TEXT("Suffix");

	SECTION("Remove prefix")
	{
		FStringView View(FullText);
		View.RemovePrefix(FCStringWide::Strlen(Prefix));

		CHECK(View.Len() == FCStringWide::Strlen(Suffix));
		CHECK(FCStringWide::Strncmp(View.GetData(), Suffix, View.Len()) == 0);
	}

	SECTION("Remove suffix")
	{
		FStringView View(FullText);
		View.RemoveSuffix(FCStringWide::Strlen(Suffix));

		CHECK(View.Len() == FCStringWide::Strlen(Prefix));
		CHECK(FCStringWide::Strncmp(View.GetData(), Prefix, View.Len()) == 0);
	}
}

TEST_CASE_NAMED(FStringViewStartsWithTest, "System::Core::String::StringView::StartsWith", "[Core][String][SmokeFilter]")
{
	SECTION("Test an empty view")
	{
		FStringView View;
		CHECK(View.StartsWith(TEXT("")));
		CHECK_FALSE(View.StartsWith(TEXT("Text")));
		CHECK_FALSE(View.StartsWith(TEXT('A')));
	}

	SECTION("Test a valid view with the correct text")
	{
		FStringView View(TEXT("String to test"));
		CHECK(View.StartsWith(TEXT("String")));
		CHECK(View.StartsWith(TEXT('S')));
	}

	SECTION("Test a valid view with incorrect text")
	{
		FStringView View(TEXT("String to test"));
		CHECK_FALSE(View.StartsWith(TEXT("test")));
		CHECK_FALSE(View.StartsWith(TEXT('t')));
	}

	SECTION("Test a valid view with the correct text but with different case")
	{
		FStringView View(TEXT("String to test"));
		CHECK(View.StartsWith(TEXT("sTrInG")));

		// Searching by char is case sensitive to keep compatibility with FString
		CHECK_FALSE(View.StartsWith(TEXT('s')));
	}
}

TEST_CASE_NAMED(FStringViewEndsWithTest, "System::Core::String::StringView::EndsWith", "[Core][String][SmokeFilter]")
{
	SECTION("Test an empty view")
	{
		FStringView View;
		CHECK(View.EndsWith(TEXT("")));
		CHECK_FALSE(View.EndsWith(TEXT("Text")));
		CHECK_FALSE(View.EndsWith(TEXT('A')));
	}

	SECTION("Test a valid view with the correct text")
	{
		FStringView View(TEXT("String to test"));
		CHECK(View.EndsWith(TEXT("test")));
		CHECK(View.EndsWith(TEXT('t')));
	}

	SECTION("Test a valid view with incorrect text")
	{
		FStringView View(TEXT("String to test"));
		CHECK_FALSE(View.EndsWith(TEXT("String")));
		CHECK_FALSE(View.EndsWith(TEXT('S')));
	}

	SECTION("Test a valid view with the correct text but with different case")
	{
		FStringView View(TEXT("String to test"));
		CHECK(View.EndsWith(TEXT("TeST")));

		// Searching by char is case sensitive to keep compatibility with FString
		CHECK_FALSE(View.EndsWith(TEXT('T'))); 
	}
}

TEST_CASE_NAMED(FStringViewStringSubStrTest, "System::Core::String::StringView::SubStr", "[Core][String][SmokeFilter]")
{
	{
		FStringView EmptyView;
		FStringView EmptyResult = EmptyView.SubStr(0, 10);
		CHECK(EmptyResult.IsEmpty());

		// The following line is commented out as it would fail an assert and currently we cannot test for this in unit tests 
		// FStringView OutofBoundsResult = EmptyView.SubStr(1000, 10000); 
		FStringView OutofBoundsResult = EmptyView.SubStr(0, 10000);
		CHECK(OutofBoundsResult.IsEmpty());
	}

	{
		FStringView View(TEXT("A test string|"));
		FStringView Word0 = View.SubStr(0, 1);
		FStringView Word1 = View.SubStr(2, 4);
		FStringView Word2 = View.SubStr(7, 6);
		FStringView NullTerminatorResult = View.SubStr(14, 1024);	// We can create a substr that starts at the end of the 
																	// string since the null terminator is still valid
		FStringView OutofBoundsResult = View.SubStr(0, 1024);

		CHECK(FCString::Strncmp(Word0.GetData(), TEXT("A"), Word0.Len()) == 0);
		CHECK(FCString::Strncmp(Word1.GetData(), TEXT("test"), Word1.Len()) == 0);
		CHECK(FCString::Strncmp(Word2.GetData(), TEXT("string"), Word2.Len()) == 0);
		CHECK(NullTerminatorResult.IsEmpty());
		CHECK(View == OutofBoundsResult);
	}
}

TEST_CASE_NAMED(FStringViewLeftTest, "System::Core::String::StringView::Left", "[Core][String][SmokeFilter]")
{
	{
		FStringView EmptyView;
		FStringView EmptyResult = EmptyView.Left(0);
		CHECK(EmptyResult.IsEmpty());

		FStringView OutofBoundsResult = EmptyView.Left(1024);
		CHECK(OutofBoundsResult.IsEmpty());
	}
	
	{
		FStringView View(TEXT("A test string padded"), 13); // "A test string" without null termination
		FStringView Result = View.Left(8);

		CHECK(FCString::Strncmp(Result.GetData(), TEXT("A test s"), Result.Len()) == 0);

		FStringView OutofBoundsResult = View.Left(1024);
		CHECK(FCString::Strncmp(OutofBoundsResult.GetData(), TEXT("A test string"), OutofBoundsResult.Len()) == 0);
	}
}

TEST_CASE_NAMED(FStringViewLeftChopTest, "System::Core::String::StringView::LeftChop", "[Core][String][SmokeFilter]")
{
	{
		FStringView EmptyView;
		FStringView EmptyResult = EmptyView.LeftChop(0);
		CHECK(EmptyResult.IsEmpty());

		FStringView OutofBoundsResult = EmptyView.LeftChop(1024);
		CHECK(OutofBoundsResult.IsEmpty());
	}

	{
		FStringView View(TEXT("A test string padded"), 13); // "A test string" without null termination
		FStringView Result = View.LeftChop(5);

		CHECK(FCString::Strncmp(Result.GetData(), TEXT("A test s"), Result.Len()) == 0);

		FStringView OutofBoundsResult = View.LeftChop(1024);
		CHECK(FCString::Strncmp(OutofBoundsResult.GetData(), TEXT("A test string"), OutofBoundsResult.Len()) == 0);
	}
}

TEST_CASE_NAMED(FStringViewRightTest, "System::Core::String::StringView::Right", "[Core][String][SmokeFilter]")
{
	{
		FStringView EmptyView;
		FStringView EmptyResult = EmptyView.Right(0);
		CHECK(EmptyResult.IsEmpty());

		FStringView OutofBoundsResult = EmptyView.Right(1024);
		CHECK(OutofBoundsResult.IsEmpty());
	}

	{
		FStringView View(TEXT("A test string padded"), 13); // "A test string" without null termination
		FStringView Result = View.Right(8);

		CHECK(FCString::Strncmp(Result.GetData(), TEXT("t string"), Result.Len()) == 0);

		FStringView OutofBoundsResult = View.Right(1024);
		CHECK(FCString::Strncmp(OutofBoundsResult.GetData(), TEXT("A test string"), OutofBoundsResult.Len()) == 0);
	}
}

TEST_CASE_NAMED(FStringViewRightChopTest, "System::Core::String::StringView::RightChop", "[Core][String][SmokeFilter]")
{
	{
		FStringView EmptyView;
		FStringView EmptyResult = EmptyView.RightChop(0);
		CHECK(EmptyResult.IsEmpty());

		FStringView OutofBoundsResult = EmptyView.RightChop(1024);
		CHECK(OutofBoundsResult.IsEmpty());
	}

	{
		FStringView View(TEXT("A test string padded"), 13); // "A test string" without null termination
		FStringView Result = View.RightChop(3);

		CHECK(FCString::Strncmp(Result.GetData(), TEXT("est string"), Result.Len()) == 0);

		FStringView OutofBoundsResult = View.RightChop(1024);
		CHECK(FCString::Strncmp(OutofBoundsResult.GetData(), TEXT("A test string"), OutofBoundsResult.Len()) == 0);
	}
}

TEST_CASE_NAMED(FStringViewMidTest, "System::Core::String::StringView::Mid", "[Core][String][SmokeFilter]")
{
	{
		FStringView EmptyView;
		FStringView EmptyResult = EmptyView.Mid(0, 10);
		CHECK(EmptyResult.IsEmpty());

		// The following line is commented out as it would fail an assert and currently we cannot test for this in unit tests 
		// FStringView OutofBoundsResult = EmptyView.Mid(1000, 10000); 
		FStringView OutofBoundsResult = EmptyView.Mid(0, 10000);
		CHECK(OutofBoundsResult.IsEmpty());
	}

	{
		FStringView View(TEXT("A test string|"));
		FStringView Word0 = View.Mid(0, 1);
		FStringView Word1 = View.Mid(2, 4);
		FStringView Word2 = View.Mid(7, 6);
		FStringView NullTerminatorResult = View.Mid(14, 1024);	// We can call Mid with a position that starts at the end of the 
																// string since the null terminator is still valid
		FStringView OutofBoundsResult = View.Mid(0, 1024);

		CHECK(FCString::Strncmp(Word0.GetData(), TEXT("A"), Word0.Len()) == 0);
		CHECK(FCString::Strncmp(Word1.GetData(), TEXT("test"), Word1.Len()) == 0);
		CHECK(FCString::Strncmp(Word2.GetData(), TEXT("string"), Word2.Len()) == 0);
		CHECK(NullTerminatorResult.IsEmpty());
		CHECK(View == OutofBoundsResult);
		CHECK(View.Mid(512, 1024).IsEmpty());
		CHECK(View.Mid(4, 0).IsEmpty());
	}
}

TEST_CASE_NAMED(FStringViewTrimStartAndEndTest, "System::Core::String::StringView::TrimStartAndEnd", "[Core][String][SmokeFilter]")
{
	CHECK(TEXTVIEW("").TrimStartAndEnd().IsEmpty());
	CHECK(TEXTVIEW(" ").TrimStartAndEnd().IsEmpty());
	CHECK(TEXTVIEW("  ").TrimStartAndEnd().IsEmpty());
	CHECK(TEXTVIEW(" \t\r\n").TrimStartAndEnd().IsEmpty());

	CHECK(TEXTVIEW("ABC123").TrimStartAndEnd() == TEXTVIEW("ABC123"));
	CHECK(TEXTVIEW("A \t\r\nB").TrimStartAndEnd() == TEXTVIEW("A \t\r\nB"));
	CHECK(TEXTVIEW(" \t\r\nABC123\n\r\t ").TrimStartAndEnd() == TEXTVIEW("ABC123"));
}

TEST_CASE_NAMED(FStringViewTrimStartTest, "System::Core::String::StringView::TrimStart", "[Core][String][SmokeFilter]")
{
	CHECK(TEXTVIEW("").TrimStart().IsEmpty());
	CHECK(TEXTVIEW(" ").TrimStart().IsEmpty());
	CHECK(TEXTVIEW("  ").TrimStart().IsEmpty());
	CHECK(TEXTVIEW(" \t\r\n").TrimStart().IsEmpty());

	CHECK(TEXTVIEW("ABC123").TrimStart() == TEXTVIEW("ABC123"));
	CHECK(TEXTVIEW("A \t\r\nB").TrimStart() == TEXTVIEW("A \t\r\nB"));
	CHECK(TEXTVIEW(" \t\r\nABC123\n\r\t ").TrimStart() == TEXTVIEW("ABC123\n\r\t "));
}

TEST_CASE_NAMED(FStringViewTrimEndTest, "System::Core::String::StringView::TrimEnd", "[Core][String][SmokeFilter]")
{
	CHECK(TEXTVIEW("").TrimEnd().IsEmpty());
	CHECK(TEXTVIEW(" ").TrimEnd().IsEmpty());
	CHECK(TEXTVIEW("  ").TrimEnd().IsEmpty());
	CHECK(TEXTVIEW(" \t\r\n").TrimEnd().IsEmpty());

	CHECK(TEXTVIEW("ABC123").TrimEnd() == TEXTVIEW("ABC123"));
	CHECK(TEXTVIEW("A \t\r\nB").TrimEnd() == TEXTVIEW("A \t\r\nB"));
	CHECK(TEXTVIEW(" \t\r\nABC123\n\r\t ").TrimEnd() == TEXTVIEW(" \t\r\nABC123"));
}

TEST_CASE_NAMED(FStringViewFindCharTest, "System::Core::String::StringView::FindChar", "[Core][String][SmokeFilter]")
{
	FStringView EmptyView;
	FStringView View = TEXT("aBce Fga");

	{
		int32 Index = INDEX_NONE;
		CHECK_FALSE(EmptyView.FindChar(TEXT('a'), Index));
		CHECK(Index == INDEX_NONE);
	}

	{
		int32 Index = INDEX_NONE;
		CHECK(View.FindChar(TEXT('a'), Index));
		CHECK(Index == 0);
	}

	{
		int32 Index = INDEX_NONE;
		CHECK(View.FindChar(TEXT('F'), Index));
		CHECK(Index == 5);
	}

	{
		int32 Index = INDEX_NONE;
		CHECK_FALSE(View.FindChar(TEXT('A'), Index));
		CHECK(Index == INDEX_NONE);
	}

	{
		int32 Index = INDEX_NONE;
		CHECK_FALSE(View.FindChar(TEXT('d'), Index));
		CHECK(Index == INDEX_NONE);
	}

	{
		int32 Index = INDEX_NONE;
		CHECK(View.FindChar(TEXT(' '), Index));
		CHECK(Index == 4);
	}
}

TEST_CASE_NAMED(FStringViewFindLastCharTest, "System::Core::String::StringView::FindLastChar", "[Core][String][SmokeFilter]")
{
	FStringView EmptyView;
	FStringView View = TEXT("aBce Fga");

	{
		int32 Index = INDEX_NONE;
		CHECK_FALSE(EmptyView.FindLastChar(TEXT('a'), Index));
		CHECK(Index == INDEX_NONE);
	}

	{
		int32 Index = INDEX_NONE;
		CHECK(View.FindLastChar(TEXT('a'), Index));
		CHECK(Index == 7);
	}

	{
		int32 Index = INDEX_NONE;
		CHECK(View.FindLastChar(TEXT('B'), Index));
		CHECK(Index == 1);
	}

	{
		int32 Index = INDEX_NONE;
		CHECK_FALSE(View.FindLastChar(TEXT('A'), Index));
		CHECK(Index == INDEX_NONE);
	}

	{
		int32 Index = INDEX_NONE;
		CHECK_FALSE(View.FindLastChar(TEXT('d'), Index));
		CHECK(Index == INDEX_NONE);
	}

	{
		int32 Index = INDEX_NONE;
		CHECK(View.FindLastChar(TEXT(' '), Index));
		CHECK(Index == 4);
	}
}

static void TestSlicing(const FString& Str)
{
	const FStringView View = Str;
	const int32       Len  = Str.Len();

	// Left tests
	{
		// Try all lengths of the string, including +/- 5 either side
		for (int32 Index = -5; Index != Len + 5; ++Index)
		{
			FString     Substring = Str .Left(Index);
			FStringView Subview   = View.Left(Index);

			CHECK(FString(Subview) == Substring);
		}
	}

	// LeftChop tests
	{
		// Try all lengths of the string, including +/- 5 either side
		for (int32 Index = -5; Index != Len + 5; ++Index)
		{
			FString     Substring = Str .LeftChop(Index);
			FStringView Subview   = View.LeftChop(Index);

			CHECK(FString(Subview) == Substring);
		}
	}

	// Right tests
	{
		// Try all lengths of the string, including +/- 5 either side
		for (int32 Index = -5; Index != Len + 5; ++Index)
		{
			FString     Substring = Str .Right(Index);
			FStringView Subview   = View.Right(Index);

			CHECK(FString(Subview) == Substring);
		}
	}

	// RightChop tests
	{
		// Try all lengths of the string, including +/- 5 either side
		for (int32 Index = -5; Index != Len + 5; ++Index)
		{
			FString     Substring = Str .RightChop(Index);
			FStringView Subview   = View.RightChop(Index);

			CHECK(FString(Subview) == Substring);
		}
	}

	// Mid tests
	{
		// Try all lengths of the string, including +/- 5 either side
		for (int32 Index = -5; Index != Len + 5; ++Index)
		{
			for (int32 Count = -5; Count != Len + 5; ++Count)
			{
				FString     Substring = Str .Mid(Index, Count);
				FStringView Subview   = View.Mid(Index, Count);

				CHECK(FString(Subview) == Substring);
			}
		}

		// Test near limits of int32
		for (int32 IndexOffset = 0; IndexOffset != Len + 5; ++IndexOffset)
		{
			for (int32 CountOffset = 0; CountOffset != Len + 5; ++CountOffset)
			{
				int32 Index = MIN_int32 + IndexOffset;
				int32 Count = MAX_int32 - CountOffset;

				FString     Substring = Str .Mid(Index, Count);
				FStringView Subview   = View.Mid(Index, Count);

				CHECK(FString(Subview) == Substring);
			}
		}
	}
}

TEST_CASE_NAMED(FStringViewSliceTest, "System::Core::String::StringView::Slice", "[Core][String][SmokeFilter]")
{
	// We assume that FString has already passed its tests, and we just want views to be consistent with it

	// Test an aribtrary string
	TestSlicing(TEXT("Test string"));

	// Test an empty string
	TestSlicing(FString());

	// Test an null-terminator-only empty string
	FString TerminatorOnly;
	TerminatorOnly.GetCharArray().Add(TEXT('\0'));
	TestSlicing(TerminatorOnly);
}

#endif // WITH_TESTS
