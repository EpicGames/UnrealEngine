// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Async/ManualResetEvent.h"
#include "Async/ParallelFor.h"
#include "Async/Fundamental/Scheduler.h"
#include "Containers/StripedMap.h"

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FStripedMapTests, "System::Core::Containers::TStripedMap", "[Core][Containers][TStripedMap]")
{
	TStripedMap<32, int32, int32> MapUnderTest;

	MapUnderTest.Add(5, 55);
	MapUnderTest.Add(1, 11);
	MapUnderTest.Add(3, 33);
	MapUnderTest.Emplace(4, 44);

	int32 VisitedItemsCount = 0;
	MapUnderTest.RemoveIf(
		[&VisitedItemsCount](const TPair<int32, int32>& Pair)
		{
			++VisitedItemsCount;
			return Pair.Key == 1;
		}
	);

	CHECK(MapUnderTest.Num() == 3);
	
	CHECK(VisitedItemsCount == 4);
	
	CHECK(MapUnderTest.FindRef(5) == 55);
	CHECK(MapUnderTest.FindRef(1) == 0); // Will return the default value
	CHECK(MapUnderTest.FindRef(3) == 33);
	CHECK(MapUnderTest.FindRef(4) == 44);
	
	// The read-only FindAndApply
	CHECK(MapUnderTest.FindAndApply(4,
		[](const int32& Value)
		{
			CHECK(Value == 44);
		}
	));

	// The read-write FindAndApply
	CHECK(MapUnderTest.FindAndApply(4,
		[](int32& Value)
		{
			CHECK(Value == 44);
			Value = 45;
		}
	));

	CHECK(MapUnderTest.RemoveIf(
		[](TPair<int32, int32>& Pair)
		{
			return Pair.Key == 4 && Pair.Value == 45;
		}
	));
	CHECK(!MapUnderTest.Contains(4));

	CHECK(MapUnderTest.Contains(5));

	// Verify thread safety
	{
		std::atomic<int32> ProduceCount = 0;
		std::atomic<int32> ApplyCount = 0;
		ParallelFor(10,
			[&](int32)
			{
				// One thread should succeed, all the other one verify the value
				MapUnderTest.FindOrProduceAndApply(1,
					[&ProduceCount]()
					{
						ProduceCount++;
						return 2000;
					},
					[&ApplyCount](const int32& Value)
					{
						CHECK(Value == 2000);
						ApplyCount++;
					}
				);

				CHECK(MapUnderTest.FindAndApply(5, [](int32& Value) { Value++; }));
			}
		);

		CHECK(ProduceCount.load() == 1);
		CHECK(ApplyCount.load() == 10);

		CHECK(MapUnderTest.Contains(5));

		// Make sure the value was safely incremented under a lock
		CHECK(MapUnderTest.FindRef(5) == 55 + 10);
	}

	// Validate that only a shared lock is taken if the function supports receiving a const ref.
	{
		UE::FManualResetEvent DoneEvent;
		std::atomic<int32> ConcurrentCount = 0;
		const int32 WorkerCount = LowLevelTasks::FScheduler::Get().GetNumWorkers();
		ParallelFor(WorkerCount,
			[&MapUnderTest, &ConcurrentCount, &WorkerCount, &DoneEvent](int32)
			{
				// All threads should be able to read at the same time, once the count is reached we can end the test.
				CHECK(MapUnderTest.FindOrTryProduceAndApply(5,
					[](int32& ProducedValue)
					{
						CHECK(false);
						return false; // do not add anything as we should not reach this point.
					},
					[&ConcurrentCount, &WorkerCount, &DoneEvent](const int32& FoundValue)
					{
						if (++ConcurrentCount == WorkerCount)
						{
							DoneEvent.Notify();
						}

						// Use a time limit to fail in case we end up deadlocked.
						CHECK(DoneEvent.WaitFor(UE::FMonotonicTimeSpan::FromMilliseconds(1000)));
					}
				));
			},
			EParallelForFlags::Unbalanced
		);
	}

	// Test for add failure
	CHECK(!MapUnderTest.FindOrTryProduceAndApply(10,
		[](int32& ProducedValue)
		{
			return false; // error out 
		},
		[&](const int32& FoundValue) 
		{ 
			// Since the produce failed, apply should not be called.
			CHECK(false); 
		}
	));

	CHECK(!MapUnderTest.Contains(10));

	// Test for add and write apply
	CHECK(MapUnderTest.FindOrTryProduceAndApplyForWrite(10,
		[](int32& ProducedValue)
		{
			ProducedValue = 10;
			return true;
		},
		[](int32& ApplyValue)
		{
			CHECK(ApplyValue == 10);
			ApplyValue = 11;
		}
	));

	CHECK(MapUnderTest.FindRef(10) == 11);

	// Just make sure those function exists
	MapUnderTest.Compact();
	MapUnderTest.Shrink();
	MapUnderTest.Reset();
	MapUnderTest.Empty();
	CHECK(MapUnderTest.Num() == 0);

	// Validate that pointer as key also work properly
	{
		TStripedMap<32, void*, int32> MapPointerTest;
		MapPointerTest.Add(&MapPointerTest, 55);
		CHECK(MapPointerTest.FindRef(&MapPointerTest) == 55);
		MapPointerTest.FindOrProduceAndApply(&MapPointerTest,
			[]()
			{
				CHECK(false);
				return 55;
			},
			[](const int32& Value)
			{
				CHECK(Value == 55);
			}
		);

		MapPointerTest.FindOrProduceAndApplyForWrite(&MapPointerTest,
			[]()
			{
				CHECK(false);
				return 55;
			},
			[](int32& Value)
			{
				CHECK(Value == 55);
				Value = 56;
			}
		);

		CHECK(MapPointerTest.Contains(&MapPointerTest));
		CHECK(MapPointerTest.FindRef(&MapPointerTest) == 56);
		CHECK(MapPointerTest.Num() == 1);
		CHECK(MapPointerTest.Remove(&MapPointerTest));
		CHECK(!MapPointerTest.Contains(&MapPointerTest));
		CHECK(MapPointerTest.Num() == 0);
	}
}

#endif //WITH_LOW_LEVEL_TESTS
