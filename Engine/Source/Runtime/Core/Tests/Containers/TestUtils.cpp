// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestUtils.h"

#include <numeric>

namespace Test
{
	constexpr int32 DefaultRandomSeed = 283281277;
}

void Test::ResetRandomSeed()
{
	FMath::SRandInit(DefaultRandomSeed);
}

TArray<int32> Test::MakeIndexArray(int32 Size)
{
	TArray<int32> Indices;
	Indices.SetNum(Size);
	std::iota(Indices.begin(), Indices.end(), 0);
	return Indices;
}
