// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"

namespace Test
{
void ResetRandomSeed();

/**
 * Returns an array with the parameter number of sequentially increased index values starting from zero.
 * This method is equivalent to using std::iota with a zero starting value in a sequential container.
 **/
TArray<int32> MakeIndexArray(int32 Size);

/**
 * Returns an array with a subset of the elements in the parameter input array. The subset size is expressed
 * by the parameter size which must be smaller than the number of elements in the input array.
 **/
template <typename T>
TArray<T> MakeRandomSubset(const TArray<T>& InArray, int32 Size)
{
	TArray<T> Result;
	if (InArray.Num() > 1 && Size < InArray.Num())
	{
		Result.Reset(Size);
		while (Result.Num() < Size)
		{
			const int32 i = FMath::RandRange(0, InArray.Num() - 1);
			if (Result.Find(InArray[i]) == INDEX_NONE)
			{
				Result.Add(InArray[i]);
			}
		}
	}
	return Result;
}

/**
 * Performs a number of randomized swaps in the parameter array equivalent to its number of elements.
 **/
template <typename T>
void Shuffle(TArray<T>& OutArray)
{
	if (OutArray.Num() > 1)
	{
		for (int32 i = 0; i < OutArray.Num(); i++)
		{
			const int32 j = FMath::RandRange(i, OutArray.Num() - 1);
			if (i != j)
			{
				OutArray.Swap(i, j);
			}
		}
	}
}
}  // namespace Test