// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_TESTS
#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "Tests/Benchmark.h"
#include "Containers/Ticker.h"
#include "Tasks/Task.h"
#include "Tests/Benchmark.h"


#include "Tests/TestHarnessAdapter.h"


template<uint32 NumDelegates, uint32 NumTicks>
void TSTickerPerfTest()
{
	FTSTicker Ticker;

	TArray<FTSTicker::FDelegateHandle> DelegateHandles;
	DelegateHandles.Reserve(NumDelegates);
	for (uint32 i = 0; i != NumDelegates; ++i)
	{
		DelegateHandles.Add(Ticker.AddTicker(UE_SOURCE_LOCATION, 0.0f, [](float DeltaTime) { return true; }));
	}

	for (uint32 i = 0; i != NumTicks; ++i)
	{
		Ticker.Tick(0.0f);
	}

	for (FTSTicker::FDelegateHandle& DelegateHandle : DelegateHandles)
	{
		FTSTicker::RemoveTicker(DelegateHandle);
	}
}

TEST_CASE_NAMED(FTSTickerTest,"System::Core::Containers::TSTicker", "[ApplicationContextMask][EngineFilter]")
{
	{	// a delegate returning false is executed once
		FTSTicker Ticker;
		bool bExecuted = false;
		FTSTicker::FDelegateHandle DelegateHandle = Ticker.AddTicker(UE_SOURCE_LOCATION, 0.0f,
			[&bExecuted](float DeltaTime)
			{
				check(!bExecuted);
		bExecuted = true;
		return false;
			}
		);
		Ticker.Tick(0.0f);
		Ticker.Tick(0.0f);
		check(bExecuted);
		Ticker.RemoveTicker(DelegateHandle);
	}

	{	// a delegate returning true is executed multiple times
		FTSTicker Ticker;
		uint32 NumExecuted = 0;
		FTSTicker::FDelegateHandle DelegateHandle = Ticker.AddTicker(UE_SOURCE_LOCATION, 0.0f,
			[&NumExecuted](float DeltaTime)
			{
				++NumExecuted;
		return true;
			}
		);
		Ticker.Tick(0.0f);
		Ticker.Tick(0.0f);
		check(NumExecuted == 2);
		Ticker.RemoveTicker(DelegateHandle);
	}

	{	// a delegate removal while it's being ticked doesn't return until its execution finished
		using namespace UE::Tasks;

		FTSTicker Ticker;

		FTaskEvent DelegateResumeEvent{ UE_SOURCE_LOCATION };
		FTSTicker::FDelegateHandle DelegateHandle = Ticker.AddTicker(UE_SOURCE_LOCATION, 0.0f,
			[&DelegateResumeEvent](float DeltaTime)
			{
				DelegateResumeEvent.Wait();
		return false;
			}
		);

		FTask RemoveTickerTask = Launch(UE_SOURCE_LOCATION,
			[&Ticker, &DelegateHandle]
			{
				FPlatformProcess::Sleep(0.1f); // let the ticking start and the delegate block on the event
			Ticker.RemoveTicker(DelegateHandle);
			}
			);

		FTask TickTask = Launch(UE_SOURCE_LOCATION,
			[&Ticker]
			{
				Ticker.Tick(0.0);
			}
			);

		FPlatformProcess::Sleep(0.1f); // let workers pick up the tasks and start execution

		verify(!TickTask.Wait(FTimespan::FromSeconds(0.1))); // tick is blocked because the delegate is blocked
		verify(!RemoveTickerTask.Wait(FTimespan::FromSeconds(0.1))); // removal is blocked because the delegate is blocked

		DelegateResumeEvent.Trigger();

		verify(TickTask.Wait(FTimespan::FromSeconds(0.1)));
		verify(RemoveTickerTask.Wait(FTimespan::FromSeconds(0.1)));
	}

	{	// a delegate removal from inside the delegate execution (used to be a deadlock)
		FTSTicker Ticker;
		FTSTicker::FDelegateHandle DelegateHandle;
		DelegateHandle = Ticker.AddTicker(nullptr, 0.0f, [&DelegateHandle](float) { FTSTicker::RemoveTicker(DelegateHandle); return true; });
		Ticker.Tick(0.0f);
	}

	{	// same delegate removed multiple times (used to be an assert)
		FTSTicker Ticker;
		FTSTicker::FDelegateHandle DelegateHandle = Ticker.AddTicker(UE_SOURCE_LOCATION, 0.0f, [](float) { return true; /* keep ticking*/ });
		FTSTicker::RemoveTicker(DelegateHandle);
		FTSTicker::RemoveTicker(DelegateHandle);
	}

	{	// check that delegate is called in the same tick that it was added, for backward compatibility with the previous implementation
		FTSTicker Ticker;
		bool bTicked = false;
		Ticker.AddTicker(UE_SOURCE_LOCATION, 0.0f,
			[&Ticker, &bTicked](float)
			{
				Ticker.AddTicker(UE_SOURCE_LOCATION, 0.0f,
				[&bTicked](float)
					{
						bTicked = true;
		return false;
					}
		);
		return false;
			}
		);
		Ticker.Tick(0.0f);
		check(bTicked);
	}

	// multithreaded stress test
	{
		using namespace UE::Tasks;

		FTSTicker Ticker;

		std::atomic<bool> bQuit{ false };
		FTask TickTask = Launch(UE_SOURCE_LOCATION,
			[&Ticker, &bQuit]
			{
				while (!bQuit)
				{
					Ticker.Tick(0.0f);
				}
			}
			);

		TArray<FTask> Tasks;
		Tasks.Reserve(500);
		for (int i = 0; i != 10; ++i)
		{
			Tasks.Add(Launch(UE_SOURCE_LOCATION,
				[&Ticker, &bQuit]
				{
					while (!bQuit)
					{
						FTSTicker::FDelegateHandle DelegateHandle = Ticker.AddTicker(UE_SOURCE_LOCATION, 0.0f,
							[](float DeltaTime)
							{
								return true;
							}
						);

						FTask RemoveTickerTask = Launch(UE_SOURCE_LOCATION,
							[DelegateHandle = MoveTemp(DelegateHandle)]
							{
								FTSTicker::RemoveTicker(DelegateHandle);
							}
							);
						RemoveTickerTask.Wait();
					}
				}
				));
		}

		FPlatformProcess::Sleep(0.3f); // let it run for a while
		bQuit = true;
		Tasks.Add(TickTask);
		verify(Wait(Tasks, FTimespan::FromSeconds(5)));
	}

	UE_BENCHMARK(5, TSTickerPerfTest<100, 100>);

}

TEST_CASE_NAMED(ExecuteOnGameThreadTest, "System::Core::Containers::TSTicker::ExecuteOnGameThreadTest", "[ApplicationContextMask][EngineFilter]")
{
	{
		uint32 NumExecuted = 0;
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [&NumExecuted] { ++NumExecuted; });
		check(NumExecuted == 0);

		FTSTicker::GetCoreTicker().Tick(0.0f); // the command above is executed by the core ticker
		check(NumExecuted == 1);

		FTSTicker::GetCoreTicker().Tick(0.0f); // tick again to check that the command was executed only once
		check(NumExecuted == 1);
	}

	{	// mutable functor
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [i = 0]() mutable { ++i; });
	}
}

#endif // WITH_TESTS