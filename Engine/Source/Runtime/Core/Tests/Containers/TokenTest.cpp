// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "TokenTest.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FTokenDefaultTest, "System::Core::Containers::TToken::Default", "[Core][Containers][TToken]")
{
	int32Token::Reset();
	{
		int32Token Token;
		CHECK(*Token == 0);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls());
	CHECK(int32Token::NumConstructorCalls() == 1);
	CHECK(int32Token::NumCopyConstructorCalls() == 0);
	CHECK(int32Token::NumMoveConstructorCalls() == 0);
	CHECK(int32Token::NumConstructionCalls() == 1);
	CHECK(int32Token::NumDestructionCalls() == 1);
}

TEST_CASE_NAMED(FTokenExplicitConstructorTest, "System::Core::Containers::TToken::Explicit constructor", "[Core][Containers][TToken]")
{
	int32Token::Reset();
	{
		int32Token Token = int32Token(1);
		CHECK(*Token == 1);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(1));
	CHECK(int32Token::NumCopyCalls() == 0);
	CHECK(int32Token::NumMoveCalls() == 0);
	CHECK(int32Token::NumConstructorCalls() == 1);
	CHECK(int32Token::NumCopyConstructorCalls() == 0);
	CHECK(int32Token::NumMoveConstructorCalls() == 0);
	CHECK(int32Token::NumCopyAssignmentCalls() == 0);
	CHECK(int32Token::NumMoveAssignmentCalls() == 0);
}

TEST_CASE_NAMED(FTokenCopyConstructorTest, "System::Core::Containers::TToken::Copy constructor", "[Core][Containers][TToken]")
{
	int32Token::Reset();
	{
		int32Token TempToken(2);
		int32Token Token = TempToken;
		CHECK(*Token == 2);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
	CHECK(int32Token::NumCopyCalls() == 1);
	CHECK(int32Token::NumMoveCalls() == 0);
	CHECK(int32Token::NumConstructorCalls() == 1);
	CHECK(int32Token::NumCopyConstructorCalls() == 1);
	CHECK(int32Token::NumMoveConstructorCalls() == 0);
	CHECK(int32Token::NumCopyAssignmentCalls() == 0);
	CHECK(int32Token::NumMoveAssignmentCalls() == 0);
}

TEST_CASE_NAMED(FTokenMoveConstructorTest, "System::Core::Containers::TToken::Move constructor", "[Core][Containers][TToken]")
{
	int32Token::Reset();
	{
		int32Token TempToken(3);
		int32Token Token = MoveTemp(TempToken);
		CHECK(*Token == 3);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
	CHECK(int32Token::NumCopyCalls() == 0);
	CHECK(int32Token::NumMoveCalls() == 1);
	CHECK(int32Token::NumConstructorCalls() == 1);
	CHECK(int32Token::NumCopyConstructorCalls() == 0);
	CHECK(int32Token::NumMoveConstructorCalls() == 1);
	CHECK(int32Token::NumCopyAssignmentCalls() == 0);
	CHECK(int32Token::NumMoveAssignmentCalls() == 0);
}

TEST_CASE_NAMED(FTokenCopyAssignmentTest, "System::Core::Containers::TToken::Copy assignment", "[Core][Containers][TToken]")
{
	int32Token::Reset();
	{
		int32Token TempToken(4);
		int32Token Token;
		Token = TempToken;
		CHECK(*Token == 4);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
	CHECK(int32Token::NumCopyCalls() == 1);
	CHECK(int32Token::NumMoveCalls() == 0);
	CHECK(int32Token::NumConstructorCalls() == 2);
	CHECK(int32Token::NumCopyConstructorCalls() == 0);
	CHECK(int32Token::NumMoveConstructorCalls() == 0);
	CHECK(int32Token::NumCopyAssignmentCalls() == 1);
	CHECK(int32Token::NumMoveAssignmentCalls() == 0);
}

TEST_CASE_NAMED(FTokenMoveAssignmentTest, "System::Core::Containers::TToken::Move assignment", "[Core][Containers][TToken]")
{
	int32Token::Reset();
	{
		int32Token TempToken(5);
		int32Token Token;
		Token = MoveTemp(TempToken);
		CHECK(*Token == 5);
	}
	CHECK(int32Token::EvenConstructionDestructionCalls(2));
	CHECK(int32Token::NumCopyCalls() == 0);
	CHECK(int32Token::NumMoveCalls() == 1);
	CHECK(int32Token::NumConstructorCalls() == 2);
	CHECK(int32Token::NumCopyConstructorCalls() == 0);
	CHECK(int32Token::NumMoveConstructorCalls() == 0);
	CHECK(int32Token::NumCopyAssignmentCalls() == 0);
	CHECK(int32Token::NumMoveAssignmentCalls() == 1);
}

#endif //WITH_TESTS