// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/UnrealTemplate.h"

template <typename InValueType>
class TTestToken
{
	static int32 ConstructorCalls;
	static int32 DestructorCalls;
	static int32 CopyConstructorCalls;
	static int32 MoveConstructorCalls;
	static int32 CopyAssignmentCalls;
	static int32 MoveAssignmentCalls;

public:
	using ValueType = InValueType;

	TTestToken(const ValueType& value)
	{
		ConstructorCalls++;
		Value = value;
	}

	TTestToken() noexcept : Value(ValueType{}) { ConstructorCalls++; }

	TTestToken(TTestToken&& Other) noexcept : Value(MoveTemp(Other.Value))
	{
		ensure(&Other != this);
		MoveConstructorCalls++;
	}

	TTestToken(const TTestToken& Other) : Value(Other.Value)
	{
		ensure(&Other != this);
		CopyConstructorCalls++;
	}

	~TTestToken()
	{
		DestructorCalls++;
	}

	const ValueType& operator*() const { return Value; }

	ValueType& operator*() { return Value; }

	const ValueType* operator->() const { return &Value; }

	ValueType* operator->() { return &Value; }

	TTestToken& operator=(const TTestToken& Other)
	{
		ensure(&Other != this);
		Value = Other.Value;
		CopyAssignmentCalls++;
		return *this;
	}

	TTestToken& operator=(TTestToken&& Other) noexcept
	{
		ensure(&Other != this);
		Value = MoveTemp(Other.Value);
		MoveAssignmentCalls++;
		return *this;
	}

	bool operator<(const TTestToken& Other) const { return Value < Other.Value; }

	static int32 NumConstructionCalls() { return ConstructorCalls + CopyConstructorCalls + MoveConstructorCalls; }

	static int32 NumConstructorCalls() { return ConstructorCalls; }

	static int32 NumCopyConstructorCalls() { return CopyConstructorCalls; }

	static int32 NumMoveConstructorCalls() { return MoveConstructorCalls; }

	static int32 ConstructionDestructionCallDifference() { return NumConstructionCalls() - DestructorCalls; }

	static int32 NumCopyCalls() { return CopyConstructorCalls + CopyAssignmentCalls; }

	static int32 NumCopyAssignmentCalls() { return CopyAssignmentCalls; }

	static int32 NumMoveCalls() { return MoveConstructorCalls + MoveAssignmentCalls; }

	static int32 NumMoveAssignmentCalls() { return MoveAssignmentCalls; }

	static int32 NumDestructionCalls() { return DestructorCalls; }

	static bool EvenConstructionDestructionCalls() { return ConstructionDestructionCallDifference() == 0; }

	static bool EvenConstructionDestructionCalls(int32 ExpectedNum)
	{
		return EvenConstructionDestructionCalls() && DestructorCalls == ExpectedNum;
	}

	static void Reset()
	{
		ConstructorCalls = 0;
		DestructorCalls = 0;
		CopyConstructorCalls = 0;
		MoveConstructorCalls = 0;
		CopyAssignmentCalls = 0;
		MoveAssignmentCalls = 0;
	}

	ValueType Value;
};

template <typename InValueType>
int32 TTestToken<InValueType>::ConstructorCalls = 0;

template <typename InValueType>
int32 TTestToken<InValueType>::DestructorCalls = 0;

template <typename InValueType>
int32 TTestToken<InValueType>::CopyConstructorCalls = 0;

template <typename InValueType>
int32 TTestToken<InValueType>::MoveConstructorCalls = 0;

template <typename InValueType>
int32 TTestToken<InValueType>::CopyAssignmentCalls = 0;

template <typename InValueType>
int32 TTestToken<InValueType>::MoveAssignmentCalls = 0;

using int32Token = TTestToken<int32>;

template <typename InValueType>
inline bool operator==(const TTestToken<InValueType>& left, const TTestToken<InValueType>& right)
{
	return left.Value == right.Value;
}

template <typename InValueType>
inline bool operator==(const TTestToken<InValueType>& left, const InValueType& right)
{
	return left.Value == right;
}

template <typename InValueType>
inline bool operator!=(const TTestToken<InValueType>& left, const TTestToken<InValueType>& right)
{
	return !operator==(left, right);
}

template <typename InValueType>
inline bool operator!=(const TTestToken<InValueType>& left, const InValueType& right)
{
	return !operator==(left, right);
}

template <typename InValueType>
inline FArchive& operator<<(FArchive& Ar, TTestToken<InValueType>& TestToken)
{
	return Ar << TestToken.Value;
}

template <typename InValueType>
inline void operator<<(FStructuredArchive::FSlot Slot, TTestToken<InValueType>& TestToken)
{
	Slot.EnterRecord() << SA_VALUE(TEXT("Value"), TestToken.Value);
}