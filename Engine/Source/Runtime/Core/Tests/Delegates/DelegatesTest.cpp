// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Misc/AutomationTest.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_TESTS

namespace UE::DelegatesTest
{
	struct FMoveOnlyParam
	{
		explicit FMoveOnlyParam(FString&& InIdentifier)
			: Identifier(MoveTemp(InIdentifier))
		{
		}

		FMoveOnlyParam(const FMoveOnlyParam&) = delete;
		FMoveOnlyParam& operator=(const FMoveOnlyParam&) = delete;
		~FMoveOnlyParam() = default;

		FMoveOnlyParam(FMoveOnlyParam&& Other)
			: Identifier(MoveTemp(Other.Identifier))
		{
			++Other.MovedFromCount;
		}

		FMoveOnlyParam& operator=(FMoveOnlyParam&& Other)
		{
			if (this != &Other)
			{
				Identifier = MoveTemp(Other.Identifier);
				++Other.MovedFromCount;
			}
			return *this;
		}

		FString Identifier;
		uint32  MovedFromCount = 0;
	};

	TEST_CASE_NAMED(FUnicastDelegateForwardingTest, "System::Core::Delegates::Unicast::Forwarding", "[ApplicationContextMask][EngineFilter]")
	{
		// Ensure that we forward arguments correctly
		{
			TDelegate<void(FMoveOnlyParam&&)> Delegate;

			FMoveOnlyParam Dest(TEXT("Dest"));
			Delegate.BindLambda([&Dest](FMoveOnlyParam&& Arg)
			{
				Dest = MoveTemp(Arg);
			});

			FMoveOnlyParam Src(TEXT("Src"));
			Delegate.Execute(MoveTemp(Src));

			check(Dest.Identifier == TEXT("Src"));
			check(Dest.MovedFromCount == 0);
			check(Src.Identifier.IsEmpty());
			check(Src.MovedFromCount == 1);
		}
	}
}

#endif // WITH_TESTS
