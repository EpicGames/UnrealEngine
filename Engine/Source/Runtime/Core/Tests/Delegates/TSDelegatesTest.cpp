// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "Delegates/Delegate.h"
#include "Tasks/Task.h"
#include "Tests/TestHarnessAdapter.h"
#include "Async/ParallelFor.h"

#include "HAL/PlatformTLS.h"

#if WITH_TESTS

namespace UE { namespace TSDelegatesTest 
{
	using namespace Tasks;

	TEST_CASE_NAMED(FUnicastDelegateBasicTest, "System::Core::Delegates::Unicast::Basic", "[ApplicationContextMask][EngineFilter]")
	{
		{	// single thread, basic functionality, check compilation/execution and that there's no false race detection
			bool bExecuted = false;
			TDelegate<void()> Delegate;
			Delegate.BindLambda([&bExecuted] { bExecuted = true; });
			Delegate.Execute();
			Delegate.Unbind();
			check(bExecuted);
		}

		{	// binding on "any thread"
			Launch(UE_SOURCE_LOCATION, [] 
			{ 
				TDelegate<void()>().BindLambda([]{});
			}).Wait();
		}

		{	// executing on "any thread"
			TDelegate<void()> Delegate;
			Delegate.BindLambda([]{});
			Launch(UE_SOURCE_LOCATION, [&Delegate] { Delegate.Execute(); }).Wait();
		}

		{	// unbinding on "any thread"
			TDelegate<void()> Delegate;
			Delegate.BindLambda([]{});
			Launch(UE_SOURCE_LOCATION, [&Delegate] { Delegate.Unbind(); }).Wait();
		}

		{	// compilation test for returning a non-copyable type
			struct FNoncopyableExample
			{
				UE_NONCOPYABLE(FNoncopyableExample);
				FNoncopyableExample() = default;
			};
			TDelegate<const FNoncopyableExample& ()> Delegate;
			FNoncopyableExample RetValue;
			Delegate.BindLambda([&RetValue] () -> const FNoncopyableExample& { return RetValue; });
			Delegate.Execute();
		}

		{	// move
			TDelegate<void()> Source;
			bool bExecuted = false;
			Source.BindLambda([&bExecuted] { bExecuted = true; });
			
			// move construction
			TDelegate<void()> Dest = MoveTemp(Source);
			check(!Source.IsBound());
			Dest.Execute();
			check(bExecuted);

			// move assignment
			Source.BindLambda([&bExecuted] { bExecuted = true; });
			Dest = MoveTemp(Source);
			check(!Source.IsBound());
			bExecuted = false;
			Dest.Execute();
			check(bExecuted);
		}

		{	// copy
			TDelegate<void()> Source;
			bool bExecuted = false;
			Source.BindLambda([&bExecuted] { bExecuted = true; });
			// copy construction
			TDelegate<void()> Dest = Source;
			Dest.Execute();
			check(bExecuted);
			// copy assignment
			Dest = Source;
			bExecuted = false;
			Dest.Execute();
			check(bExecuted);
		}

		///////////////////////////////////////////////////////////////////
		// unicast delegates can destroy themselves from inside their execution

		{
			TDelegate<void()>* Delegate = new TDelegate<void()>;
			Delegate->BindLambda([Delegate] { delete Delegate; });
			Delegate->Execute();
		}

		{	// nested destruction of different delegates
			TDelegate<void()>* Delegate = new TDelegate<void()>;
			Delegate->BindLambda(
				[Delegate] 
				{
					TDelegate<void()>* NestedDelegate = new TDelegate<void()>;
					NestedDelegate->BindLambda([NestedDelegate] { delete NestedDelegate; });
					NestedDelegate->Execute();
					
					delete Delegate;
				}
			);
			Delegate->Execute();
		}

		{	// delegate destruction from inside nested access
			bool bDestroy = false;
			TDelegate<void()>* Delegate = new TDelegate<void()>;
			Delegate->BindLambda(
				[Delegate, &bDestroy] 
				{ 
					if (!bDestroy)
					{
						bDestroy = true;
						Delegate->Execute();
					}
					else
					{
						delete Delegate;
					}
				}
			);
			Delegate->Execute();
			check(bDestroy);
		}

		{	// delegate destruction from inside nested access with intermediate access to another delegate
			TDelegate<void()>* Delegate = new TDelegate<void()>;
			Delegate->BindLambda(
				[Delegate]
				{
					TDelegate<void()> AnotherDelegate;
					AnotherDelegate.BindLambda(
						[Delegate] 
						{
							Delegate->Unbind();
							Delegate->BindLambda([Delegate] { delete Delegate; });
						}
					);
					AnotherDelegate.Execute();
				}
			);
			Delegate->Execute();
		}

		{
			struct FDelegateHolder
			{
				TDelegate<void()> Delegate;

				void SelfDestruct()
				{
					delete this;
				}
			};

			FDelegateHolder* DelegateHolder = new FDelegateHolder;
			DelegateHolder->Delegate.BindRaw(DelegateHolder, &FDelegateHolder::SelfDestruct);
			DelegateHolder->Delegate.Execute();
		}

		//////////////////////////////////////////////////////////////////////

		{	// Executing (which is a read op) a delegate concurrently, which can be safe if the binding is safe
			// the same case with a TS delegate would deadlock as we don't use a reader-writer mutex.
			// Broadcasting a multicast delegate is a write op and so would be caught by its race detector.
			// The executions on different threads are deliberately partially overlapped to check for an edge case.

			FTaskEvent FirstExecuting{ UE_SOURCE_LOCATION };
			FTaskEvent FirstFinish{ UE_SOURCE_LOCATION };
			FTaskEvent SecondExecuting{ UE_SOURCE_LOCATION };
			FTaskEvent SecondFinish{ UE_SOURCE_LOCATION };

			TDelegate<void(FTaskEvent&, FTaskEvent&)> Delegate;
			Delegate.BindLambda([](FTaskEvent& Executing, FTaskEvent& Finish)
			{
				Executing.Trigger();
				Finish.Wait();
			});

			FTask FirstTask = Launch(UE_SOURCE_LOCATION, [&Delegate, &FirstExecuting, &FirstFinish] { Delegate.Execute(FirstExecuting, FirstFinish); });
			FTask SecondTask = Launch(UE_SOURCE_LOCATION, [&Delegate, &SecondExecuting, &SecondFinish] { Delegate.Execute(SecondExecuting, SecondFinish); });
			Wait(TArray{ FirstExecuting, SecondExecuting });
			// at this moment there're two concurrent executions of the same delegate
			FirstFinish.Trigger();
			SecondFinish.Trigger();
			Wait(TArray{ FirstTask, SecondTask });
		}

		{	// executing a delegate stored in an array with binding that modifies the array causing realloc.
			// this is incompatible with race detection, so a "not checked" delegate is used
			using FNotCheckedDelegate = TDelegate<void(), FNotThreadSafeNotCheckedDelegateUserPolicy>;
			TArray<FNotCheckedDelegate> Ds;
			FNotCheckedDelegate D;
			D.BindLambda([&Ds] { Ds.Add(FNotCheckedDelegate{}); });
			Ds.Add(D);
			Ds.Shrink();
			Ds[0].Execute();
		}

		{ // destroying a container with delegates
			TArray<TDelegate<void()>> Array;
			Array.AddDefaulted();
			TDelegate<void()> Delegate;
			Delegate.BindLambda([] {});
			Array.Add(Delegate);
			Array.Add(MoveTemp(Delegate));
		}
	}

	TEST_CASE_NAMED(FUnicastTSDelegateBasicTest, "System::Core::Delegates::TS::Unicast::Basic", "[ApplicationContextMask][EngineFilter]")
	{
		{	// single thread
			TTSDelegate<void()> Delegate;
			bool bExecuted = false;
			Delegate.BindLambda([&bExecuted] { bExecuted = true; });
			Delegate.Execute();
			check(bExecuted);
		}

		{	// unbind from inside execution
			TTSDelegate<void()> Delegate;
			Delegate.BindLambda([&Delegate] { Delegate.Unbind(); });
			Delegate.Execute();
			check(!Delegate.IsBound());
		}

		{	// binging/unbinding concurrently with execution
			TTSDelegate<void()> Delegate;
			FTask Task = Launch(UE_SOURCE_LOCATION, [&Delegate] 
			{ 
				Delegate.BindLambda([] {}); 
				Delegate.Unbind();
			});
			Delegate.ExecuteIfBound();
			Task.Wait();
		}
	}

	TEST_CASE_NAMED(FUnicastTSDelegateStressTest, "System::Core::Delegates::TS::Unicast::Stress", "[ApplicationContextMask][EngineFilter]")
	{
		{	// one thread repeatedly binds/unbinds the delegate, another executes
			TTSDelegate<void()> Delegate;
			std::atomic<bool> bQuit{ false };
			std::atomic<int32> NumExecuted = 0;
			TArray<FTask> Tasks;

			Tasks.Add(Launch(UE_SOURCE_LOCATION, [&bQuit, &Delegate]
			{
				while (!bQuit)
				{
					Delegate.BindLambda([]{});
					FPlatformProcess::YieldCycles(100);
					Delegate.Unbind();
					FPlatformProcess::YieldCycles(100);
				}
			}));

			for (int i = 0; i != FPlatformMisc::NumberOfCores() / 2; ++i)
			{
				Tasks.Add(Launch(UE_SOURCE_LOCATION,
					[&bQuit, &Delegate, &NumExecuted]
					{
						while (!bQuit)
						{
							if (Delegate.ExecuteIfBound())
							{
								++NumExecuted;
							}
						}
					}
				));
			}

			FPlatformProcess::Sleep(1.0);
			bQuit = true;
			Wait(Tasks);
		}
	}

	TEST_CASE_NAMED(FMulticastDelegateBasicTest, "System::Core::Delegates::Multicast::Basic", "[ApplicationContextMask][EngineFilter]")
	{
		{ // add 2 bindings, broadcast, remove one binding, broadcast
			TMulticastDelegate<void()> Delegate;
			bool bExecuted1 = false;
			bool bExecuted2 = false;
			FDelegateHandle DelegateHandle1 = Delegate.AddLambda([&bExecuted1] { bExecuted1 = true; });
			FDelegateHandle DelegateHandle2 = Delegate.AddLambda([&bExecuted2] { bExecuted2 = true; });
			Delegate.Broadcast();
			check(bExecuted1 && bExecuted2);
			Delegate.Remove(DelegateHandle1);
			bExecuted1 = bExecuted2 = false;
			Delegate.Broadcast();
			check(!bExecuted1 && bExecuted2);
		}

		{	// check removing "raw method" binding by object ptr
			struct FDummy
			{
				void Method(int InValue)
				{
					Value = InValue;
				}

				int Value = 0;
			};

			TMulticastDelegate<void(int)> Delegate;
			FDummy Dummy1;
			Delegate.AddRaw(&Dummy1, &FDummy::Method);
			{
				FDummy Dummy2;
				Delegate.AddRaw(&Dummy2, &FDummy::Method);
				Delegate.Broadcast(1);
				check(Dummy1.Value == 1 && Dummy2.Value == 1);
				verify(Delegate.RemoveAll(&Dummy2) == 1);
			}
			Delegate.Broadcast(2);
			check(Dummy1.Value == 2);
			verify(Delegate.RemoveAll(&Dummy1));
		}

		{	// using unicast API instead of `Add*` shortcuts
			TMulticastDelegate<void()> Delegate;
			TDelegate<void()> UniDelegate;
			bool bExecuted = false;
			UniDelegate.BindLambda([&bExecuted] { bExecuted = true; });
			Delegate.Add(UniDelegate);
			Delegate.Broadcast();
			check(bExecuted);
		}

		{	// move
			TMulticastDelegate<void()> Source;
			int32 NumExecuted = 0;
			Source.AddLambda([&NumExecuted] { ++NumExecuted; });
			Source.AddLambda([&NumExecuted] { ++NumExecuted; });
			// move construction
			TMulticastDelegate<void()> Dest = MoveTemp(Source);
			check(!Source.IsBound());
			Dest.Broadcast();
			check(NumExecuted == 2);
			// move assignment
			Source.AddLambda([&NumExecuted] { ++NumExecuted; });
			Source.AddLambda([&NumExecuted] { ++NumExecuted; });
			Dest = MoveTemp(Source);
			check(!Source.IsBound());
			NumExecuted = 0;
			Dest.Broadcast();
			check(NumExecuted == 2);
		}

		{	// copy
			TMulticastDelegate<void()> Source;
			int32 NumExecuted = 0;
			Source.AddLambda([&NumExecuted] { ++NumExecuted; });
			Source.AddLambda([&NumExecuted] { ++NumExecuted; });
			// copy construction
			TMulticastDelegate<void()> Dest = Source;
			Dest.Broadcast();
			check(NumExecuted == 2);
			// copy assignment
			Dest = Source;
			NumExecuted = 0;
			Dest.Broadcast();
			check(NumExecuted == 2);
		}

	}

	TEST_CASE_NAMED(FTSMulticastDelegateStressTest, "System::Core::Delegates::MulticastStress", "[ApplicationContextMask][EngineFilter]")
	{
		{	// single thread
			TTSMulticastDelegate<void()> Delegate;
			int32 ExecRes = 0;
			FDelegateHandle DelegateHandle = Delegate.AddLambda([&ExecRes] { ++ExecRes; });
			Delegate.AddLambda([&ExecRes] { ExecRes += 2; });
			Delegate.Broadcast();
			check(ExecRes == 3);
			Delegate.Remove(DelegateHandle);
			ExecRes = 0;
			Delegate.Broadcast();
			check(ExecRes == 2);
		}

		{	// concurrent Add/Remove and Execute
			TTSMulticastDelegate<void()> Delegate;
			FTask AddBindingTask = Launch(UE_SOURCE_LOCATION, [&Delegate] 
			{ 
				Delegate.AddLambda([]{});
			});
			FTask BroadcastTask = Launch(UE_SOURCE_LOCATION, [&Delegate] { Delegate.Broadcast(); });
			FDelegateHandle DelegateHandle = Delegate.AddLambda([]{});
			Delegate.Remove(DelegateHandle);
			Wait(TArray{ AddBindingTask, BroadcastTask });
		}

		{ // same as above, but Add/Remove from different threads (not concurrently)
			TTSMulticastDelegate<void()> Delegate;
			TTask<FDelegateHandle> AddBindingTask = Launch(UE_SOURCE_LOCATION, [&Delegate] 
			{ 
				return Delegate.AddLambda([]{});
			});
			FTask RemoveBindingTask = Launch(UE_SOURCE_LOCATION, [&Delegate, &AddBindingTask] { Delegate.Remove(AddBindingTask.GetResult()); }, Prerequisites(AddBindingTask));
			Delegate.Broadcast();
			RemoveBindingTask.Wait();
		}

		{	// recursive locking: removing a delegate handle from inside broadcasting
			TTSMulticastDelegate<void()> Delegate;
			FDelegateHandle DelegateHandle;
			auto DelegateLambda = [&Delegate, &DelegateHandle] { Delegate.Remove(DelegateHandle); };
			DelegateHandle = Delegate.AddLambda(MoveTemp(DelegateLambda));
			check(Delegate.IsBound());
			Delegate.Broadcast();
			check(!Delegate.IsBound());
		}

#if UE_DETECT_DELEGATES_RACE_CONDITIONS
		{	// thread-unsafe delegate with disabled race detection
			// unbinding would assert if `FNotThreadSafeNotCheckedDelegateUserPolicy` wasn't specified.
			// It's still a bug as the bound lambda gets destroyed on the delegate removal, while it's in the middle of execution and can try accessing
			// its capture that is destroyed

			//TMulticastDelegate<void(), FNotThreadSafeNotCheckedDelegateUserPolicy> Delegate;

			//// model concurrent execution and unbinding
			//FTaskEvent BroadcastingStarted{ UE_SOURCE_LOCATION };
			//FTaskEvent ResumeBroadcasting{ UE_SOURCE_LOCATION };
			//// get blocked inside broadcasting
			//FDelegateHandle DelegateHandle = Delegate.AddLambda([&BroadcastingStarted, &ResumeBroadcasting]
			//{
			//	BroadcastingStarted.Trigger();
			//	ResumeBroadcasting.Wait(); 
			//});

			//FTask Task = Launch(UE_SOURCE_LOCATION, [&Delegate] { Delegate.Broadcast(); });
			//BroadcastingStarted.Wait();
			//Delegate.Remove(DelegateHandle); // happens concurrently with broadcasting on a thread-unsafe delegate, so it's a race
			//ResumeBroadcasting.Trigger();
			//Task.Wait();
		}
#endif
	}

	TEST_CASE_NAMED(FMulticastTSDelegateStressTest, "System::Core::Delegates::TS::Multicast::Stress", "[ApplicationContextMask][EngineFilter]")
	{
		{	// one thread repeatedly adds/removes delegates, another broadcasts
			TTSMulticastDelegate<void()> Delegate;
			std::atomic<bool> bQuit{ false };

			FTask Binding = Launch(UE_SOURCE_LOCATION,
				[&bQuit, &Delegate]
				{
					while (!bQuit)
					{
						FDelegateHandle Handle = Delegate.AddLambda([]{});
						Delegate.Remove(Handle);
					}
				}
			);

			FTask Executing = Launch(UE_SOURCE_LOCATION,
				[&bQuit, &Delegate]
				{
					while (!bQuit)
					{
						Delegate.Broadcast();
					}
				}
			);

			FPlatformProcess::Sleep(1.0);
			bQuit = true;
			Wait(TArray{ Binding, Executing });
		}
	}
}}

#endif //WITH_TESTS