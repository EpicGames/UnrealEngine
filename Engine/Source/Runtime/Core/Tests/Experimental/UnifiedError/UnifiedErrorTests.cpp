// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Experimental/UnifiedError/UnifiedError.h"
#include "Logging/StructuredLog.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Tests/TestHarnessAdapter.h"

enum class ETestErrorCodes
{
	Empty = 1,
	WithInt,
	WithUint,
	WithIntString,
	WithIntStringFloat,
	WithArray,
	WithStruct,
	WithErrorArray,
};

namespace UE::UnifiedErrorTest
{

// Test structure which serializes to compact binary for testing
struct FErrorVector3
{
	int32 X;
	int32 Y;
	int32 Z;
};

FCbWriter& operator<<(FCbWriter& Writer, const FErrorVector3& Value)
{
	Writer.BeginObject();
	Writer.AddInteger(UTF8TEXTVIEW("X"), Value.X);
	Writer.AddInteger(UTF8TEXTVIEW("Y"), Value.Y);
	Writer.AddInteger(UTF8TEXTVIEW("Z"), Value.Z);
	Writer.EndObject();
	return Writer;
}

}

UE_DECLARE_ERROR_MODULE(CORE_API, UE::UnifiedErrorTest)

UE_DECLARE_ERROR(CORE_API, UE::UnifiedErrorTest, ETestErrorCodes::Empty, 	 	  	  Empty, 				"Empty error")
UE_DECLARE_ERROR(CORE_API, UE::UnifiedErrorTest, ETestErrorCodes::WithInt, 		 	  WithInt, 				"Error with int {IntParam}", (int32, IntParam, 0))
UE_DECLARE_ERROR(CORE_API, UE::UnifiedErrorTest, ETestErrorCodes::WithUint, 	 	  WithUint, 			"Error with uint {UintParam}", (uint32, UintParam, 0))
UE_DECLARE_ERROR(CORE_API, UE::UnifiedErrorTest, ETestErrorCodes::WithIntString, 	  WithIntString, 		"Error with int {IntParam} and string {StringParam}", (int32, IntParam), (FString, StringParam, {}))
UE_DECLARE_ERROR(CORE_API, UE::UnifiedErrorTest, ETestErrorCodes::WithIntStringFloat, WithIntStringFloat,	"Error with int {IntParam} and string {StringParam} and float {FloatParam}", (int32, IntParam, 0), (FString, StringParam, {}), (float, FloatParam, 0.0f))
UE_DECLARE_ERROR(CORE_API, UE::UnifiedErrorTest, ETestErrorCodes::WithArray, 		  WithArray,			"Error with array {ArrayParam}", (TArray<int32>, ArrayParam, {}))
UE_DECLARE_ERROR(CORE_API, UE::UnifiedErrorTest, ETestErrorCodes::WithStruct, 		  WithStruct,			"Error with struct with custom formatting {Param/X},{Param/Y},{Param/Z}", (FErrorVector3, Param))
UE_DECLARE_ERROR(CORE_API, UE::UnifiedErrorTest, ETestErrorCodes::WithErrorArray,     WithErrorArray,		"Error with array of other errors {InnerErrors}", (TArray<UE::UnifiedError::FError>, InnerErrors, {}))
// Unsupported: Need support for handling string-able key types or coercing keys to strings
// UE_DECLARE_ERROR(CORE_API, UE::UnifiedErrorTest, ETestErrorCodes::WithMap, WithMap,			"Error with map {MapParam}", ((TMap<FName, int32>), MapParam, {}))

UE_DECLARE_ERROR_CONTEXT(CORE_API, UE::UnifiedErrorTest, StringContext, "Unified error string context {StringParam}", (FString, StringParam, {}));
UE_DECLARE_ERROR_CONTEXT(CORE_API, UE::UnifiedErrorTest, StringArrayContext, "Unified error TArray<FString> context {StringParam}", (TArray<FString>, StringArrayParam, {}));

UE_DEFINE_ERROR_MODULE(UE::UnifiedErrorTest)
UE_DEFINE_ERROR(UE::UnifiedErrorTest, Empty)
UE_DEFINE_ERROR(UE::UnifiedErrorTest, WithInt)
UE_DEFINE_ERROR(UE::UnifiedErrorTest, WithUint)
UE_DEFINE_ERROR(UE::UnifiedErrorTest, WithIntString)
UE_DEFINE_ERROR(UE::UnifiedErrorTest, WithIntStringFloat)
UE_DEFINE_ERROR(UE::UnifiedErrorTest, WithArray)
UE_DEFINE_ERROR(UE::UnifiedErrorTest, WithStruct)
UE_DEFINE_ERROR(UE::UnifiedErrorTest, WithErrorArray)
// UE_DEFINE_ERROR(UE::UnifiedErrorTest, WithMap)

/** Test constructing errors by generated factory method/callable */
TEST_CASE_NAMED(FUnifiedErrorTest_FactoryMethod, "System::Core::Experimental::UnifiedError::FactoryMethod", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	FError Empty = UE::UnifiedErrorTest::Empty();
	CHECK(Empty.IsValid());
	CHECK(Empty.GetModuleId() == UE::UnifiedErrorTest::GetStaticModuleId());
	CHECK(Empty.GetErrorCode() == (int32)ETestErrorCodes::Empty);
	CHECK(Empty.GetErrorCode() == UE::UnifiedErrorTest::Empty.ErrorCode);

	FError WithInt = UE::UnifiedErrorTest::WithInt(-7);
	CHECK(WithInt.IsValid());
	CHECK(WithInt.GetModuleId() == UE::UnifiedErrorTest::GetStaticModuleId());
	CHECK(WithInt.GetErrorCode() == (int32)ETestErrorCodes::WithInt);
	CHECK(WithInt.GetErrorCode() == UE::UnifiedErrorTest::WithInt.ErrorCode);
	const UE::UnifiedErrorTest::FWithInt* IntContext = WithInt.GetErrorContext<UE::UnifiedErrorTest::FWithInt>();
	REQUIRE(IntContext != nullptr);
	CHECK(IntContext->IntParam == -7);
	CHECK(IntContext == WithInt.GetErrorContext<UE::UnifiedErrorTest::WithInt>());
	
	FError WithUint = UE::UnifiedErrorTest::WithUint(18);
	CHECK(WithUint.IsValid());
	CHECK(WithUint.GetModuleId() == UE::UnifiedErrorTest::GetStaticModuleId());
	CHECK(WithUint.GetErrorCode() == (int32)ETestErrorCodes::WithUint);
	CHECK(WithUint.GetErrorCode() == UE::UnifiedErrorTest::WithUint.ErrorCode);
	const UE::UnifiedErrorTest::FWithUint* UintContext = WithUint.GetErrorContext<UE::UnifiedErrorTest::FWithUint>();
	REQUIRE(UintContext != nullptr);
	CHECK(UintContext->UintParam == 18);
	CHECK(UintContext == WithUint.GetErrorContext<UE::UnifiedErrorTest::WithUint>());

	FError WithIntString = UE::UnifiedErrorTest::WithIntString(180, FString(TEXT("Hello")));
	CHECK(WithIntString.IsValid());
	CHECK(WithIntString.GetModuleId() == UE::UnifiedErrorTest::GetStaticModuleId());
	CHECK(WithIntString.GetErrorCode() == (int32)ETestErrorCodes::WithIntString);
	CHECK(WithIntString.GetErrorCode() == UE::UnifiedErrorTest::WithIntString.ErrorCode);
	const UE::UnifiedErrorTest::FWithIntString* IntStringContext = WithIntString.GetErrorContext<UE::UnifiedErrorTest::FWithIntString>();
	REQUIRE(IntStringContext != nullptr);
	CHECK(IntStringContext->IntParam == 180);
	CHECK(IntStringContext->StringParam == TEXT("Hello"));
	CHECK(IntStringContext == WithIntString.GetErrorContext<UE::UnifiedErrorTest::WithIntString>());

	FError WithIntStringFloat = UE::UnifiedErrorTest::WithIntStringFloat(138, TEXT("abcdef"), 99.0f);
	CHECK(WithIntStringFloat.IsValid());
	CHECK(WithIntStringFloat.GetModuleId() == UE::UnifiedErrorTest::GetStaticModuleId());
	CHECK(WithIntStringFloat.GetErrorCode() == (int32)ETestErrorCodes::WithIntStringFloat);
	CHECK(WithIntStringFloat.GetErrorCode() == UE::UnifiedErrorTest::WithIntStringFloat.ErrorCode);
	const UE::UnifiedErrorTest::FWithIntStringFloat* IntStringFloatContext = WithIntStringFloat.GetErrorContext<UE::UnifiedErrorTest::FWithIntStringFloat>();
	REQUIRE(IntStringFloatContext != nullptr);
	CHECK(IntStringFloatContext->IntParam == 138);
	CHECK(IntStringFloatContext->StringParam == TEXT("abcdef"));
	CHECK(IntStringFloatContext->FloatParam == 99.0f);
	CHECK(IntStringFloatContext == WithIntStringFloat.GetErrorContext<UE::UnifiedErrorTest::WithIntStringFloat>());

	FError WithArray = UE::UnifiedErrorTest::WithArray({ 1, 2, 20, 45});
	CHECK(WithArray.IsValid());
	CHECK(WithArray.GetModuleId() == UE::UnifiedErrorTest::GetStaticModuleId());
	CHECK(WithArray.GetErrorCode() == (int32)ETestErrorCodes::WithArray);
	CHECK(WithArray.GetErrorCode() == UE::UnifiedErrorTest::WithArray.ErrorCode);
	const UE::UnifiedErrorTest::FWithArray* ArrayContext = WithArray.GetErrorContext<UE::UnifiedErrorTest::FWithArray>();
	REQUIRE(ArrayContext != nullptr);
	CHECK(ArrayContext->ArrayParam.Num() == 4);
	CHECK(ArrayContext->ArrayParam[0] == 1);
	CHECK(ArrayContext->ArrayParam[1] == 2);
	CHECK(ArrayContext->ArrayParam[2] == 20);
	CHECK(ArrayContext->ArrayParam[3] == 45);
	CHECK(ArrayContext == WithArray.GetErrorContext<UE::UnifiedErrorTest::WithArray>());

	FError WithStruct = UE::UnifiedErrorTest::WithStruct( { .X = 1, .Y = 2, .Z = 3 });
	CHECK(WithStruct.IsValid());
	CHECK(WithStruct.GetModuleId() == UE::UnifiedErrorTest::GetStaticModuleId());
	CHECK(WithStruct.GetErrorCode() == (int32)ETestErrorCodes::WithStruct);
	CHECK(WithStruct.GetErrorCode() == UE::UnifiedErrorTest::WithStruct.ErrorCode);
	const UE::UnifiedErrorTest::FWithStruct* StructContext = WithStruct.GetErrorContext<UE::UnifiedErrorTest::FWithStruct>();
	REQUIRE(StructContext != nullptr);
	CHECK(StructContext->Param.X == 1);
	CHECK(StructContext->Param.Y == 2);
	CHECK(StructContext->Param.Z == 3);

	FError WithErrorArray = UE::UnifiedErrorTest::WithErrorArray(TArray<FError>{ Empty, WithInt });
	CHECK(WithErrorArray.IsValid());
	CHECK(WithErrorArray.GetModuleId() == UE::UnifiedErrorTest::GetStaticModuleId());
	CHECK(WithErrorArray.GetErrorCode() == (int32)ETestErrorCodes::WithErrorArray);
	CHECK(WithErrorArray.GetErrorCode() == UE::UnifiedErrorTest::WithErrorArray.ErrorCode);
	const UE::UnifiedErrorTest::FWithErrorArray* ErrorArrayContext = WithErrorArray.GetErrorContext<UE::UnifiedErrorTest::FWithErrorArray>();
	REQUIRE(ErrorArrayContext != nullptr);
	CHECK(ErrorArrayContext->InnerErrors.Num() == 2);
	CHECK(ErrorArrayContext->InnerErrors[0] == Empty);
	CHECK(ErrorArrayContext->InnerErrors[1] == WithInt);
	CHECK(ErrorArrayContext == WithErrorArray.GetErrorContext<UE::UnifiedErrorTest::WithErrorArray>());
}

TEST_CASE_NAMED(FUnifiedErrorTest_Comparison, "System::Core::Experimental::UnifiedError::Comparison", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	FError Empty = UE::UnifiedErrorTest::Empty();
	CHECK(Empty == UE::UnifiedErrorTest::Empty);

	FError WithInt = UE::UnifiedErrorTest::WithInt(-7);
	CHECK(WithInt == UE::UnifiedErrorTest::WithInt);
	
	FError WithUint = UE::UnifiedErrorTest::WithUint(18);
	CHECK(WithUint == UE::UnifiedErrorTest::WithUint);

	FError WithIntString = UE::UnifiedErrorTest::WithIntString(18, FString(TEXT("Hello")));
	CHECK(WithIntString == UE::UnifiedErrorTest::WithIntString);

	FError WithIntStringFloat = UE::UnifiedErrorTest::WithIntStringFloat(18, TEXT("abcdef"), 99.0f);
	CHECK(WithIntStringFloat == UE::UnifiedErrorTest::WithIntStringFloat);

	FError WithStruct = UE::UnifiedErrorTest::WithStruct( { .X = 1, .Y = 2, .Z = 3 });
	CHECK(WithStruct == UE::UnifiedErrorTest::WithStruct);
}

TEST_CASE_NAMED(FUnifiedErrorTest_DesignatedInitializers, "System::Core::Experimental::UnifiedError::DesignatedInitializers", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	FError WithUint = UE::UnifiedErrorTest::WithUint({ .UintParam = 81 });
	CHECK(WithUint.GetModuleId() == UE::UnifiedErrorTest::GetStaticModuleId());
	CHECK(WithUint.GetErrorCode() == (int32)ETestErrorCodes::WithUint);
	const UE::UnifiedErrorTest::FWithUint* UintContext = WithUint.GetErrorContext<UE::UnifiedErrorTest::FWithUint>();
	REQUIRE(UintContext != nullptr);
	CHECK(UintContext->UintParam == 81);

	FError WithIntStringFloat = UE::UnifiedErrorTest::WithIntStringFloat( { .IntParam = 18, .StringParam = TEXT("abcdef"), .FloatParam = 99.0f });
	CHECK(WithIntStringFloat.GetModuleId() == UE::UnifiedErrorTest::GetStaticModuleId());
	CHECK(WithIntStringFloat.GetErrorCode() == (int32)ETestErrorCodes::WithIntStringFloat);
	const UE::UnifiedErrorTest::FWithIntStringFloat* IntStringFloatContext = WithIntStringFloat.GetErrorContext<UE::UnifiedErrorTest::FWithIntStringFloat>();
	REQUIRE(IntStringFloatContext != nullptr);
	CHECK(IntStringFloatContext->IntParam == 18);
	CHECK(IntStringFloatContext->StringParam == TEXT("abcdef"));
	CHECK(IntStringFloatContext->FloatParam == 99.0f);
}

TEST_CASE_NAMED(FUnifiedErrorTest_PushErrorContext, "System::Core::Experimental::UnifiedError::PushErrorContext", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	FError Empty = UE::UnifiedErrorTest::Empty();
	Empty.PushErrorContext(UE::UnifiedErrorTest::FStringContext{ .StringParam = TEXT("Hello") });
	const UE::UnifiedErrorTest::FStringContext* StringContext = Empty.GetErrorContext<UE::UnifiedErrorTest::FStringContext>();
	REQUIRE(StringContext != nullptr);
	CHECK(StringContext->StringParam == TEXT("Hello"));

	FError WithInt = UE::UnifiedErrorTest::WithInt();
	WithInt.PushErrorContext(UE::UnifiedErrorTest::FStringArrayContext{ .StringArrayParam{{ TEXT("One"), TEXT("Two"), TEXT("Three") }} });
	const UE::UnifiedErrorTest::FStringArrayContext* StringArrayContext = WithInt.GetErrorContext<UE::UnifiedErrorTest::FStringArrayContext>();
	REQUIRE(StringArrayContext != nullptr);
	CHECK(StringArrayContext->StringArrayParam.Num() == 3);
	CHECK(StringArrayContext->StringArrayParam[0] == TEXT("One"));
	CHECK(StringArrayContext->StringArrayParam[1] == TEXT("Two"));
	CHECK(StringArrayContext->StringArrayParam[2] == TEXT("Three"));
}

TEST_CASE_NAMED(FUnifiedErrorTest_CbObject, "System::Core::Experimental::UnifiedError::CbObject", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	auto CountFields = [](FCbObjectView View)
	{
		int32 NumFields = 0;
		for (FCbFieldView Field : View)
		{
			++NumFields;
		}
		return NumFields;
	};

	auto TestWithoutContext = [CountFields](const FError& Error, TFunctionRef<void(FCbObjectView)> CheckDetails)
	{
		// Error writes a sequence of fields. It's more convenient for the test to index fields in an object by name, so wrap it in an object.
		FCbWriter Writer;
		Writer.BeginObject();
		Error.SerializeToCompactBinary(Writer, UE::UnifiedError::EDetailFilter::All);
		Writer.EndObject();
		FCbObject Obj = Writer.Save().AsObject();
		CHECK(!!Obj);
		CHECK(CountFields(Obj) == 9);
		CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedError::FError"));
		CHECK(Obj["$locformat"].IsString());
		CHECK(Obj["$lockey"].IsString());
		CHECK(Obj["$locns"].IsString());
		CHECK(Obj["ErrorCodeString"].AsString() == Error.GetErrorCodeString());
		CHECK(Obj["ModuleIdString"].AsString() == Error.GetModuleIdString());
		CHECK(Obj["ErrorCode"].AsInt32() == Error.GetErrorCode());
		CHECK(Obj["ModuleId"].AsInt32() == Error.GetModuleId());
		FCbArrayView Details = Obj["Details"].AsArrayView();
		CHECK(!!Details);
		CHECK(Details.Num() == 1); // These errors have no added context so they should only have the originating error
		FCbObjectView MandatoryDetails = Details.CreateViewIterator().AsObjectView();
		CHECK(!!MandatoryDetails);
		CheckDetails(MandatoryDetails);
	};

	TestWithoutContext(UE::UnifiedErrorTest::Empty(), [CountFields](FCbObjectView Obj) 
		{
			CHECK(CountFields(Obj) == 4);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FEmpty"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Empty error"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("Empty"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
		});
	TestWithoutContext(UE::UnifiedErrorTest::WithInt(-7), [CountFields](FCbObjectView Obj) 
		{
			CHECK(CountFields(Obj) == 5);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithInt"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam}"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("WithInt"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Obj["IntParam"].AsInt32() == -7);
		});
	TestWithoutContext(UE::UnifiedErrorTest::WithUint(18), [CountFields](FCbObjectView Obj) 
		{
			CHECK(CountFields(Obj) == 5);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithUint"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Error with uint {UintParam}"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("WithUint"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Obj["UintParam"].AsUInt32() == 18);
		});
	TestWithoutContext(UE::UnifiedErrorTest::WithIntString(180, FString(TEXT("Hello"))), [CountFields](FCbObjectView Obj) 
		{
			CHECK(CountFields(Obj) == 6);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithIntString"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam} and string {StringParam}"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("WithIntString"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Obj["IntParam"].AsUInt32() == 180);
			CHECK(Obj["StringParam"].AsString() == UTF8TEXTVIEW("Hello"));
		});
	TestWithoutContext(UE::UnifiedErrorTest::WithIntStringFloat(138, TEXT("abcdef"), 99.0f), [CountFields](FCbObjectView Obj) 
		{
			CHECK(CountFields(Obj) == 7);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithIntStringFloat"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam} and string {StringParam} and float {FloatParam}"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("WithIntStringFloat"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Obj["IntParam"].AsUInt32() == 138);
			CHECK(Obj["StringParam"].AsString() == UTF8TEXTVIEW("abcdef"));
			CHECK(Obj["FloatParam"].AsFloat() == 99.0f);
		});
	TestWithoutContext(UE::UnifiedErrorTest::WithArray({1, 2, 3}), [CountFields](FCbObjectView Obj) 
		{
			CHECK(CountFields(Obj) == 5);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithArray"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Error with array {ArrayParam}"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("WithArray"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			FCbArrayView Array = Obj["ArrayParam"].AsArrayView();
			CHECK(!!Array);
			CHECK(Array.Num() == 3);
			FCbFieldViewIterator ArrayIt = Array.CreateViewIterator();
			CHECK(ArrayIt->AsInt32() == 1);
			++ArrayIt;
			CHECK(ArrayIt->AsInt32() == 2);
			++ArrayIt;
			CHECK(ArrayIt->AsInt32() == 3);
		});

	// Add context to an underlying error and check the serialization
	auto TestWithContext = [CountFields](FError Error, UE::UnifiedError::CErrorContext auto Context, TFunctionRef<void(FCbObjectView, FCbObjectView)> CheckDetails)
	{
		Error.PushErrorContext(Context);
		// Error writes a sequence of fields. It's more convenient for the test to index fields in an object by name, so wrap it in an object.
		FCbWriter Writer;
		Writer.BeginObject();
		Error.SerializeToCompactBinary(Writer, UE::UnifiedError::EDetailFilter::All);
		Writer.EndObject();
		FCbObject Obj = Writer.Save().AsObject();
		CHECK(!!Obj);
		CHECK(CountFields(Obj) == 9);
		CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedError::FError"));
		CHECK(Obj["$locformat"].IsString());
		CHECK(Obj["$lockey"].IsString());
		CHECK(Obj["$locns"].IsString());
		CHECK(Obj["ErrorCodeString"].AsString() == Error.GetErrorCodeString());
		CHECK(Obj["ModuleIdString"].AsString() == Error.GetModuleIdString());
		CHECK(Obj["ErrorCode"].AsInt32() == Error.GetErrorCode());
		CHECK(Obj["ModuleId"].AsInt32() == Error.GetModuleId());
		FCbArrayView Details = Obj["Details"].AsArrayView();
		CHECK(!!Details);
		CHECK(Details.Num() == 2); // These errors should have the mandatory details and a single additional context
		FCbFieldViewIterator DetailsIt = Details.CreateViewIterator();
		FCbObjectView MandatoryDetails = DetailsIt.AsObjectView();
		CHECK(!!MandatoryDetails);
		++DetailsIt;
		FCbObjectView ContextDetails = DetailsIt.AsObjectView();
		CHECK(!!ContextDetails);
		CheckDetails(MandatoryDetails, ContextDetails);
	};

	TestWithContext(UE::UnifiedErrorTest::Empty(), UE::UnifiedErrorTest::FStringContext{ TEXT("abcdef") },
		[CountFields](FCbObjectView Mandatory, FCbObjectView Context) 
		{
			CHECK(CountFields(Mandatory) == 4);
			CHECK(Mandatory["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FEmpty"));
			CHECK(Mandatory["$locformat"].AsString() == UTF8TEXTVIEW("Empty error"));
			CHECK(Mandatory["$lockey"].AsString() == UTF8TEXTVIEW("Empty"));
			CHECK(Mandatory["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(CountFields(Context) == 5);
			CHECK(Context["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FStringContext"));
			CHECK(Context["$locformat"].AsString() == UTF8TEXTVIEW("Unified error string context {StringParam}"));
			CHECK(Context["$lockey"].AsString() == UTF8TEXTVIEW("StringContext"));
			CHECK(Context["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Context["StringParam"].AsString() == UTF8TEXTVIEW("abcdef"));
		});
	TestWithContext(UE::UnifiedErrorTest::WithInt(-7), UE::UnifiedErrorTest::FStringContext(TEXT("defabc")),
		[CountFields](FCbObjectView Mandatory, FCbObjectView Context) 
		{
			CHECK(CountFields(Mandatory) == 5);
			CHECK(Mandatory["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithInt"));
			CHECK(Mandatory["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam}"));
			CHECK(Mandatory["$lockey"].AsString() == UTF8TEXTVIEW("WithInt"));
			CHECK(Mandatory["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Mandatory["IntParam"].AsInt32() == -7);
			CHECK(CountFields(Context) == 5);
			CHECK(Context["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FStringContext"));
			CHECK(Context["$locformat"].AsString() == UTF8TEXTVIEW("Unified error string context {StringParam}"));
			CHECK(Context["$lockey"].AsString() == UTF8TEXTVIEW("StringContext"));
			CHECK(Context["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Context["StringParam"].AsString() == UTF8TEXTVIEW("defabc"));
		});
	TestWithContext(UE::UnifiedErrorTest::WithUint(18),
		UE::UnifiedErrorTest::FStringContext(TEXT("abc123")),
		[CountFields](FCbObjectView Mandatory, FCbObjectView Context) 
		{
			CHECK(CountFields(Mandatory) == 5);
			CHECK(Mandatory["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithUint"));
			CHECK(Mandatory["$locformat"].AsString() == UTF8TEXTVIEW("Error with uint {UintParam}"));
			CHECK(Mandatory["$lockey"].AsString() == UTF8TEXTVIEW("WithUint"));
			CHECK(Mandatory["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Mandatory["UintParam"].AsUInt32() == 18);
			CHECK(CountFields(Context) == 5);
			CHECK(Context["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FStringContext"));
			CHECK(Context["$locformat"].AsString() == UTF8TEXTVIEW("Unified error string context {StringParam}"));
			CHECK(Context["$lockey"].AsString() == UTF8TEXTVIEW("StringContext"));
			CHECK(Context["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Context["StringParam"].AsString() == UTF8TEXTVIEW("abc123"));
		});
	TestWithContext(UE::UnifiedErrorTest::WithIntString(180, FString(TEXT("Hello"))),
		UE::UnifiedErrorTest::FStringContext(TEXT("123abc")),
		[CountFields](FCbObjectView Mandatory, FCbObjectView Context) 
		{
			CHECK(CountFields(Mandatory) == 6);
			CHECK(Mandatory["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithIntString"));
			CHECK(Mandatory["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam} and string {StringParam}"));
			CHECK(Mandatory["$lockey"].AsString() == UTF8TEXTVIEW("WithIntString"));
			CHECK(Mandatory["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Mandatory["IntParam"].AsUInt32() == 180);
			CHECK(Mandatory["StringParam"].AsString() == UTF8TEXTVIEW("Hello"));
			CHECK(CountFields(Context) == 5);
			CHECK(Context["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FStringContext"));
			CHECK(Context["$locformat"].AsString() == UTF8TEXTVIEW("Unified error string context {StringParam}"));
			CHECK(Context["$lockey"].AsString() == UTF8TEXTVIEW("StringContext"));
			CHECK(Context["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Context["StringParam"].AsString() == UTF8TEXTVIEW("123abc"));
		});
	TestWithContext(UE::UnifiedErrorTest::WithIntStringFloat(138, TEXT("abcdef"), 99.0f),
		UE::UnifiedErrorTest::FStringContext(TEXT("abc123456")),
		[CountFields](FCbObjectView Mandatory, FCbObjectView Context) 
		{
			CHECK(CountFields(Mandatory) == 7);
			CHECK(Mandatory["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithIntStringFloat"));
			CHECK(Mandatory["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam} and string {StringParam} and float {FloatParam}"));
			CHECK(Mandatory["$lockey"].AsString() == UTF8TEXTVIEW("WithIntStringFloat"));
			CHECK(Mandatory["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Mandatory["IntParam"].AsUInt32() == 138);
			CHECK(Mandatory["StringParam"].AsString() == UTF8TEXTVIEW("abcdef"));
			CHECK(Mandatory["FloatParam"].AsFloat() == 99.0f);
			CHECK(CountFields(Context) == 5);
			CHECK(Context["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FStringContext"));
			CHECK(Context["$locformat"].AsString() == UTF8TEXTVIEW("Unified error string context {StringParam}"));
			CHECK(Context["$lockey"].AsString() == UTF8TEXTVIEW("StringContext"));
			CHECK(Context["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Context["StringParam"].AsString() == UTF8TEXTVIEW("abc123456"));
		});
	TestWithContext(UE::UnifiedErrorTest::WithArray({1, 2, 3}), 
		UE::UnifiedErrorTest::FStringContext(TEXT("123abc456")),
		[CountFields](FCbObjectView Mandatory, FCbObjectView Context) 
		{
			CHECK(Mandatory["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithArray"));
			CHECK(Mandatory["$locformat"].AsString() == UTF8TEXTVIEW("Error with array {ArrayParam}"));
			CHECK(Mandatory["$lockey"].AsString() == UTF8TEXTVIEW("WithArray"));
			CHECK(Mandatory["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			FCbArrayView Array = Mandatory["ArrayParam"].AsArrayView();
			CHECK(!!Array);
			CHECK(Array.Num() == 3);
			FCbFieldViewIterator ArrayIt = Array.CreateViewIterator();
			CHECK(ArrayIt->AsInt32() == 1);
			++ArrayIt;
			CHECK(ArrayIt->AsInt32() == 2);
			++ArrayIt;
			CHECK(ArrayIt->AsInt32() == 3);
			CHECK(CountFields(Context) == 5);
			CHECK(Context["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FStringContext"));
			CHECK(Context["$locformat"].AsString() == UTF8TEXTVIEW("Unified error string context {StringParam}"));
			CHECK(Context["$lockey"].AsString() == UTF8TEXTVIEW("StringContext"));
			CHECK(Context["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Context["StringParam"].AsString() == UTF8TEXTVIEW("123abc456"));
		});
}

TEST_CASE_NAMED(FUnifiedErrorTest_CreateErrorMessage, "System::Core::Experimental::UnifiedError::CreateErrorMessage", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	FError Empty = UE::UnifiedErrorTest::Empty();
	FText EmptyText = Empty.CreateErrorMessage(false);
	CHECK(EmptyText.ToString() == TEXTVIEW("UE::UnifiedErrorTest::Empty: Empty error"));

	FError WithInt = UE::UnifiedErrorTest::WithInt(-7);
	FText WithIntText = WithInt.CreateErrorMessage(false);
	CHECK(WithIntText.ToString() == TEXTVIEW("UE::UnifiedErrorTest::WithInt: Error with int -7"));
	
	FError WithUint = UE::UnifiedErrorTest::WithUint(18);
	FText WithUintText = WithUint.CreateErrorMessage(false);
	CHECK(WithUintText.ToString() == TEXTVIEW("UE::UnifiedErrorTest::WithUint: Error with uint 18"));

	FError WithIntString = UE::UnifiedErrorTest::WithIntString(180, FString(TEXT("Hello")));
	FText WithIntStringText = WithIntString.CreateErrorMessage(false);
	CHECK(WithIntStringText.ToString() == TEXTVIEW("UE::UnifiedErrorTest::WithIntString: Error with int 180 and string Hello"));

	FError WithIntStringFloat = UE::UnifiedErrorTest::WithIntStringFloat(138, TEXT("abcdef"), 99.0f);
	FText WithIntStringFloatText = WithIntStringFloat.CreateErrorMessage(false);
	CHECK(WithIntStringFloatText.ToString() == TEXTVIEW("UE::UnifiedErrorTest::WithIntStringFloat: Error with int 138 and string abcdef and float 99"));

	FError WithArray = UE::UnifiedErrorTest::WithArray({1, 2, 3});
	FText WithArrayText = WithArray.CreateErrorMessage(false);
	CHECK(WithArrayText.ToString() == TEXTVIEW("UE::UnifiedErrorTest::WithArray: Error with array [1, 2, 3]"));

	FError WithStruct = UE::UnifiedErrorTest::WithStruct({.X = 1, .Y = 2, .Z = 3});
	FText WithStructText = WithStruct.CreateErrorMessage(false);
	CHECK(WithStructText.ToString() == TEXTVIEW("UE::UnifiedErrorTest::WithStruct: Error with struct with custom formatting 1,2,3"));
}

TEST_CASE_NAMED(FUnifiedErrorTest_CreateErrorMessageWithContext, "System::Core::Experimental::UnifiedError::CreateErrorMessageWithContext", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	// Without context — should not include pushed context details
	{
		FError Err = UE::UnifiedErrorTest::WithInt(-7);
		Err.PushErrorContext(UE::UnifiedErrorTest::FStringContext{ .StringParam = TEXT("extra_info") });
		FText WithoutContext = Err.CreateErrorMessage(false);
		FText WithContext = Err.CreateErrorMessage(true);
		CHECK(WithoutContext.ToString() == TEXTVIEW("UE::UnifiedErrorTest::WithInt: Error with int -7"));
		CHECK(WithContext.ToString() == TEXTVIEW("UE::UnifiedErrorTest::WithInt: [Error with int -7, Unified error string context extra_info]"));
	}

	// Empty error with context appended
	{
		FError Err = UE::UnifiedErrorTest::Empty();
		Err.PushErrorContext(UE::UnifiedErrorTest::FStringContext{ .StringParam = TEXT("context_value") });
		FText WithoutContext = Err.CreateErrorMessage(false);
		FText WithContext = Err.CreateErrorMessage(true);
		CHECK(WithoutContext.ToString() == TEXTVIEW("UE::UnifiedErrorTest::Empty: Empty error"));
		CHECK(WithContext.ToString() == TEXTVIEW("UE::UnifiedErrorTest::Empty: [Empty error, Unified error string context context_value]"));
	}
}

// This test validates the analytics JSON by verifying the underlying CompactBinary structure
// under the IncludeInAnalytics filter, then checking that CompactBinaryToCompactJson produces
// a string matching SerializeToJsonForAnalytics(). This is open-box testing — it mirrors the
// internal implementation path — but it lets us structurally validate every field without
// pulling in a JSON parser from another module.
TEST_CASE_NAMED(FUnifiedErrorTest_SerializeToJsonForAnalytics, "System::Core::Experimental::UnifiedError::SerializeToJsonForAnalytics", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	auto CountFields = [](FCbObjectView View)
	{
		int32 NumFields = 0;
		for (FCbFieldView Field : View)
		{
			++NumFields;
		}
		return NumFields;
	};

	auto TestWithoutContext = [CountFields](const FError& Error, TFunctionRef<void(FCbObjectView)> CheckDetails)
	{
		// Build the CB the same way SerializeToJsonForAnalytics does internally, then verify both
		// the CB structure and the final JSON string.
		FCbWriter Writer;
		Writer.BeginObject();
		Error.SerializeToCompactBinary(Writer, UE::UnifiedError::EDetailFilter::IncludeInAnalytics);
		Writer.EndObject();
		FCbObject Obj = Writer.Save().AsObject();
		CHECK(!!Obj);
		CHECK(CountFields(Obj) == 9);
		CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedError::FError"));
		CHECK(Obj["$locformat"].IsString());
		CHECK(Obj["$lockey"].IsString());
		CHECK(Obj["$locns"].IsString());
		CHECK(Obj["ErrorCodeString"].AsString() == Error.GetErrorCodeString());
		CHECK(Obj["ModuleIdString"].AsString() == Error.GetModuleIdString());
		CHECK(Obj["ErrorCode"].AsInt32() == Error.GetErrorCode());
		CHECK(Obj["ModuleId"].AsInt32() == Error.GetModuleId());
		FCbArrayView Details = Obj["Details"].AsArrayView();
		CHECK(!!Details);
		CHECK(Details.Num() == 1); // Only mandatory details — no pushed context
		FCbObjectView MandatoryDetails = Details.CreateViewIterator().AsObjectView();
		CHECK(!!MandatoryDetails);
		CheckDetails(MandatoryDetails);

		// Verify that SerializeToJsonForAnalytics produces JSON matching CB-to-JSON conversion
		TStringBuilder<1024> ExpectedJson;
		CompactBinaryToCompactJson(Obj, ExpectedJson);
		FString ActualJson = Error.SerializeToJsonForAnalytics();
		CHECK(FStringView(ExpectedJson) == ActualJson);
	};

	TestWithoutContext(UE::UnifiedErrorTest::Empty(), [CountFields](FCbObjectView Obj)
		{
			CHECK(CountFields(Obj) == 4);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FEmpty"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Empty error"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("Empty"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
		});
	TestWithoutContext(UE::UnifiedErrorTest::WithInt(-7), [CountFields](FCbObjectView Obj)
		{
			CHECK(CountFields(Obj) == 5);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithInt"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam}"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("WithInt"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Obj["IntParam"].AsInt32() == -7);
		});
	TestWithoutContext(UE::UnifiedErrorTest::WithUint(18), [CountFields](FCbObjectView Obj)
		{
			CHECK(CountFields(Obj) == 5);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithUint"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Error with uint {UintParam}"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("WithUint"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Obj["UintParam"].AsUInt32() == 18);
		});
	TestWithoutContext(UE::UnifiedErrorTest::WithIntString(180, FString(TEXT("Hello"))), [CountFields](FCbObjectView Obj)
		{
			CHECK(CountFields(Obj) == 6);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithIntString"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam} and string {StringParam}"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("WithIntString"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Obj["IntParam"].AsUInt32() == 180);
			CHECK(Obj["StringParam"].AsString() == UTF8TEXTVIEW("Hello"));
		});
	TestWithoutContext(UE::UnifiedErrorTest::WithIntStringFloat(138, TEXT("abcdef"), 99.0f), [CountFields](FCbObjectView Obj)
		{
			CHECK(CountFields(Obj) == 7);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithIntStringFloat"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam} and string {StringParam} and float {FloatParam}"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("WithIntStringFloat"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Obj["IntParam"].AsUInt32() == 138);
			CHECK(Obj["StringParam"].AsString() == UTF8TEXTVIEW("abcdef"));
			CHECK(Obj["FloatParam"].AsFloat() == 99.0f);
		});
	TestWithoutContext(UE::UnifiedErrorTest::WithArray({1, 2, 3}), [CountFields](FCbObjectView Obj)
		{
			CHECK(CountFields(Obj) == 5);
			CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithArray"));
			CHECK(Obj["$locformat"].AsString() == UTF8TEXTVIEW("Error with array {ArrayParam}"));
			CHECK(Obj["$lockey"].AsString() == UTF8TEXTVIEW("WithArray"));
			CHECK(Obj["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			FCbArrayView Array = Obj["ArrayParam"].AsArrayView();
			CHECK(!!Array);
			CHECK(Array.Num() == 3);
			FCbFieldViewIterator ArrayIt = Array.CreateViewIterator();
			CHECK(ArrayIt->AsInt32() == 1);
			++ArrayIt;
			CHECK(ArrayIt->AsInt32() == 2);
			++ArrayIt;
			CHECK(ArrayIt->AsInt32() == 3);
		});

	// Add context to an underlying error and check the serialization.
	// Context must be pushed with IncludeInAnalytics to appear under this filter.
	auto TestWithContext = [CountFields](FError Error, UE::UnifiedError::CErrorContext auto Context, TFunctionRef<void(FCbObjectView, FCbObjectView)> CheckDetails)
	{
		Error.PushErrorContext(Context, UE::UnifiedError::EDetailFilter::IncludeInAnalytics);
		FCbWriter Writer;
		Writer.BeginObject();
		Error.SerializeToCompactBinary(Writer, UE::UnifiedError::EDetailFilter::IncludeInAnalytics);
		Writer.EndObject();
		FCbObject Obj = Writer.Save().AsObject();
		CHECK(!!Obj);
		CHECK(CountFields(Obj) == 9);
		CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedError::FError"));
		CHECK(Obj["$locformat"].IsString());
		CHECK(Obj["$lockey"].IsString());
		CHECK(Obj["$locns"].IsString());
		CHECK(Obj["ErrorCodeString"].AsString() == Error.GetErrorCodeString());
		CHECK(Obj["ModuleIdString"].AsString() == Error.GetModuleIdString());
		CHECK(Obj["ErrorCode"].AsInt32() == Error.GetErrorCode());
		CHECK(Obj["ModuleId"].AsInt32() == Error.GetModuleId());
		FCbArrayView Details = Obj["Details"].AsArrayView();
		CHECK(!!Details);
		CHECK(Details.Num() == 2); // Mandatory details + pushed context
		FCbFieldViewIterator DetailsIt = Details.CreateViewIterator();
		FCbObjectView MandatoryDetails = DetailsIt.AsObjectView();
		CHECK(!!MandatoryDetails);
		++DetailsIt;
		FCbObjectView ContextDetails = DetailsIt.AsObjectView();
		CHECK(!!ContextDetails);
		CheckDetails(MandatoryDetails, ContextDetails);

		TStringBuilder<1024> ExpectedJson;
		CompactBinaryToCompactJson(Obj, ExpectedJson);
		FString ActualJson = Error.SerializeToJsonForAnalytics();
		CHECK(FStringView(ExpectedJson) == ActualJson);
	};

	TestWithContext(UE::UnifiedErrorTest::Empty(), UE::UnifiedErrorTest::FStringContext{ TEXT("abcdef") },
		[CountFields](FCbObjectView Mandatory, FCbObjectView Context)
		{
			CHECK(CountFields(Mandatory) == 4);
			CHECK(Mandatory["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FEmpty"));
			CHECK(Mandatory["$locformat"].AsString() == UTF8TEXTVIEW("Empty error"));
			CHECK(Mandatory["$lockey"].AsString() == UTF8TEXTVIEW("Empty"));
			CHECK(Mandatory["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(CountFields(Context) == 5);
			CHECK(Context["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FStringContext"));
			CHECK(Context["$locformat"].AsString() == UTF8TEXTVIEW("Unified error string context {StringParam}"));
			CHECK(Context["$lockey"].AsString() == UTF8TEXTVIEW("StringContext"));
			CHECK(Context["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Context["StringParam"].AsString() == UTF8TEXTVIEW("abcdef"));
		});
	TestWithContext(UE::UnifiedErrorTest::WithInt(-7), UE::UnifiedErrorTest::FStringContext(TEXT("defabc")),
		[CountFields](FCbObjectView Mandatory, FCbObjectView Context)
		{
			CHECK(CountFields(Mandatory) == 5);
			CHECK(Mandatory["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithInt"));
			CHECK(Mandatory["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam}"));
			CHECK(Mandatory["$lockey"].AsString() == UTF8TEXTVIEW("WithInt"));
			CHECK(Mandatory["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Mandatory["IntParam"].AsInt32() == -7);
			CHECK(CountFields(Context) == 5);
			CHECK(Context["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FStringContext"));
			CHECK(Context["$locformat"].AsString() == UTF8TEXTVIEW("Unified error string context {StringParam}"));
			CHECK(Context["$lockey"].AsString() == UTF8TEXTVIEW("StringContext"));
			CHECK(Context["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Context["StringParam"].AsString() == UTF8TEXTVIEW("defabc"));
		});
	TestWithContext(UE::UnifiedErrorTest::WithIntStringFloat(138, TEXT("abcdef"), 99.0f),
		UE::UnifiedErrorTest::FStringContext(TEXT("abc123456")),
		[CountFields](FCbObjectView Mandatory, FCbObjectView Context)
		{
			CHECK(CountFields(Mandatory) == 7);
			CHECK(Mandatory["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithIntStringFloat"));
			CHECK(Mandatory["$locformat"].AsString() == UTF8TEXTVIEW("Error with int {IntParam} and string {StringParam} and float {FloatParam}"));
			CHECK(Mandatory["$lockey"].AsString() == UTF8TEXTVIEW("WithIntStringFloat"));
			CHECK(Mandatory["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Mandatory["IntParam"].AsUInt32() == 138);
			CHECK(Mandatory["StringParam"].AsString() == UTF8TEXTVIEW("abcdef"));
			CHECK(Mandatory["FloatParam"].AsFloat() == 99.0f);
			CHECK(CountFields(Context) == 5);
			CHECK(Context["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FStringContext"));
			CHECK(Context["$locformat"].AsString() == UTF8TEXTVIEW("Unified error string context {StringParam}"));
			CHECK(Context["$lockey"].AsString() == UTF8TEXTVIEW("StringContext"));
			CHECK(Context["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Context["StringParam"].AsString() == UTF8TEXTVIEW("abc123456"));
		});
	TestWithContext(UE::UnifiedErrorTest::WithArray({1, 2, 3}),
		UE::UnifiedErrorTest::FStringContext(TEXT("123abc456")),
		[CountFields](FCbObjectView Mandatory, FCbObjectView Context)
		{
			CHECK(Mandatory["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FWithArray"));
			CHECK(Mandatory["$locformat"].AsString() == UTF8TEXTVIEW("Error with array {ArrayParam}"));
			CHECK(Mandatory["$lockey"].AsString() == UTF8TEXTVIEW("WithArray"));
			CHECK(Mandatory["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			FCbArrayView Array = Mandatory["ArrayParam"].AsArrayView();
			CHECK(!!Array);
			CHECK(Array.Num() == 3);
			FCbFieldViewIterator ArrayIt = Array.CreateViewIterator();
			CHECK(ArrayIt->AsInt32() == 1);
			++ArrayIt;
			CHECK(ArrayIt->AsInt32() == 2);
			++ArrayIt;
			CHECK(ArrayIt->AsInt32() == 3);
			CHECK(CountFields(Context) == 5);
			CHECK(Context["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::FStringContext"));
			CHECK(Context["$locformat"].AsString() == UTF8TEXTVIEW("Unified error string context {StringParam}"));
			CHECK(Context["$lockey"].AsString() == UTF8TEXTVIEW("StringContext"));
			CHECK(Context["$locns"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
			CHECK(Context["StringParam"].AsString() == UTF8TEXTVIEW("123abc456"));
		});
}

TEST_CASE_NAMED(FUnifiedErrorTest_MoveAndCopy, "System::Core::Experimental::UnifiedError::MoveAndCopy", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	// Move constructor leaves source invalid
	{
		FError Original = UE::UnifiedErrorTest::WithInt(-7);
		CHECK(Original.IsValid());
		FError Moved(MoveTemp(Original));
		CHECK(Moved.IsValid());
		CHECK(Moved.GetErrorCode() == (int32)ETestErrorCodes::WithInt);
		const UE::UnifiedErrorTest::FWithInt* Context = Moved.GetErrorContext<UE::UnifiedErrorTest::FWithInt>();
		REQUIRE(Context != nullptr);
		CHECK(Context->IntParam == -7);
		CHECK(!Original.IsValid());
	}

	// Copy constructor shares context
	{
		FError Original = UE::UnifiedErrorTest::WithInt(42);
		FError Copy(Original);
		CHECK(Original.IsValid());
		CHECK(Copy.IsValid());
		CHECK(Copy.GetErrorCode() == Original.GetErrorCode());
		CHECK(Copy.GetModuleId() == Original.GetModuleId());
		const UE::UnifiedErrorTest::FWithInt* OrigContext = Original.GetErrorContext<UE::UnifiedErrorTest::FWithInt>();
		const UE::UnifiedErrorTest::FWithInt* CopyContext = Copy.GetErrorContext<UE::UnifiedErrorTest::FWithInt>();
		REQUIRE(OrigContext != nullptr);
		REQUIRE(CopyContext != nullptr);
		CHECK(OrigContext->IntParam == 42);
		CHECK(CopyContext->IntParam == 42);
	}

	// Copy accumulates context independently
	{
		FError Original = UE::UnifiedErrorTest::Empty();
		FError Copy(Original);
		Copy.PushErrorContext(UE::UnifiedErrorTest::FStringContext{ .StringParam = TEXT("CopyOnly") });
		// Copy has the pushed context, original does not
		const UE::UnifiedErrorTest::FStringContext* CopyCtx = Copy.GetErrorContext<UE::UnifiedErrorTest::FStringContext>();
		const UE::UnifiedErrorTest::FStringContext* OrigCtx = Original.GetErrorContext<UE::UnifiedErrorTest::FStringContext>();
		REQUIRE(CopyCtx != nullptr);
		CHECK(CopyCtx->StringParam == TEXT("CopyOnly"));
		CHECK(OrigCtx == nullptr);
	}
}


TEST_CASE_NAMED(FUnifiedErrorTest_StringAccessors, "System::Core::Experimental::UnifiedError::StringAccessors", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	FError Empty = UE::UnifiedErrorTest::Empty();
	CHECK(Empty.GetErrorCodeString() == UTF8TEXTVIEW("Empty"));
	CHECK(Empty.GetModuleIdString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
	CHECK(Empty.GetModuleIdAndErrorCodeString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::Empty"));

	FError WithInt = UE::UnifiedErrorTest::WithInt(-7);
	CHECK(WithInt.GetErrorCodeString() == UTF8TEXTVIEW("WithInt"));
	CHECK(WithInt.GetModuleIdString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
	CHECK(WithInt.GetModuleIdAndErrorCodeString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::WithInt"));

	FError WithIntString = UE::UnifiedErrorTest::WithIntString(0, FString(TEXT("x")));
	CHECK(WithIntString.GetErrorCodeString() == UTF8TEXTVIEW("WithIntString"));
	CHECK(WithIntString.GetModuleIdString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
	CHECK(WithIntString.GetModuleIdAndErrorCodeString() == UTF8TEXTVIEW("UE::UnifiedErrorTest::WithIntString"));
}

TEST_CASE_NAMED(FUnifiedErrorTest_InvalidateAndIsValid, "System::Core::Experimental::UnifiedError::InvalidateAndIsValid", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	// Invalidate makes IsValid return false
	{
		FError Err = UE::UnifiedErrorTest::WithInt(10);
		CHECK(Err.IsValid());
		Err.Invalidate();
		CHECK(!Err.IsValid());
	}

	// Moved-from error is invalid
	{
		FError Err = UE::UnifiedErrorTest::Empty();
		CHECK(Err.IsValid());
		FError Moved(MoveTemp(Err));
		CHECK(!Err.IsValid());
		CHECK(Moved.IsValid());
	}

	// CreateErrorMessage on an invalidated error does not crash
	{
		FError Err = UE::UnifiedErrorTest::Empty();
		Err.Invalidate();
		FText Msg = Err.CreateErrorMessage(false);
		CHECK(!Msg.IsEmpty());
	}
}

TEST_CASE_NAMED(FUnifiedErrorTest_EqualsAndErrorComparison, "System::Core::Experimental::UnifiedError::EqualsAndErrorComparison", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	// Equals with matching module/error code
	{
		FError Err = UE::UnifiedErrorTest::WithInt(-7);
		CHECK(Err.Equals(UE::UnifiedErrorTest::GetStaticModuleId(), UE::UnifiedErrorTest::WithInt.ErrorCode));
	}

	// Equals with non-matching error code
	{
		FError Err = UE::UnifiedErrorTest::WithInt(-7);
		CHECK(!Err.Equals(UE::UnifiedErrorTest::GetStaticModuleId(), UE::UnifiedErrorTest::Empty.ErrorCode));
	}

	// Equals with non-matching module id
	{
		FError Err = UE::UnifiedErrorTest::WithInt(-7);
		CHECK(!Err.Equals(0, UE::UnifiedErrorTest::WithInt.ErrorCode));
	}

	// FError == FError: same error type
	{
		FError A = UE::UnifiedErrorTest::WithInt(-7);
		FError B = UE::UnifiedErrorTest::WithInt(99);
		CHECK(A == B); // Same module and error code, different payload
	}

	// FError == FError: different error type
	{
		FError A = UE::UnifiedErrorTest::WithInt(-7);
		FError B = UE::UnifiedErrorTest::WithUint(18);
		CHECK(!(A == B));
	}

	// FError != FError
	{
		FError A = UE::UnifiedErrorTest::WithInt(-7);
		FError B = UE::UnifiedErrorTest::Empty();
		CHECK(A != B);
	}

	// FError == FError: same type, one with pushed context — context does not affect comparison
	{
		FError A = UE::UnifiedErrorTest::Empty();
		FError B = UE::UnifiedErrorTest::Empty();
		B.PushErrorContext(UE::UnifiedErrorTest::FStringContext{ .StringParam = TEXT("extra") });
		CHECK(A == B);
	}
}

TEST_CASE_NAMED(FUnifiedErrorTest_SerializeForLog, "System::Core::Experimental::UnifiedError::SerializeForLog", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	auto CountFields = [](FCbObjectView View)
	{
		int32 NumFields = 0;
		for (FCbFieldView Field : View)
		{
			++NumFields;
		}
		return NumFields;
	};

	// SerializeForLog wraps the error in an object suitable for structured logging
	{
		FError Empty = UE::UnifiedErrorTest::Empty();
		FCbWriter Writer;
		SerializeForLog(Writer, Empty);
		FCbObject Obj = Writer.Save().AsObject();
		CHECK(!!Obj);
		// Should contain the fields from SerializeToCompactBinary wrapped in an object
		CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedError::FError"));
		CHECK(Obj["ErrorCodeString"].AsString() == UTF8TEXTVIEW("Empty"));
		CHECK(Obj["ModuleIdString"].AsString() == UTF8TEXTVIEW("UE::UnifiedErrorTest"));
		CHECK(Obj["Details"].IsArray());
	}

	// Error with fields
	{
		FError WithInt = UE::UnifiedErrorTest::WithInt(-7);
		FCbWriter Writer;
		SerializeForLog(Writer, WithInt);
		FCbObject Obj = Writer.Save().AsObject();
		CHECK(!!Obj);
		CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedError::FError"));
		CHECK(Obj["ErrorCodeString"].AsString() == UTF8TEXTVIEW("WithInt"));
		FCbArrayView Details = Obj["Details"].AsArrayView();
		CHECK(!!Details);
		CHECK(Details.Num() == 1);
		FCbObjectView DetailObj = Details.CreateViewIterator().AsObjectView();
		CHECK(DetailObj["IntParam"].AsInt32() == -7);
	}

	// Error with pushed context
	{
		FError WithContext = UE::UnifiedErrorTest::WithInt(42);
		WithContext.PushErrorContext(UE::UnifiedErrorTest::FStringContext{ .StringParam = TEXT("log_test") });
		FCbWriter Writer;
		SerializeForLog(Writer, WithContext);
		FCbObject Obj = Writer.Save().AsObject();
		CHECK(!!Obj);
		CHECK(Obj["$type"].AsString() == UTF8TEXTVIEW("UE::UnifiedError::FError"));
	}
}

TEST_CASE_NAMED(FUnifiedErrorTest_StructuredLogSmoke, "System::Core::Experimental::UnifiedError::StructuredLogSmoke", "[Core][SmokeFilter]")
{
	using UE::UnifiedError::FError;

	// Verify that FError can be passed to UE_LOGFMT without crashing.
	// This exercises the full SerializeForLog -> structured log pipeline.
	FError Empty = UE::UnifiedErrorTest::Empty();
	UE_LOGFMT(LogTemp, Log, "Error test: {Error}", Empty);

	FError WithInt = UE::UnifiedErrorTest::WithInt(-7);
	UE_LOGFMT(LogTemp, Log, "Error with param: {Error}", WithInt);

	FError WithContext = UE::UnifiedErrorTest::Empty();
	WithContext.PushErrorContext(UE::UnifiedErrorTest::FStringContext{ .StringParam = TEXT("smoke_test") });
	UE_LOGFMT(LogTemp, Log, "Error with context: {Error}", WithContext);

	// Invalidated error — SerializeForLog must not crash when details are null
	FError Invalidated = UE::UnifiedErrorTest::WithInt(99);
	Invalidated.Invalidate();
	UE_LOGFMT(LogTemp, Log, "FError that has been invalidated: {Error}", Invalidated);

	// Moved-from error — same concern
	FError MovedFrom = UE::UnifiedErrorTest::Empty();
	FError MovedTo(MoveTemp(MovedFrom));
	UE_LOGFMT(LogTemp, Log, "FError that has been moved from: {Error}", MovedFrom);
}

#endif // WITH_TESTS