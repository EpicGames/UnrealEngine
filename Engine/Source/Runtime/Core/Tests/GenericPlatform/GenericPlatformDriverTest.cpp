// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS 

#include "GenericPlatform/GenericPlatformDriver.h"
#include "Tests/TestHarnessAdapter.h"

static FString MakeVersionString(const TArray<uint32> Values)
{
	if (Values.IsEmpty())
	{
		return TEXT("");
	}
	FString VersionString;
	for (int32 Idx = 0; Idx < Values.Num(); ++Idx)
	{
		VersionString += FString::FromInt(Values[Idx]);
		if (Idx != Values.Num() - 1)
		{
			VersionString += TEXT(".");
		}
	}
	return VersionString;
}

TEST_CASE_NAMED(DriverVersionTest, "System::Core::GenericPlatform::Driver", "[ApplicationContextMask][SmokeFilter]")
{
	SECTION("Driver version parsing")
	{
		{
			FDriverVersion Version;
			CHECK(Version.GetVersionValues().Num() == 0);
		}
		{
			// All of these should parse the same.
			TArray<FDriverVersion> Versions;
			Versions.Emplace(MakeVersionString({ 1,2,3,4 }));
			Versions.Emplace(TEXT("1.2.3.4"));
			Versions.Emplace(TEXT("  1.2.3.4"));
			Versions.Emplace(TEXT("1.2.3.4  "));
			Versions.Emplace(TEXT("1 . 2. 3 .4"));

			for (const FDriverVersion& Version : Versions)
			{
				CHECK(Version.GetVersionValues().Num() == 4);
				CHECK(Version.GetVersionValue(0) == 1);
				CHECK(Version.GetVersionValue(1) == 2);
				CHECK(Version.GetVersionValue(2) == 3);
				CHECK(Version.GetVersionValue(3) == 4);
			}
		}
	}

	SECTION("Driver version comparisons")
	{
		{
			FDriverVersion A(TEXT("1.2.3.4"));
			FDriverVersion B(TEXT("1.2.3.4"));
			CHECK(A == B);
			CHECK(A <= B);
			CHECK(A >= B);
			CHECK(!(A != B));
			CHECK(!(A < B));
			CHECK(!(A > B));
		}
		{
			TArray<uint32> BaseVersion = { 1,2,3,4 };

			for (int32 Idx = 0; Idx < 4; ++Idx)
			{
				// Make the first version larger by incrementing each dot version.
				TArray<uint32> LargerVersion = BaseVersion;
				LargerVersion[Idx] += 1;

				FDriverVersion A(MakeVersionString(LargerVersion));
				FDriverVersion B(MakeVersionString(BaseVersion));
				CHECK(A != B);
				CHECK(A > B);
				CHECK(A >= B);
				CHECK(!(A == B));
				CHECK(!(A < B));
				CHECK(!(A <= B));
			}
		}
		{
			TArray<uint32> BaseVersion = { 1,2,3,4 };

			for (int32 Idx = 0; Idx < 4; ++Idx)
			{
				// Make the first version smaller by decrementing each dot version.
				TArray<uint32> SmallerVersion = BaseVersion;
				SmallerVersion[Idx] -= 1;

				FDriverVersion A(MakeVersionString(SmallerVersion));
				FDriverVersion B(MakeVersionString(BaseVersion));
				CHECK(A != B);
				CHECK(A < B);
				CHECK(A <= B);
				CHECK(!(A == B));
				CHECK(!(A > B));
				CHECK(!(A >= B));
			}
		}
	}

	SECTION("Unified driver version parsing")
	{
		{
			FGPUDriverInfo Info;
			Info.SetNVIDIA();
			Info.InternalDriverVersion = TEXT("31.0.15.5123");
			CHECK(Info.GetUnifiedDriverVersion() == TEXT("551.23"));
		}
		{
			FGPUDriverInfo Info;
			Info.SetIntel();
			Info.InternalDriverVersion = TEXT("9.18.10.3310");
			CHECK(Info.GetUnifiedDriverVersion() == TEXT("10.3310"));
		}
		{
			FGPUDriverInfo Info;
			Info.SetAMD();
			Info.InternalDriverVersion = TEXT("15.200.1062.1004");
			CHECK(Info.GetUnifiedDriverVersion() == TEXT("15.200.1062.1004"));
		}
		{
			FGPUDriverInfo Info;
			Info.SetQualcomm();
			Info.InternalDriverVersion = TEXT("31.0.117.0");
			CHECK(Info.GetUnifiedDriverVersion() == TEXT("31.0.117.0"));
		}
		{
			FGPUDriverInfo Info;
			Info.SetIntel();
			Info.InternalDriverVersion = TEXT("9.18.10.3310");
			CHECK(Info.IsSameDriverVersionGeneration(FDriverVersion(TEXT("15.3501"))));
			CHECK(!Info.IsSameDriverVersionGeneration(FDriverVersion(TEXT("101.5124"))));

			Info.InternalDriverVersion = TEXT("30.0.101.1404");
			CHECK(!Info.IsSameDriverVersionGeneration(FDriverVersion(TEXT("15.3501"))));
			CHECK(Info.IsSameDriverVersionGeneration(FDriverVersion(TEXT("100.5124"))));
		}
	}

	SECTION("Driver denylist and suggested entry parsing")
	{
		{
			FDriverDenyListEntry Entry(TEXT("(DriverVersion=\"301.12\", RHI=\"D3D12\", AdapterNameRegex=\".*4080.*\", Reason=\"Test version\")"));
			CHECK(Entry.IsValid());
			CHECK(Entry.DenylistReason == TEXT("Test version"));
			CHECK(*Entry.RHINameConstraint == TEXT("D3D12"));
			CHECK(Entry.AdapterNameRegexConstraint);
			CHECK(Entry.DriverVersion->ComparisonOp == EComparisonOp::ECO_Equal);
			CHECK(Entry.DriverVersion->Version == FDriverVersion(MakeVersionString({ 301, 12 })));
			CHECK(!Entry.AppliesToLatestDrivers());
		}
		{
			FDriverDenyListEntry Entry(TEXT("(DriverVersion=\"<=311.16\", Reason=\"Test with no constraints\")"));
			CHECK(Entry.IsValid());
			CHECK(Entry.DenylistReason == TEXT("Test with no constraints"));
			CHECK(Entry.DriverVersion->ComparisonOp == EComparisonOp::ECO_LessOrEqual);
			CHECK(Entry.DriverVersion->Version == FDriverVersion(MakeVersionString({ 311, 16 })));
			CHECK(!Entry.AppliesToLatestDrivers());
		}
		{
			FDriverDenyListEntry Entry(TEXT("(DriverDate=\">=12-31-2021\", RHI=\"D3D11\", Reason=\"Test date\")"));
			CHECK(Entry.IsValid());
			CHECK(Entry.DenylistReason == TEXT("Test date"));
			CHECK(*Entry.RHINameConstraint == TEXT("D3D11"));
			CHECK(Entry.DriverDate->ComparisonOp == EComparisonOp::ECO_GreaterOrEqual);
			CHECK(Entry.DriverDate->Date == FDateTime(2021, 12, 31));
			CHECK(Entry.AppliesToLatestDrivers());
		}
		{
			FDriverDenyListEntry Entry(TEXT("(DriverVersion=\"<=311.16\", AdapterGeneration=\"<=5\", Reason=\"Test generation\")"));
			CHECK(Entry.IsValid());
			CHECK(Entry.DenylistReason == TEXT("Test generation"));
			CHECK(Entry.AdapterGenerationConstraint->ComparisonOp == EComparisonOp::ECO_LessOrEqual);
			CHECK(Entry.AdapterGenerationConstraint->Value == 5);
			CHECK(Entry.DriverVersion->ComparisonOp == EComparisonOp::ECO_LessOrEqual);
			CHECK(Entry.DriverVersion->Version == FDriverVersion(MakeVersionString({ 311, 16 })));
			CHECK(!Entry.AppliesToLatestDrivers());
		}
		{
			FSuggestedDriverEntry Entry(TEXT("(DriverVersion=\"512.15\", RHI=\"D3D11\", AdapterNameRegex=\".*4050.*\")"));
			CHECK(Entry.IsValid());
			CHECK(*Entry.RHINameConstraint == TEXT("D3D11"));
			CHECK(Entry.AdapterNameRegexConstraint);
			CHECK(Entry.SuggestedDriverVersion == TEXT("512.15"));
		}
		{
			FSuggestedDriverEntry Entry(TEXT("(DriverVersion=\"101.56\")"));
			CHECK(Entry.IsValid());
			CHECK(Entry.SuggestedDriverVersion == TEXT("101.56"));
		}
		{
			FSuggestedDriverEntry Entry(TEXT("111.56"));
			CHECK(Entry.IsValid());
			CHECK(Entry.SuggestedDriverVersion == TEXT("111.56"));
		}
		{
			FSuggestedDriverEntry Entry(TEXT("BadString=111.56"));
			CHECK(!Entry.IsValid());
		}
	}
}

#endif //WITH_TESTS