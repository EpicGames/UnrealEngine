// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS 

#include "GenericPlatform/GenericPlatformMisc.h"

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FGenericPlatformMiscTest, "System::Core::GenericPlatform", "[Core][GenericPlatform][SmokeFilter]")
{
	SECTION("Parse pakchunk index")
	{
		CHECK(1 == FGenericPlatformMisc::GetPakchunkIndexFromPakFile(FString(FGenericPlatformMisc::GetPakFilenamePrefix()) + TEXT("1-Windows")));
		CHECK(12 == FGenericPlatformMisc::GetPakchunkIndexFromPakFile(TEXT("../../../Content/Paks/") + FString(FGenericPlatformMisc::GetPakFilenamePrefix()) + TEXT("12-Windows")));
		CHECK(42 == FGenericPlatformMisc::GetPakchunkIndexFromPakFile(FString(FGenericPlatformMisc::GetPakFilenamePrefix()) + TEXT("42")));
		CHECK(INDEX_NONE == FGenericPlatformMisc::GetPakchunkIndexFromPakFile(FString(FGenericPlatformMisc::GetPakFilenamePrefix())));
	}
}

#endif // WITH_TESTS
