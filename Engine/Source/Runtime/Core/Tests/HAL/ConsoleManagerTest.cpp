// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "HAL/ConsoleManager.h"

#include "Misc/OutputDeviceNull.h"
#include "Misc/ScopeExit.h"
#include "Tests/TestHarnessAdapter.h"

// TODO: Try to move these into the test
static TAutoConsoleVariable<int32> CVarDebugEarlyDefault(
	TEXT("con.DebugEarlyDefault"),
	21,
	TEXT("used internally to test the console variable system"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarDebugEarlyCheat(
	TEXT("con.DebugEarlyCheat"),
	22,
	TEXT("used internally to test the console variable system"),
	ECVF_Cheat);

TEST_CASE_NAMED(FConsoleManagerTest, "System::Core::HAL::ConsoleManager", "[ApplicationContextMask][Core]")
{
	// FConsoleManager ManagerImpl{};
	// IConsoleManager& Manager = ManagerImpl;

	IConsoleManager& Manager = IConsoleManager::Get();
	FConsoleManager& ManagerImpl = static_cast<FConsoleManager&>(Manager);

	// HACK: Temporarily disable any thread propagation callback. This causes setting a variable to be deferred which
	// the write-read test below to become a race condition. The proper solution here is to create a temporary
	// FConsoleManager instance so we aren't mucking with the real runtime instance. However,
	// IConsoleObject::OnCVarChange looks up the global IConsoleManager::Singleton, instead of using our temporary
	// instance. Changing console objects to contain a pointer to their manager instead of doing a global lookup would
	// solve that problem, at the expense of making objects 8 bytes larger.
	IConsoleThreadPropagation* Prop = ManagerImpl.GetThreadPropagationCallback();
	Manager.RegisterThreadPropagation(0, nullptr);
	ON_SCOPE_EXIT { Manager.RegisterThreadPropagation(0, Prop); };

	// we only test the main thread side of ECVF_RenderThreadSafe so we expect the same results
	for (EConsoleVariableFlags Flags : {ECVF_Default, ECVF_RenderThreadSafe})
	{
		int32 RefD = 2;
		float RefE = 2.1f;

		IConsoleVariable* VarA = Manager.RegisterConsoleVariable(TEXT("TestNameA"), 1, TEXT("TestHelpA"), Flags);
		IConsoleVariable* VarB = Manager.RegisterConsoleVariable(TEXT("TestNameB"), 1.2f, TEXT("TestHelpB"), Flags);
		IConsoleVariable* VarD = Manager.RegisterConsoleVariableRef(TEXT("TestNameD"), RefD, TEXT("TestHelpD"), Flags);
		IConsoleVariable* VarE = Manager.RegisterConsoleVariableRef(TEXT("TestNameE"), RefE, TEXT("TestHelpE"), Flags);

		SECTION("Console Variable Sinks")
		{
			// Sinks are initiallially queued
			Manager.CallAllConsoleVariableSinks();

			uint32 SinkCounter = 0;
			auto SinkCallback = [&SinkCounter]() { ++SinkCounter; };
			FConsoleCommandDelegate SinkDelegate = FConsoleCommandDelegate::CreateLambda(SinkCallback);
			FConsoleVariableSinkHandle SinkHandle = Manager.RegisterConsoleVariableSink_Handle(SinkDelegate);
			ON_SCOPE_EXIT { Manager.UnregisterConsoleVariableSink_Handle(SinkHandle); };

			IConsoleVariable* Var = Manager.RegisterConsoleVariable(TEXT("TestNameX"), 1, TEXT("TestHelpX"), Flags);
			ON_SCOPE_EXIT { Manager.UnregisterConsoleObject(Var, false); };

			Manager.CallAllConsoleVariableSinks();
			CHECK(SinkCounter == 0);

			Var->Set(2);

			// this should trigger the callback
			Manager.CallAllConsoleVariableSinks();
			CHECK(SinkCounter == 1);

			// this should not trigger the callback
			Manager.CallAllConsoleVariableSinks();
			CHECK(SinkCounter == 1);
		}

		uint32 ChangeCounter = 0;
		auto ChangeCallback = [&ChangeCounter](IConsoleVariable* Var)
		{
			CHECK(Var);
			if (Var)
			{
				float Value = Var->GetFloat();
				CHECK(FMath::IsNearlyEqual(Value, 3.1f, UE_KINDA_SMALL_NUMBER));
				++ChangeCounter;
			}
		};
		FConsoleVariableDelegate ChangeDelegate = FConsoleVariableDelegate::CreateLambda(ChangeCallback);
		VarB->SetOnChangedCallback(ChangeDelegate);
		CHECK(ChangeCounter == 0);

		SECTION("Register variables")
		{
			// at the moment ECVF_SetByConstructor has to be 0 or we set ECVF_Default to ECVF_SetByConstructor
			CHECK((VarA->GetFlags() & ECVF_SetByMask) == ECVF_SetByConstructor);

			CHECK(VarA == Manager.FindConsoleVariable(TEXT("TestNameA")));
			CHECK(VarB == Manager.FindConsoleVariable(TEXT("TestNameB")));
			CHECK(VarD == Manager.FindConsoleVariable(TEXT("TestNameD")));
			CHECK(VarE == Manager.FindConsoleVariable(TEXT("TestNameE")));
		}

		SECTION("Get variable values")
		{
			CHECK(VarA->GetInt() == 1);
			CHECK(VarA->GetFloat() == (float)1);
			CHECK(VarA->GetString() == FString("1"));

			CHECK(VarB->GetInt() == 1);
			CHECK(FMath::IsNearlyEqual(VarB->GetFloat(), 1.2f, UE_KINDA_SMALL_NUMBER));
			CHECK(VarB->GetString() == FString("1.2"));

			CHECK(RefD == 2);
			CHECK(VarD->GetInt() == 2);
			CHECK(VarD->GetFloat() == (float)2);
			CHECK(VarD->GetString() == FString("2"));

			CHECK(FMath::IsNearlyEqual(RefE, 2.1f, UE_KINDA_SMALL_NUMBER));
			CHECK(VarE->GetInt() == (int32)RefE);
			CHECK(VarE->GetFloat() == RefE);
			CHECK(VarE->GetString() == FString("2.1"));
		}

		SECTION("Set variable values (string)")
		{
			VarA->Set(TEXT("3.1"), ECVF_SetByConsoleVariablesIni);
			VarB->Set(TEXT("3.1"), ECVF_SetByConsoleVariablesIni);
			VarD->Set(TEXT("3.1"), ECVF_SetByConsoleVariablesIni);
			VarE->Set(TEXT("3.1"), ECVF_SetByConsoleVariablesIni);
			CHECK(ChangeCounter == 1);

			CHECK(VarA->GetString() == FString("3"));
			CHECK(VarB->GetString() == FString("3.1"));
			CHECK(VarD->GetString() == FString("3"));
			CHECK(VarE->GetString() == FString("3.1"));
			CHECK(RefD == 3);
			CHECK(RefE == 3.1f);

			VarB->Set(TEXT("3.1"), ECVF_SetByConsoleVariablesIni);
			CHECK(ChangeCounter == 2);
		}

		if ((Flags & ECVF_RenderThreadSafe) == 0)
		{
			// string is not supported with the flag ECVF_RenderThreadSafe
			IConsoleVariable* VarC = Manager.RegisterConsoleVariable(TEXT("TestNameC"), TEXT("1.23"), TEXT("TestHelpC"), Flags);
			CHECK(VarC == Manager.FindConsoleVariable(TEXT("TestNameC")));
			CHECK(VarC->GetInt() == 1);
			// note: exact comparison fails in Win32 release
			CHECK(FMath::IsNearlyEqual(VarC->GetFloat(), 1.23f, UE_KINDA_SMALL_NUMBER));
			CHECK(VarC->GetString() == FString("1.23"));
			VarC->Set(TEXT("3.1"), ECVF_SetByConsole);
			CHECK(VarC->GetString() == FString("3.1"));

			Manager.UnregisterConsoleObject(TEXT("TestNameC"), false);
			CHECK(!Manager.FindConsoleVariable(TEXT("TestNameC")));
		}

		Manager.UnregisterConsoleObject(VarA);
		Manager.UnregisterConsoleObject(VarB, false);
		Manager.UnregisterConsoleObject(TEXT("TestNameD"), false);
		Manager.UnregisterConsoleObject(TEXT("TestNameE"), false);

		SECTION("Unregister variables")
		{
			CHECK(!Manager.FindConsoleVariable(TEXT("TestNameA")));
			CHECK(!Manager.FindConsoleVariable(TEXT("TestNameB")));
			CHECK(!Manager.FindConsoleVariable(TEXT("TestNameD")));
			CHECK(!Manager.FindConsoleVariable(TEXT("TestNameE")));
		}

		SECTION("Re-register variables but maintain state")
		{
			IConsoleVariable* SecondVarA = Manager.RegisterConsoleVariable(TEXT("TestNameA"), 1234, TEXT("TestHelpSecondA"), Flags);
			CHECK(SecondVarA == VarA);
			CHECK(SecondVarA->GetInt() == 3);
			CHECK(Manager.FindConsoleVariable(TEXT("TestNameA")));

			Manager.UnregisterConsoleObject(TEXT("TestNameA"), false);
			CHECK(!Manager.FindConsoleVariable(TEXT("TestNameA")));
		}

		SECTION("Priority")
		{
			IConsoleVariable* VarX = Manager.RegisterConsoleVariable(TEXT("TestNameX"), 1, TEXT("TestHelpX"), Flags);
			ON_SCOPE_EXIT { Manager.UnregisterConsoleObject(VarX, false); };

			CHECK(((uint32)VarX->GetFlags() & ECVF_SetByMask) == ECVF_SetByConstructor);

			VarX->Set(TEXT("3.1"), ECVF_SetByConsoleVariablesIni);
			CHECK(((uint32)VarX->GetFlags() & ECVF_SetByMask) == ECVF_SetByConsoleVariablesIni);

			// lower should fail
			VarX->Set(TEXT("111"), ECVF_SetByScalability);
			CHECK(VarX->GetString() == FString("3"));
			CHECK(((uint32)VarX->GetFlags() & ECVF_SetByMask) == ECVF_SetByConsoleVariablesIni);

			// higher should work
			VarX->Set(TEXT("222"), ECVF_SetByCommandline);
			CHECK(VarX->GetString() == FString("222"));
			CHECK(((uint32)VarX->GetFlags() & ECVF_SetByMask) == ECVF_SetByCommandline);

			// lower should fail
			VarX->Set(TEXT("333"), ECVF_SetByConsoleVariablesIni);
			CHECK(VarX->GetString() == FString("222"));
			CHECK(((uint32)VarX->GetFlags() & ECVF_SetByMask) == ECVF_SetByCommandline);

			// higher should work
			VarX->Set(TEXT("444"), ECVF_SetByConsole);
			CHECK(VarX->GetString() == FString("444"));
			CHECK(((uint32)VarX->GetFlags() & ECVF_SetByMask) == ECVF_SetByConsole);
		}
	}

	// We don't load config in Low Level Tests
	if constexpr (!WITH_LOW_LEVEL_TESTS)
	{
		SECTION("ECVF_Cheat")
		{
			IConsoleVariable* VarC = Manager.RegisterConsoleVariable(TEXT("con.DebugLateDefault"), 23, TEXT(""), ECVF_Default);
			ON_SCOPE_EXIT { Manager.UnregisterConsoleObject(VarC, true); };

			IConsoleVariable* VarD = Manager.RegisterConsoleVariable(TEXT("con.DebugLateCheat"), 24, TEXT(""), ECVF_Cheat);
			ON_SCOPE_EXIT { Manager.UnregisterConsoleObject(VarD, true); };

			// in BaseEngine.ini we set all 4 cvars to "True" but only the non cheat one should pick up the value
			CHECK(CVarDebugEarlyDefault.GetValueOnGameThread() == 1);
			CHECK(CVarDebugEarlyCheat.GetValueOnGameThread() == 22);
			CHECK(VarC->GetInt() == 1);
			CHECK(VarD->GetInt() == 24);
		}
	}

	SECTION("Deprecated console variables should not assert in dump commands")
	{
		constexpr TCHAR OldName[] = TEXT("TestVar.Old");
		constexpr TCHAR NewName[] = TEXT("TestVar.New");

		FAutoConsoleVariable NewVar(NewName, false, TEXT(""));
		FAutoConsoleVariableDeprecated OldVar(OldName, NewName, TEXT("0.0"));
		ON_SCOPE_EXIT { Manager.UnregisterConsoleObject(OldName, false); };

		FOutputDeviceNull OutNull;
		ManagerImpl.DumpObjects(TEXT("-ShowHelp"), OutNull, false);
		ManagerImpl.DumpObjects(TEXT("-ShowHelp -Deprecated"), OutNull, false);
	}

	SECTION("Deprecated console commands should not assert in dump commands")
	{
		constexpr TCHAR OldName[] = TEXT("TestCmd.Old");
		constexpr TCHAR NewName[] = TEXT("TestCmd.New");

		FAutoConsoleCommand NewCmd(NewName, TEXT(""), FConsoleCommandDelegate());
		FAutoConsoleCommandDeprecated OldCmd(OldName, NewName, TEXT("0.0"));
		ON_SCOPE_EXIT { Manager.UnregisterConsoleObject(OldName, false); };

		FOutputDeviceNull OutNull;
		ManagerImpl.DumpObjects(TEXT("-ShowHelp"), OutNull, true);
		ManagerImpl.DumpObjects(TEXT("-ShowHelp -Deprecated"), OutNull, true);
	}
};

#endif
