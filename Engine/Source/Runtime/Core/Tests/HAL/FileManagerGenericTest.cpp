// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/FileManagerGeneric.h"
#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_TESTS

class FArchiveFileReaderGenericTest
{
public:
	~FArchiveFileReaderGenericTest();
	void TestInternalPrecache();

private:
	void CreateTestFile();
	void TestBytesValid(const TCHAR* What, uint8* Data, int64 NumBytes, int64 FileOffset);
	int32 GetExpectedValue(int64 IntOffset);
	void SetPosAndBuffer(TUniquePtr<FArchiveFileReaderGeneric>& Reader, int64 Pos, int64 BufferBase, int64 BufferSize);

	FString TestFileName;
	TUniquePtr<IFileHandle> ReadHandle;
	int64 FileSize;
};

FArchiveFileReaderGenericTest::~FArchiveFileReaderGenericTest()
{
	ReadHandle.Reset();
	if (!TestFileName.IsEmpty())
	{
		IFileManager& FileManager = IFileManager::Get();
		FileManager.Delete(*TestFileName, false /* bRequireExists */, true /* bEvenReadOnly */, true /* Quiet */);
	}
	TestFileName.Empty();
}

void FArchiveFileReaderGenericTest::TestInternalPrecache()
{
	CreateTestFile();

	uint32 BufferSize = 1024;
	TArray<uint8> UnusedBytes;
	UnusedBytes.SetNumUninitialized(BufferSize);
	TUniquePtr<FArchiveFileReaderGeneric> Reader(new FArchiveFileReaderGeneric(ReadHandle.Release(), *TestFileName, FileSize, BufferSize));
	CHECK_MESSAGE(TEXT("Initial Pos should be 0"), Reader->Pos == 0LL);
	CHECK_MESSAGE(TEXT("Size should be what was passed in"), Reader->Size == FileSize);
	CHECK_MESSAGE(TEXT("BufferSize should be what was passed in"), Reader->BufferSize == BufferSize);

	// Vanilla InternalPrecache at start of file
	bool Result = Reader->InternalPrecache(0, BufferSize);
	CHECK_MESSAGE(TEXT("Vanilla0 - InternalPrecache should succeed"), Result);
	CHECK_MESSAGE(TEXT("Vanilla0 - InternalPrecache should not alter Pos"), Reader->Pos == 0LL);
	CHECK_MESSAGE(TEXT("Vanilla0 - BufferBase should have be set to PrecacheOffset aka Pos"), Reader->BufferBase == 0LL);
	CHECK_MESSAGE(TEXT("Vanilla0 - Bytes precached should be set to BufferSize unless it runs out of room"), Reader->BufferArray.Num() == BufferSize);
	TestBytesValid(TEXT("Vanilla0 - BufferArray should be expected bytes"), Reader->BufferArray.GetData(), Reader->BufferArray.Num(), Reader->BufferBase);

	// InternalPrecache at PrecacheOffset != Pos is ignored, and returns false if the first byte at PrecacheOffset is not cached
	Result = Reader->InternalPrecache(BufferSize  + BufferSize, BufferSize);
	CHECK_FALSE_MESSAGE(TEXT("PrecacheNotAtPos - Should fail"), Result);
	CHECK_MESSAGE(TEXT("PrecacheNotAtPos - BufferBase should not be altered"), Reader->BufferBase == 0LL);
	CHECK_MESSAGE(TEXT("PrecacheNotAtPos - Buffe should not be altered"), Reader->BufferArray.Num() == BufferSize);

	// InternalPrecache at PrecacheOffset != Pos is ignored, but returns true if the first byte at PrecacheOffset is cached
	Result = Reader->InternalPrecache(BufferSize - 1, BufferSize);
	CHECK_MESSAGE(TEXT("PrecacheNotAtPos - Should succeed since the first byte at PrecacheOffset is buffered"), Result);
	CHECK_MESSAGE(TEXT("PrecacheNotAtPos - BufferBase should not be altered"), Reader->BufferBase == 0LL);
	CHECK_MESSAGE(TEXT("PrecacheNotAtPos - Buffe should not be altered"), Reader->BufferArray.Num() == BufferSize);

	// InternalPrecache partway through the buffer should allocate the full buffer/not allocate anything, depending on bPrecacheAsSoonAsPossible
	int64 PosStart = BufferSize / 2;
	SetPosAndBuffer(Reader, PosStart, 0, BufferSize);
	Result = Reader->InternalPrecache(PosStart, BufferSize);
	CHECK_MESSAGE(TEXT("PrecachePartwayPosMoreThanBuffer - Should succeed"), Result);
	CHECK_MESSAGE(TEXT("PrecachePartwayPosMoreThanBuffer - BufferBase should not be updated"), Reader->BufferBase == 0LL);
	CHECK_MESSAGE(TEXT("PrecachePartwayPosMoreThanBuffer - BufferCount should be set to BufferSize"), Reader->BufferArray.Num() == BufferSize);
	TestBytesValid(TEXT("PrecachePartwayPosMoreThanBuffer - BufferBytes should match BufferBase"), Reader->BufferArray.GetData(), Reader->BufferArray.Num(), Reader->BufferBase);

	// InternalPrecache partway through the buffer should allocate the full buffer/not allocate anything, and should handle the case of BufferArray's allocation being smaller than BufferSize
	SetPosAndBuffer(Reader, 3*BufferSize/4, BufferSize/2, BufferSize/2);
	Reader->BufferArray.Shrink();
	Result = Reader->InternalPrecache(3*BufferSize/4, BufferSize);
	CHECK_MESSAGE(TEXT("PrecachePartwayPosMoreThanBufferReallocation - Should succeed"), Result);
	CHECK_MESSAGE(TEXT("PrecachePartwayPosMoreThanBufferReallocation - BufferBase should not be updated"), Reader->BufferBase == BufferSize/2);
	CHECK_MESSAGE(TEXT("PrecachePartwayPosMoreThanBufferReallocation - BufferCount should not be updated"), Reader->BufferArray.Num() == BufferSize/2);
	TestBytesValid(TEXT("PrecachePartwayPosMoreThanBufferReallocation - BufferBytes should match BufferBase"), Reader->BufferArray.GetData(), Reader->BufferArray.Num(), Reader->BufferBase);

	// InternalPrecache partway through the buffer, when the requested size is within the remaining buffer, should not change the buffer
	PosStart = BufferSize / 2;
	SetPosAndBuffer(Reader, PosStart, 0, BufferSize);
	Result = Reader->InternalPrecache(PosStart, BufferSize / 4);
	CHECK_MESSAGE(TEXT("PrecachePartwayPosLessThanBuffer - Should succeed"), Result);
	CHECK_MESSAGE(TEXT("PrecachePartwayPosLessThanBuffer - BufferBase should not be updated"), Reader->BufferBase == 0LL);
	CHECK_MESSAGE(TEXT("PrecachePartwayPosLessThanBuffer - BufferCount should be set to BufferSize"), Reader->BufferArray.Num() == BufferSize);
	TestBytesValid(TEXT("PrecachePartwayPosLessThanBuffer - BufferBytes should match BufferBase"), Reader->BufferArray.GetData(), Reader->BufferArray.Num(), Reader->BufferBase);

	// InternalPrecache at the end of the buffer should allocate the full buffer
	PosStart = BufferSize;
	SetPosAndBuffer(Reader, PosStart, 0, BufferSize);
	Result = Reader->InternalPrecache(PosStart, BufferSize);
	CHECK_MESSAGE(TEXT("PrecachePosEndOfBuffer - Should succeed"), Result);
	CHECK_MESSAGE(TEXT("PrecachePosEndOfBuffer - BufferBase should be updated"), Reader->BufferBase == PosStart);
	CHECK_MESSAGE(TEXT("PrecachePosEndOfBuffer - BufferCount should be set to BufferSize"), Reader->BufferArray.Num() == BufferSize);
	TestBytesValid(TEXT("PrecachePosEndOfBuffer - BufferBytes should match BufferBase"), Reader->BufferArray.GetData(), Reader->BufferArray.Num(), Reader->BufferBase);

	// InternalPrecache near the end of the file should clamp BufferSize to Size - Pos
	PosStart = FileSize - 1;
	SetPosAndBuffer(Reader, PosStart, 0, 0);
	Result = Reader->InternalPrecache(PosStart, BufferSize);
	CHECK_MESSAGE(TEXT("PrecacheNearEndOfFile - Should succeed"), Result);
	CHECK_MESSAGE(TEXT("PrecacheNearEndOfFile - BufferBase should be updated"), Reader->BufferBase == PosStart);
	CHECK_MESSAGE(TEXT("PrecacheNearEndOfFile - BufferCount should be set Min(BufferSize, Size - Pos)"), Reader->BufferArray.Num() == 1LL);
	TestBytesValid(TEXT("PrecacheNearEndOfFile - BufferBytes should match BufferBase"), Reader->BufferArray.GetData(), Reader->BufferArray.Num(), Reader->BufferBase);

	FAIL_ON_MESSAGE(TEXT("ReadFile failed"));
	// InternalPrecache at the end of the file should fail
	SetPosAndBuffer(Reader, FileSize, 0, 16);
	Result = Reader->InternalPrecache(FileSize, BufferSize);
	CHECK_FALSE_MESSAGE(TEXT("PrecacheEndOfFile - Should fail"), Result);
	CHECK_MESSAGE(TEXT("PrecacheEndOfFile - BufferBase should not be updated"), Reader->BufferBase == 0LL);
	CHECK_MESSAGE(TEXT("PrecacheEndOfFile - BufferCount should not be updated"), Reader->BufferArray.Num() == 16LL);

	// If ReadLowLevel fails, InternalPrecache should return true if the first byte of PrecacheOffset is in the buffer
	SetPosAndBuffer(Reader, 8, 0, 16);
	Reader->SeekLowLevel(FileSize);
	Result = Reader->InternalPrecache(8, 16);
	CHECK_MESSAGE(TEXT("PrecacheReadFailsBytesRemain - Should succeed if bytes are left in the buffer"), Result);
	CHECK_MESSAGE(TEXT("PrecacheReadFailsBytesRemain - BufferBase should not be updated"), Reader->BufferBase == 0LL);
	CHECK_MESSAGE(TEXT("PrecacheReadFailsBytesRemain - BufferCount should not be updated"), Reader->BufferArray.Num() == 16LL);

	// If ReadLowLevel fails, InternalPrecache should return false if the first byte of PrecacheOffset is not in the buffer
	SetPosAndBuffer(Reader, 16, 0, 16);
	Reader->SeekLowLevel(FileSize);
	Result = Reader->InternalPrecache(16, 16);
	CHECK_FALSE_MESSAGE(TEXT("PrecacheReadFails - Should fail if bytes are not left in the buffer"), Result);
	CHECK_MESSAGE(TEXT("PrecacheReadFails - BufferCount should be updated to empty"), Reader->BufferArray.Num() == 0LL);

	// TODO: Call ClearExpectedErrors to clear the AddExpectedError called above when it is implemented. Until then, make sure these failure-testing cases are the last
	// cases tested in the test
	//ClearExpectedErrors();
}

void FArchiveFileReaderGenericTest::CreateTestFile()
{
	IFileManager& FileManager = IFileManager::Get();
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();;
	TestFileName = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("ArchiveFileReaderGenericTest"));
	TUniquePtr<IFileHandle> WriteHandle(PlatformFile.OpenWrite(*TestFileName, false /* bAppend */, false /* bAllowRead */));
	REQUIRE_MESSAGE(FString::Printf(TEXT("Could not create test file %s."), *TestFileName), WriteHandle.IsValid());
	TArray<int32> Builder;
	int32 NumInts = 1024 * 128 + 17; // Make sure the file size is not a multiple of buffersize; we need to test that behavior
	Builder.SetNumUninitialized(NumInts);
	int32* Data = Builder.GetData();
	for (int n = 0; n < Builder.Num(); ++n)
	{
		Data[n] = GetExpectedValue(n);
	}
	FileSize = NumInts * sizeof(int32);
	WriteHandle->Write(reinterpret_cast<uint8*>(Data), FileSize);
	WriteHandle.Reset();

	ReadHandle.Reset(PlatformFile.OpenRead(*TestFileName, false /* bAllowWrite */));

	REQUIRE_MESSAGE(FString::Printf(TEXT("Could not open test file %s."), *TestFileName), ReadHandle.IsValid());
	REQUIRE_MESSAGE(FString::Printf(TEXT("Received incorrect file size from test file %s. Expected = %lld. Actual = %lld."), *TestFileName, FileSize, ReadHandle->Size()), ReadHandle->Size() == FileSize);
}

void FArchiveFileReaderGenericTest::TestBytesValid(const TCHAR* What, uint8* InData, int64 InNumBytes, int64 InFileOffset)
{
	uint8* Data = InData;
	int64 NumBytes = InNumBytes;
	int64 FileOffset = InFileOffset;

	int64 StartByte = FileOffset % sizeof(int32);
	if (StartByte != 0)
	{
		int64 FloorIndex = (FileOffset - StartByte)/sizeof(int32);
		int64 ExpectedValue = GetExpectedValue(FloorIndex);
		for (int n = 0; n < NumBytes && n < sizeof(int32) - StartByte; ++n)
		{

			if (Data[n] != reinterpret_cast<uint8*>(&ExpectedValue)[StartByte + n])
			{
				CHECK_MESSAGE(What, Data[n] == reinterpret_cast<uint8*>(&ExpectedValue)[StartByte + n]);
				return;
			}

		}
		if (NumBytes < static_cast<int64>(sizeof(int32) - StartByte))
		{
			return;
		}
		int32 BytesRead = sizeof(int32) - StartByte;
		Data += BytesRead;
		FileOffset += BytesRead;
		NumBytes -= BytesRead;
	}
	int32* IntData = reinterpret_cast<int32*>(Data);
	int64 IntOffset = FileOffset / sizeof(int32);
	int64 NumInts = NumBytes / sizeof(int32);
	for (int n = 0; n < NumInts; ++n)
	{
		if (IntData[n] != GetExpectedValue(IntOffset + n))
		{
			CHECK_MESSAGE(What, IntData[n] == GetExpectedValue(IntOffset + n));
			return;
		}
	}

	int64 NumEndBytes = NumBytes - NumInts * sizeof(int32);
	if (NumEndBytes != 0)
	{
		uint8* DataStart = Data + NumInts * sizeof(int32);
		int32 ExpectedValue = GetExpectedValue(IntOffset + NumInts);
		for (int n = 0; n < NumEndBytes; ++n)
		{

			if (DataStart[n] != reinterpret_cast<uint8*>(&ExpectedValue)[n])
			{
				CHECK_MESSAGE(What, DataStart[n] == reinterpret_cast<uint8*>(&ExpectedValue)[n]);
				return;
			}
		}
	}
}

void FArchiveFileReaderGenericTest::SetPosAndBuffer(TUniquePtr<FArchiveFileReaderGeneric>& Reader, int64 Pos, int64 BufferBase, int64 BufferSize)
{
	Reader->Pos = Pos;
	Reader->BufferBase = BufferBase;
	Reader->BufferArray.SetNumUninitialized(BufferSize, EAllowShrinking::No);
	if (BufferSize)
	{
		Reader->SeekLowLevel(BufferBase);
		int64 BytesRead;
		Reader->ReadLowLevel(Reader->BufferArray.GetData(), BufferSize, BytesRead);
		CHECK_MESSAGE(TEXT("SetPosAndBuffer - ReadLowLevel read the requested bytes"), BytesRead == BufferSize);
	}
	// See the contract for Buffer window and LowLevel Pos in the variable comment on BufferArray
	bool bPosWithinBuffer = BufferBase <= Pos && Pos < BufferBase + BufferSize;
	if (!bPosWithinBuffer)
	{
		Reader->SeekLowLevel(Pos);
	}
	CHECK_MESSAGE(TEXT("SetPosAndBuffer - Pos set correctly"), Reader->Pos == Pos);
	CHECK_MESSAGE(TEXT("SetPosAndBuffer - BufferBase set correctly"), Reader->BufferBase == BufferBase);
	CHECK_MESSAGE(TEXT("SetPosAndBuffer - BufferSize set correctly"), Reader->BufferArray.Num() == BufferSize);
	TestBytesValid(TEXT("SetPosAndBuffer - BufferBytes should match BufferBase"), Reader->BufferArray.GetData(), Reader->BufferArray.Num(), Reader->BufferBase);
}

int32 FArchiveFileReaderGenericTest::GetExpectedValue(int64 IntOffset)
{
	return 0xbe000000 + static_cast<int32>(IntOffset);
}

TEST_CASE_NAMED(FArchiveFileReaderGenericTestRunner, "System::Core::HAL::FArchiveFileReaderGeneric", "[.][ApplicationContextMask][EngineFilter]")
{
	FArchiveFileReaderGenericTest Instance = FArchiveFileReaderGenericTest();
	Instance.TestInternalPrecache();
}

#endif //WITH_TESTS
