// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Tests/TestHarnessAdapter.h"
#include "Containers/UnrealString.h"

#if WITH_TESTS

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformAtomics.h"
#endif
#include "HAL/PlatformAtomics.h"

template<typename T>
static bool TestInterlocked(volatile T* Dest, T ExpectedReturnValue, T ExpectedFinalValue, TFunctionRef<T(volatile T*)> InterlockedFunc, const TCHAR* FunctionName, const TCHAR* TypeName)
{
	const T ReturnValue = InterlockedFunc(Dest);
	if (ReturnValue != ExpectedReturnValue)
	{
		check(false);
		FAIL_CHECK(FString::Printf(TEXT("FPlatformAtomics::Interlocked%s on %s failed"), FunctionName, TypeName));
		return false;
	}
	if (FPlatformAtomics::AtomicRead(Dest) != ExpectedFinalValue)
	{
		check(false);
		FAIL_CHECK(FString::Printf(TEXT("FPlatformAtomics::Interlocked%s on %s failed"), FunctionName, TypeName));
		return false;
	}
	return true;
}

static bool TestInterlockedAnd()
{
	bool bSuccess = true;

	const TCHAR* FunctionName = TEXT("And");
	{
		// Test and with value where some bits are set and some aren't set in the current value.
		volatile int8 Value0 = 0x30;
		bSuccess = TestInterlocked<int8>(&Value0, 0x30, 0x20, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, 0x66); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test and with all bits set.
		volatile int8 Value1 = 0x30;
		bSuccess = TestInterlocked<int8>(&Value1, 0x30, 0x30, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, -1); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test and with zero.
		volatile int8 Value2 = 0x30;
		bSuccess = TestInterlocked<int8>(&Value2, 0x30, 0, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, 0); }, FunctionName, TEXT("int8")) && bSuccess;
	}

	{
		// Test and with value where some bits are set and some aren't set in the current value.
		volatile int16 Value0 = 0x3030;
		bSuccess = TestInterlocked<int16>(&Value0, 0x3030, 0x2020, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, 0x6666); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test and with all bits set.
		volatile int16 Value1 = 0x3030;
		bSuccess = TestInterlocked<int16>(&Value1, 0x3030, 0x3030, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, -1); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test and with zero.
		volatile int16 Value2 = 0x3030;
		bSuccess = TestInterlocked<int16>(&Value2, 0x3030, 0, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, 0); }, FunctionName, TEXT("int16")) && bSuccess;
	}

	{
		// Test and with value where some bits are set and some aren't set in the current value.
		volatile int32 Value0 = 0x30303030;
		bSuccess = TestInterlocked<int32>(&Value0, 0x30303030, 0x20202020, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, 0x66666666); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test and with all bits set.
		volatile int32 Value1 = 0x30303030;
		bSuccess = TestInterlocked<int32>(&Value1, 0x30303030, 0x30303030, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, -1); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test and with zero.
		volatile int32 Value2 = 0x30303030;
		bSuccess = TestInterlocked<int32>(&Value2, 0x30303030, 0, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, 0); }, FunctionName, TEXT("int32")) && bSuccess;
	}

	{
		// Test and with value where some bits are set and some aren't set in the current value.
		alignas(8) volatile int64 Value0 = 0x3030303030303030LL;
		bSuccess = TestInterlocked<int64>(&Value0, 0x3030303030303030LL, 0x2020202020202020LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, 0x6666666666666666); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test and with all bits set.
		alignas(8) volatile int64 Value1 = 0x3030303030303030LL;
		bSuccess = TestInterlocked<int64>(&Value1, 0x3030303030303030LL, 0x3030303030303030LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, -1); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test and with zero.
		alignas(8) volatile int64 Value2 = 0x3030303030303030LL;
		bSuccess = TestInterlocked<int64>(&Value2, 0x3030303030303030LL, 0, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedAnd(Dest, 0); }, FunctionName, TEXT("int64")) && bSuccess;
	}

	return bSuccess;
}

static bool TestInterlockedOr()
{
	bool bSuccess = true;

	const TCHAR* FunctionName = TEXT("Or");
	{
		// Test or with value where some bits are set and some aren't set in the current value.
		volatile int8 Value0 = 0x30;
		bSuccess = TestInterlocked<int8>(&Value0, 0x30, 0x76, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedOr(Dest, 0x66); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test or with all bits set.
		volatile int8 Value1 = 0x30;
		bSuccess = TestInterlocked<int8>(&Value1, 0x30, -1, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedOr(Dest, -1); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test or with zero.
		volatile int8 Value2 = 0x30;
		bSuccess = TestInterlocked<int8>(&Value2, 0x30, 0x30, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedOr(Dest, 0); }, FunctionName, TEXT("int8")) && bSuccess;
	}

	{
		// Test or with value where some bits are set and some aren't set in the current value.
		volatile int16 Value0 = 0x3030;
		bSuccess = TestInterlocked<int16>(&Value0, 0x3030, 0x7676, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedOr(Dest, 0x6666); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test or with all bits set.
		volatile int16 Value1 = 0x3030;
		bSuccess = TestInterlocked<int16>(&Value1, 0x3030, -1, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedOr(Dest, -1); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test or with zero.
		volatile int16 Value2 = 0x3030;
		bSuccess = TestInterlocked<int16>(&Value2, 0x3030, 0x3030, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedOr(Dest, 0); }, FunctionName, TEXT("int16")) && bSuccess;
	}

	{
		// Test or with value where some bits are set and some aren't set in the current value.
		volatile int32 Value0 = 0x30303030;
		bSuccess = TestInterlocked<int32>(&Value0, 0x30303030, 0x76767676, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedOr(Dest, 0x66666666); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test or with all bits set.
		volatile int32 Value1 = 0x30303030;
		bSuccess = TestInterlocked<int32>(&Value1, 0x30303030, -1, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedOr(Dest, -1); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test or with zero.
		volatile int32 Value2 = 0x30303030;
		bSuccess = TestInterlocked<int32>(&Value2, 0x30303030, 0x30303030, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedOr(Dest, 0); }, FunctionName, TEXT("int32")) && bSuccess;
	}

	{
		// Test or with value where some bits are set and some aren't set in the current value.
		alignas(8) volatile int64 Value0 = 0x3030303030303030LL;
		bSuccess = TestInterlocked<int64>(&Value0, 0x3030303030303030LL, 0x7676767676767676LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedOr(Dest, 0x6666666666666666LL); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test or with all bits set.
		alignas(8) volatile int64 Value1 = 0x3030303030303030LL;
		bSuccess = TestInterlocked<int64>(&Value1, 0x3030303030303030LL, -1LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedOr(Dest, -1); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test or with zero.
		alignas(8) volatile int64 Value2 = 0x3030303030303030LL;
		bSuccess = TestInterlocked<int64>(&Value2, 0x3030303030303030LL, 0x3030303030303030LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedOr(Dest, 0); }, FunctionName, TEXT("int64")) && bSuccess;
	}

	return bSuccess;
}

static bool TestInterlockedXor()
{
	bool bSuccess = true;

	const TCHAR* FunctionName = TEXT("Xor");
	{
		// Test xor with value where some bits are set and some aren't set in the current value.
		volatile int8 Value0 = 0x30;
		bSuccess = TestInterlocked<int8>(&Value0, 0x30, 0x56, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedXor(Dest, 0x66); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test xor with all bits set.
		volatile int8 Value1 = 0x30;
		bSuccess = TestInterlocked<int8>(&Value1, 0x30, int8(~0x30), [](volatile int8* Dest) { return FPlatformAtomics::InterlockedXor(Dest, -1); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test xor with zero.
		volatile int8 Value2 = 0x30;
		bSuccess = TestInterlocked<int8>(&Value2, 0x30, 0x30, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedXor(Dest, 0); }, FunctionName, TEXT("int8")) && bSuccess;
	}

	{
		// Test xor with value where some bits are set and some aren't set in the current value.
		volatile int16 Value0 = 0x3030;
		bSuccess = TestInterlocked<int16>(&Value0, 0x3030, 0x5656, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedXor(Dest, 0x6666); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test xor with all bits set.
		volatile int16 Value1 = 0x3030;
		bSuccess = TestInterlocked<int16>(&Value1, 0x3030, int16(~0x3030), [](volatile int16* Dest) { return FPlatformAtomics::InterlockedXor(Dest, -1); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test xor with zero.
		volatile int16 Value2 = 0x3030;
		bSuccess = TestInterlocked<int16>(&Value2, 0x3030, 0x3030, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedXor(Dest, 0); }, FunctionName, TEXT("int16")) && bSuccess;
	}

	{
		// Test xor with value where some bits are set and some aren't set in the current value.
		volatile int32 Value0 = 0x30303030;
		bSuccess = TestInterlocked<int32>(&Value0, 0x30303030, 0x56565656, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedXor(Dest, 0x66666666); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test xor with all bits set.
		volatile int32 Value1 = 0x30303030;
		bSuccess = TestInterlocked<int32>(&Value1, 0x30303030, ~0x30303030, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedXor(Dest, -1); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test xor with zero.
		volatile int32 Value2 = 0x30303030;
		bSuccess = TestInterlocked<int32>(&Value2, 0x30303030, 0x30303030, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedXor(Dest, 0); }, FunctionName, TEXT("int32")) && bSuccess;
	}

	{
		// Test xor with value where some bits are set and some aren't set in the current value.
		alignas(8) volatile int64 Value0 = 0x3030303030303030LL;
		bSuccess = TestInterlocked<int64>(&Value0, 0x3030303030303030LL, 0x5656565656565656LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedXor(Dest, 0x6666666666666666LL); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test xor with all bits set.
		alignas(8) volatile int64 Value1 = 0x3030303030303030LL;
		bSuccess = TestInterlocked<int64>(&Value1, 0x3030303030303030LL, ~0x3030303030303030LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedXor(Dest, -1); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test xor with zero.
		alignas(8) volatile int64 Value2 = 0x3030303030303030LL;
		bSuccess = TestInterlocked<int64>(&Value2, 0x3030303030303030LL, 0x3030303030303030LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedXor(Dest, 0); }, FunctionName, TEXT("int64")) && bSuccess;
	}

	return bSuccess;
}

static bool TestInterlockedAdd()
{
	bool bSuccess = true;

	const TCHAR* FunctionName = TEXT("Add");
	{
		// Test adding positive value
		volatile int8 Value0 = 0x0F;
		bSuccess = TestInterlocked<int8>(&Value0, 0x0F, 0x11, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, +0x02); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test adding negative value
		volatile int8 Value1 = 0x11;
		bSuccess = TestInterlocked<int8>(&Value1, 0x11, 0x0F, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, -0x02); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test value overflow
		volatile int8 Value2 = MAX_int8 - 1;
		bSuccess = TestInterlocked<int8>(&Value2, MAX_int8 - 1, MIN_int8 + 2, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, +4); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test value underflow
		volatile int8 Value3 = MIN_int8 + 2;
		bSuccess = TestInterlocked<int8>(&Value3, MIN_int8 + 2, MAX_int8 - 1, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, -4); }, FunctionName, TEXT("int8")) && bSuccess;
	}

	{
		// Test adding positive value
		volatile int16 Value0 = 0x0F00;
		bSuccess = TestInterlocked<int16>(&Value0, 0x0F00, 0x1001, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, 0x0101); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test adding negative value
		volatile int16 Value1 = 0x1001;
		bSuccess = TestInterlocked<int16>(&Value1, 0x1001, 0x0F00, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, -0x0101); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test value overflow
		volatile int16 Value2 = MAX_int16 - 1;
		bSuccess = TestInterlocked<int16>(&Value2, MAX_int16 - 1, MIN_int16 + 2, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, +4); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test value underflow
		volatile int16 Value3 = MIN_int16 + 2;
		bSuccess = TestInterlocked<int16>(&Value3, MIN_int16 + 2, MAX_int16 - 1, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, -4); }, FunctionName, TEXT("int16")) && bSuccess;
	}

	{
		// Test adding positive value
		volatile int32 Value0 = 0x0F000000;
		bSuccess = TestInterlocked<int32>(&Value0, 0x0F000000, 0x10010101, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, 0x01010101); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test adding negative value
		volatile int32 Value1 = 0x10010101;
		bSuccess = TestInterlocked<int32>(&Value1, 0x10010101, 0x0F000000, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, -0x01010101); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test value overflow
		volatile int32 Value2 = MAX_int32 - 1;
		bSuccess = TestInterlocked<int32>(&Value2, MAX_int32 - 1, MIN_int32 + 2, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, +4); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test value underflow
		volatile int32 Value3 = MIN_int32 + 2;
		bSuccess = TestInterlocked<int32>(&Value3, MIN_int32 + 2, MAX_int32 - 1, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, -4); }, FunctionName, TEXT("int32")) && bSuccess;
	}

	{
		// Test adding positive value
		alignas(8) volatile int64 Value0 = 0x0F00000000000000LL;
		bSuccess = TestInterlocked<int64>(&Value0, 0x0F00000000000000LL, 0x1001010101010101LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, 0x0101010101010101LL); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test adding negative value
		alignas(8) volatile int64 Value1 = 0x1001010101010101LL;
		bSuccess = TestInterlocked<int64>(&Value1, 0x1001010101010101LL, 0x0F00000000000000LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, -0x0101010101010101LL); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test value overflow
		alignas(8) volatile int64 Value2 = MAX_int64 - 1LL;
		bSuccess = TestInterlocked<int64>(&Value2, MAX_int64 - 1, MIN_int64 + 2, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, +4); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test value underflow
		alignas(8) volatile int64 Value3 = MIN_int64 + 2LL;
		bSuccess = TestInterlocked<int64>(&Value3, MIN_int64 + 2, MAX_int64 - 1, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedAdd(Dest, -4); }, FunctionName, TEXT("int64")) && bSuccess;
	}

	return bSuccess;
}

static bool TestInterlockedIncrement()
{
	bool bSuccess = true;

	const TCHAR* FunctionName = TEXT("Increment");
	{
		volatile int8 Value0 = 0x0F;
		bSuccess = TestInterlocked<int8>(&Value0, 0x10, 0x10, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedIncrement(Dest); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test value overflow
		volatile int8 Value1 = MAX_int8;
		bSuccess = TestInterlocked<int8>(&Value1, MIN_int8, MIN_int8, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedIncrement(Dest); }, FunctionName, TEXT("int8")) && bSuccess;
	}

	{
		volatile int16 Value0 = 0x0F0F;
		bSuccess = TestInterlocked<int16>(&Value0, 0x0F10, 0x0F10, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedIncrement(Dest); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test value overflow
		volatile int16 Value1 = MAX_int16;
		bSuccess = TestInterlocked<int16>(&Value1, MIN_int16, MIN_int16, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedIncrement(Dest); }, FunctionName, TEXT("int16")) && bSuccess;
	}

	{
		volatile int32 Value0 = 0x0F00000F;
		bSuccess = TestInterlocked<int32>(&Value0, 0x0F000010, 0x0F000010, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedIncrement(Dest); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test value overflow
		volatile int32 Value1 = MAX_int32;
		bSuccess = TestInterlocked<int32>(&Value1, MIN_int32, MIN_int32, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedIncrement(Dest); }, FunctionName, TEXT("int32")) && bSuccess;
	}

	{
		alignas(8) volatile int64 Value0 = 0x0F0000000000000FLL;
		bSuccess = TestInterlocked<int64>(&Value0, 0x0F00000000000010LL, 0x0F00000000000010LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedIncrement(Dest); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test value overflow
		alignas(8) volatile int64 Value1 = MAX_int64;
		bSuccess = TestInterlocked<int64>(&Value1, MIN_int64, MIN_int64, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedIncrement(Dest); }, FunctionName, TEXT("int64")) && bSuccess;
	}

	return bSuccess;
}

static bool TestInterlockedDecrement()
{
	bool bSuccess = true;

	const TCHAR* FunctionName = TEXT("Decrement");
	{
		volatile int8 Value0 = 0x10;
		bSuccess = TestInterlocked<int8>(&Value0, 0x0F, 0x0F, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedDecrement(Dest); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test value underflow
		volatile int8 Value1 = MIN_int8;
		bSuccess = TestInterlocked<int8>(&Value1, MAX_int8, MAX_int8, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedDecrement(Dest); }, FunctionName, TEXT("int8")) && bSuccess;
	}

	{
		volatile int16 Value0 = 0x0F10;
		bSuccess = TestInterlocked<int16>(&Value0, 0x0F0F, 0x0F0F, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedDecrement(Dest); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test value underflow
		volatile int16 Value1 = MIN_int16;
		bSuccess = TestInterlocked<int16>(&Value1, MAX_int16, MAX_int16, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedDecrement(Dest); }, FunctionName, TEXT("int16")) && bSuccess;
	}

	{
		volatile int32 Value0 = 0x0F000010;
		bSuccess = TestInterlocked<int32>(&Value0, 0x0F00000F, 0x0F00000F, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedDecrement(Dest); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test value underflow
		volatile int32 Value1 = MIN_int32;
		bSuccess = TestInterlocked<int32>(&Value1, MAX_int32, MAX_int32, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedDecrement(Dest); }, FunctionName, TEXT("int32")) && bSuccess;
	}

	{
		alignas(8) volatile int64 Value0 = 0x0F00000000000010LL;
		bSuccess = TestInterlocked<int64>(&Value0, 0x0F0000000000000FLL, 0x0F0000000000000FLL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedDecrement(Dest); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test value underflow
		alignas(8) volatile int64 Value1 = MIN_int64;
		bSuccess = TestInterlocked<int64>(&Value1, MAX_int64, MAX_int64, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedDecrement(Dest); }, FunctionName, TEXT("int64")) && bSuccess;
	}

	return bSuccess;
}

static bool TestInterlockedExchange()
{
	bool bSuccess = true;

	const TCHAR* FunctionName = TEXT("Exchange");
	{
		volatile int8 Value = 0x10;
		bSuccess = TestInterlocked<int8>(&Value, 0x10, 0x01, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedExchange(Dest, 0x01); }, FunctionName, TEXT("int8")) && bSuccess;
	}

	{
		volatile int16 Value = 0x1000;
		bSuccess = TestInterlocked<int16>(&Value, 0x1000, 0x0001, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedExchange(Dest, 0x0001); }, FunctionName, TEXT("int16")) && bSuccess;
	}

	{
		volatile int32 Value = 0x10000000;
		bSuccess = TestInterlocked<int32>(&Value, 0x10000000, 0x00000101, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedExchange(Dest, 0x00000101); }, FunctionName, TEXT("int32")) && bSuccess;
	}

	{
		alignas(8) volatile int64 Value = 0x1000000000000000LL;
		bSuccess = TestInterlocked<int64>(&Value, 0x1000000000000000LL, 0x0000000001010101LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedExchange(Dest, 0x0000000001010101LL); }, FunctionName, TEXT("int64")) && bSuccess;
	}

	return bSuccess;
}

static bool TestInterlockedExchangePtr()
{
	int Test = 0;
	void* volatile Value = &Test;
	const void* ReturnValue = FPlatformAtomics::InterlockedExchangePtr(&Value, nullptr);
	if (ReturnValue != &Test)
	{
		FAIL_CHECK(TEXT("FPlatformAtomics::InterlockedExchangePtr failed"));
		return false;
	}
	if (Value != nullptr)
	{
		FAIL_CHECK(TEXT("FPlatformAtomics::InterlockedExchangePtr failed"));
		return false;
	}
	return true;
}

static bool TestInterlockedCompareExchange()
{
	bool bSuccess = true;

	const TCHAR* FunctionName = TEXT("CompareExchange");
	{
		volatile int8 Value = 0x10;

		// Test value isn't changed when comparand differs
		bSuccess = TestInterlocked<int8>(&Value, 0x10, 0x10, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedCompareExchange(Dest, 0x10, 0x01); }, FunctionName, TEXT("int8")) && bSuccess;

		// Test value is changed when comparand matches
		bSuccess = TestInterlocked<int8>(&Value, 0x10, 0x01, [](volatile int8* Dest) { return FPlatformAtomics::InterlockedCompareExchange(Dest, 0x01, 0x10); }, FunctionName, TEXT("int8")) && bSuccess;
	}

	{
		volatile int16 Value = 0x1000;

		// Test value isn't changed when comparand differs
		bSuccess = TestInterlocked<int16>(&Value, 0x1000, 0x1000, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedCompareExchange(Dest, 0x1000, 0x0001); }, FunctionName, TEXT("int16")) && bSuccess;

		// Test value is changed when comparand matches
		bSuccess = TestInterlocked<int16>(&Value, 0x1000, 0x0001, [](volatile int16* Dest) { return FPlatformAtomics::InterlockedCompareExchange(Dest, 0x0001, 0x1000); }, FunctionName, TEXT("int16")) && bSuccess;
	}

	{
		volatile int32 Value = 0x10000000;

		// Test value isn't changed when comparand differs
		bSuccess = TestInterlocked<int32>(&Value, 0x10000000, 0x10000000, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedCompareExchange(Dest, 0x10000000, 0x00000101); }, FunctionName, TEXT("int32")) && bSuccess;

		// Test value is changed when comparand matches
		bSuccess = TestInterlocked<int32>(&Value, 0x10000000, 0x00000101, [](volatile int32* Dest) { return FPlatformAtomics::InterlockedCompareExchange(Dest, 0x00000101, 0x10000000); }, FunctionName, TEXT("int32")) && bSuccess;
	}

	{
		alignas(8) volatile int64 Value = 0x1000000000000000LL;

		// Test value isn't changed when comparand differs
		bSuccess = TestInterlocked<int64>(&Value, 0x1000000000000000LL, 0x1000000000000000LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedCompareExchange(Dest, 0x1000000000000000LL, 0x0000000001010101LL); }, FunctionName, TEXT("int64")) && bSuccess;

		// Test value is changed when comparand matches
		bSuccess = TestInterlocked<int64>(&Value, 0x1000000000000000LL, 0x0000000001010101LL, [](volatile int64* Dest) { return FPlatformAtomics::InterlockedCompareExchange(Dest, 0x0000000001010101LL, 0x1000000000000000LL); }, FunctionName, TEXT("int64")) && bSuccess;
	}

	return bSuccess;
}

TEST_CASE_NAMED(FPlatformAtomicsTest, "System::Core::HAL::PlatformAtomics", "[ApplicationContextMask][SmokeFilter]")
{
	bool bSuccess = true;
	bSuccess = TestInterlockedAnd() && bSuccess;
	bSuccess = TestInterlockedOr() && bSuccess;
	bSuccess = TestInterlockedXor() && bSuccess;
	bSuccess = TestInterlockedAdd() && bSuccess;
	bSuccess = TestInterlockedIncrement() && bSuccess;
	bSuccess = TestInterlockedDecrement() && bSuccess;
	bSuccess = TestInterlockedExchange() && bSuccess;
	bSuccess = TestInterlockedExchangePtr() && bSuccess;
	bSuccess = TestInterlockedCompareExchange() && bSuccess;
	REQUIRE(bSuccess);
}

#endif //WITH_TESTS
