// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "HAL/PlatformManualResetEvent.h"

#include "HAL/PlatformProcess.h"
#include "Tasks/Task.h"
#include "TestHarness.h"

namespace UE
{

TEST_CASE("Core::Async::PlatformManualResetEvent", "[Core][Async]")
{
	FPlatformManualResetEvent Event;

	struct FTimer
	{
		FMonotonicTimePoint Now = FMonotonicTimePoint::Now();

		FMonotonicTimeSpan Step()
		{
			FMonotonicTimePoint Prev = Now;
			Now = FMonotonicTimePoint::Now();
			return Now - Prev;
		}
	};

	FTimer Timer;
	CHECK_FALSE(Event.Poll());
	CHECK_FALSE(Event.WaitFor(FMonotonicTimeSpan::Zero()));
	Timer.Step();
	CHECK_FALSE(Event.WaitFor(FMonotonicTimeSpan::FromMilliseconds(32.0)));
	CHECK(Timer.Step().ToMilliseconds() >= 16.0);
	CHECK_FALSE(Event.WaitFor(FMonotonicTimeSpan::FromMilliseconds(-32.0)));
	Timer.Step();
	CHECK_FALSE(Event.WaitUntil(Timer.Now - FMonotonicTimeSpan::FromMilliseconds(32.0)));
	Timer.Step();
	CHECK_FALSE(Event.WaitUntil(Timer.Now + FMonotonicTimeSpan::FromMilliseconds(32.0)));
	CHECK(Timer.Step().ToMilliseconds() >= 16.0);

	Event.Notify();
	CHECK(Event.Poll());
	Event.Wait();
	CHECK(Event.WaitFor(FMonotonicTimeSpan::Zero()));
	CHECK(Event.WaitFor(FMonotonicTimeSpan::FromSeconds(60.0)));
	Timer.Step();
	CHECK(Event.WaitUntil(Timer.Now - FMonotonicTimeSpan::FromSeconds(60.0)));
	Timer.Step();
	CHECK(Event.WaitUntil(Timer.Now + FMonotonicTimeSpan::FromSeconds(60.0)));

	Event.Reset();
	CHECK_FALSE(Event.Poll());
	Timer.Step();
	CHECK_FALSE(Event.WaitFor(FMonotonicTimeSpan::FromMilliseconds(32.0)));
	CHECK(Timer.Step().ToMilliseconds() >= 16.0);
	CHECK_FALSE(Event.WaitUntil(Timer.Now + FMonotonicTimeSpan::FromMilliseconds(32.0)));
	CHECK(Timer.Step().ToMilliseconds() >= 16.0);

	CHECK_FALSE(Event.WaitFor(-FMonotonicTimeSpan::Infinity()));

	// Wait
	Tasks::Launch(UE_SOURCE_LOCATION, [&Event]
	{
		FPlatformProcess::SleepNoStats(0.02f);
		Event.Notify();
	});
	Event.Wait();
	CHECK(Timer.Step().ToMilliseconds() >= 16.0);

	// WaitFor(Infinity)
	Event.Reset();
	Tasks::Launch(UE_SOURCE_LOCATION, [&Event]
	{
		FPlatformProcess::SleepNoStats(0.02f);
		Event.Notify();
	});
	CHECK(Event.WaitFor(FMonotonicTimeSpan::Infinity()));
	CHECK(Timer.Step().ToMilliseconds() >= 16.0);

	// WaitUntil(Infinity)
	Event.Reset();
	Tasks::Launch(UE_SOURCE_LOCATION, [&Event]
	{
		FPlatformProcess::SleepNoStats(0.02f);
		Event.Notify();
	});
	CHECK(Event.WaitUntil(FMonotonicTimePoint::Infinity()));
	CHECK(Timer.Step().ToMilliseconds() >= 16.0);

	Event.Notify();
	Event.Reset();
}

} // UE

#endif // WITH_LOW_LEVEL_TESTS
