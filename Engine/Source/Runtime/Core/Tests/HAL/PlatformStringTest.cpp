// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformString.h"

#include "Misc/CString.h"
#include "Templates/UnrealTemplate.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_TESTS

#include "Misc/CString.h"
#include "Templates/UnrealTemplate.h"

template <typename CharType, SIZE_T Size>
static void InvokePlatformStringGetVarArgs(CharType (&Dest)[Size], const CharType* Fmt, ...)
{
	va_list ap;
	va_start(ap, Fmt);
	FPlatformString::GetVarArgs(Dest, Size, Fmt, ap);
	va_end(ap);
}

TEST_CASE_NAMED(FPlatformStringTestGetVarArgs, "System::Core::HAL::PlatformString::GetVarArgs", "[ApplicationContextMask][EngineFilter]")
{
	TCHAR Buffer[128];
	InvokePlatformStringGetVarArgs(Buffer, TEXT("A%.*sZ"), 4, TEXT(" to B"));
	CHECK_MESSAGE(TEXT("GetVarArgs(%.*s)"), FCString::Strcmp(Buffer, TEXT("A to Z")) == 0);
}

TEST_CASE_NAMED(FPlatformStringTestStrnlen, "System::Core::HAL::PlatformString::Strnlen", "[ApplicationContextMask][EngineFilter]")
{
	CHECK_MESSAGE(TEXT("Strnlen(nullptr, 0)"), FPlatformString::Strnlen((const ANSICHAR*)nullptr, 0) == 0); //-V575
	CHECK_MESSAGE(TEXT("Strnlen(\"\", 0)"), FPlatformString::Strnlen("", 0) == 0);  //-V575
	CHECK_MESSAGE(TEXT("Strnlen(\"1\", 0)"), FPlatformString::Strnlen("1", 0) == 0);  //-V575
	CHECK_MESSAGE(TEXT("Strnlen(\"1\", 1)"), FPlatformString::Strnlen("1", 1) == 1);
	CHECK_MESSAGE(TEXT("Strnlen(\"1\", 2)"), FPlatformString::Strnlen("1", 2) == 1);
	CHECK_MESSAGE(TEXT("Strnlen(\"123\", 2)"), FPlatformString::Strnlen("123", 2) == 2);
	ANSICHAR AnsiBuffer[128] = "123456789";
	CHECK_MESSAGE(TEXT("Strnlen(PaddedBuffer)"), FPlatformString::Strnlen(AnsiBuffer, UE_ARRAY_COUNT(AnsiBuffer)) == 9);

	CHECK_MESSAGE(TEXT("Strnlen(nullptr, 0)"), FPlatformString::Strnlen((const TCHAR*)nullptr, 0) == 0); //-V575
	CHECK_MESSAGE(TEXT("Strnlen(\"\", 0)"), FPlatformString::Strnlen(TEXT(""), 0) == 0); //-V575
	CHECK_MESSAGE(TEXT("Strnlen(\"1\", 0)"), FPlatformString::Strnlen(TEXT("1"), 0) == 0); //-V575
	CHECK_MESSAGE(TEXT("Strnlen(\"1\", 1)"), FPlatformString::Strnlen(TEXT("1"), 1) == 1);
	CHECK_MESSAGE(TEXT("Strnlen(\"1\", 2)"), FPlatformString::Strnlen(TEXT("1"), 2) == 1);
	CHECK_MESSAGE(TEXT("Strnlen(\"123\", 2)"), FPlatformString::Strnlen(TEXT("123"), 2) == 2);
	TCHAR Buffer[128] = {};
	FCString::Strcpy(Buffer, TEXT("123456789"));
	CHECK_MESSAGE(TEXT("Strnlen(PaddedBuffer)"), FPlatformString::Strnlen(Buffer, UE_ARRAY_COUNT(Buffer)) == 9);
}

TEST_CASE_NAMED(FPlatformStringTestStrcpy, "System::Core::HAL::PlatformString::Strcpy", "[ApplicationContextMask][EngineFilter]")
{
	constexpr int32 BufferLen = 32;
	WIDECHAR WideBuffer[BufferLen];
	UTF8CHAR Utf8Buffer[BufferLen];
	ANSICHAR AnsiBuffer[BufferLen];

	const WIDECHAR* WideTest = WIDETEXT("12345");
	const UTF8CHAR* Utf8Test = UTF8TEXT("12345");
	const ANSICHAR* AnsiTest = "12345";
	constexpr int32 TestLen = 5;

	auto Reset = [&WideBuffer, &Utf8Buffer, &AnsiBuffer, BufferLen]()
		{
			for (int32 n = 0; n < BufferLen; ++n)
			{
				WideBuffer[n] = '%';
				Utf8Buffer[n] = UTF8CHAR('%');
				AnsiBuffer[n] = '%';
			}
		};

	Reset();
	FPlatformString::Strcpy(WideBuffer, WideTest);
	FPlatformString::Strcpy(Utf8Buffer, Utf8Test);
	FPlatformString::Strcpy(AnsiBuffer, AnsiTest);
	CHECK_MESSAGE(TEXT("WideStrcpy"),
		WideBuffer[TestLen] == 0 && FPlatformString::Strcmp(WideTest, WideBuffer) == 0 && WideBuffer[TestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8Strcpy"),
		Utf8Buffer[TestLen] == 0 && FPlatformString::Strcmp(Utf8Test, Utf8Buffer) == 0 && Utf8Buffer[TestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrcpy"),
		AnsiBuffer[TestLen] == 0 && FPlatformString::Strcmp(AnsiTest, AnsiBuffer) == 0 && AnsiBuffer[TestLen + 1] == '%');

	Reset();
	FPlatformString::Strncpy(WideBuffer, WideTest, TestLen + 10);
	FPlatformString::Strncpy(Utf8Buffer, Utf8Test, TestLen + 10);
	FPlatformString::Strncpy(AnsiBuffer, AnsiTest, TestLen + 10);
	CHECK_MESSAGE(TEXT("WideStrncpyTestLenPlus10"),
		WideBuffer[TestLen] == 0 && FPlatformString::Strcmp(WideTest, WideBuffer) == 0 && WideBuffer[TestLen + 10] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncpyTestLenPlus10"),
		Utf8Buffer[TestLen] == 0 && FPlatformString::Strcmp(Utf8Test, Utf8Buffer) == 0 && Utf8Buffer[TestLen + 10] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncpyTestLenPlus10"),
		AnsiBuffer[TestLen] == 0 && FPlatformString::Strcmp(AnsiTest, AnsiBuffer) == 0 && AnsiBuffer[TestLen + 10] == '%');

	Reset();
	FPlatformString::Strncpy(WideBuffer, WideTest, TestLen + 1);
	FPlatformString::Strncpy(Utf8Buffer, Utf8Test, TestLen + 1);
	FPlatformString::Strncpy(AnsiBuffer, AnsiTest, TestLen + 1);
	CHECK_MESSAGE(TEXT("WideStrncpyTestLenPlus1"),
		WideBuffer[TestLen] == 0 && FPlatformString::Strcmp(WideTest, WideBuffer) == 0 && WideBuffer[TestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncpyTestLenPlus1"),
		Utf8Buffer[TestLen] == 0 && FPlatformString::Strcmp(Utf8Test, Utf8Buffer) == 0 && Utf8Buffer[TestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncpyTestLenPlus1"),
		AnsiBuffer[TestLen] == 0 && FPlatformString::Strcmp(AnsiTest, AnsiBuffer) == 0 && AnsiBuffer[TestLen + 1] == '%');

	Reset();
	FPlatformString::Strncpy(WideBuffer, WideTest, TestLen);
	FPlatformString::Strncpy(Utf8Buffer, Utf8Test, TestLen);
	FPlatformString::Strncpy(AnsiBuffer, AnsiTest, TestLen);
	CHECK_MESSAGE(TEXT("WideStrncpyTestLen"),
		WideBuffer[TestLen - 1] == 0 && WideBuffer[TestLen] == '%' && FPlatformString::Strncmp(WideTest, WideBuffer, TestLen - 1) == 0 && WideBuffer[TestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncpyTestLen"),
		Utf8Buffer[TestLen - 1] == 0 && Utf8Buffer[TestLen] == '%' && FPlatformString::Strncmp(Utf8Test, Utf8Buffer, TestLen - 1) == 0 && Utf8Buffer[TestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncpyTestLen"),
		AnsiBuffer[TestLen - 1] == 0 && AnsiBuffer[TestLen] == '%' && FPlatformString::Strncmp(AnsiTest, AnsiBuffer, TestLen - 1) == 0 && AnsiBuffer[TestLen + 1] == '%');

	Reset();
	FPlatformString::Strncpy(WideBuffer, WideTest, TestLen - 1);
	FPlatformString::Strncpy(Utf8Buffer, Utf8Test, TestLen - 1);
	FPlatformString::Strncpy(AnsiBuffer, AnsiTest, TestLen - 1);
	CHECK_MESSAGE(TEXT("WideStrncpyTestLenMinus1"),
		WideBuffer[TestLen - 2] == 0 && WideBuffer[TestLen - 1] == '%' && FPlatformString::Strncmp(WideTest, WideBuffer, TestLen - 2) == 0 && WideBuffer[TestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncpyTestLenMinus1"),
		Utf8Buffer[TestLen - 2] == 0 && Utf8Buffer[TestLen - 1] == '%' && FPlatformString::Strncmp(Utf8Test, Utf8Buffer, TestLen - 2) == 0 && Utf8Buffer[TestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncpyTestLenMinus1"),
		AnsiBuffer[TestLen - 2] == 0 && AnsiBuffer[TestLen - 1] == '%' && FPlatformString::Strncmp(AnsiTest, AnsiBuffer, TestLen - 2) == 0 && AnsiBuffer[TestLen + 1] == '%');

	Reset();
	FPlatformString::Strncpy(WideBuffer, WideTest, 2);
	FPlatformString::Strncpy(Utf8Buffer, Utf8Test, 2);
	FPlatformString::Strncpy(AnsiBuffer, AnsiTest, 2);
	CHECK_MESSAGE(TEXT("WideStrncpyTwoLen"),
		WideBuffer[0] == WideTest[0] && WideBuffer[1] == 0 && WideBuffer[2] == '%' && WideBuffer[TestLen] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncpyTwoLen"),
		Utf8Buffer[0] == Utf8Test[0] && Utf8Buffer[1] == 0 && Utf8Buffer[2] == '%' && Utf8Buffer[TestLen] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncpyTwoLen"),
		AnsiBuffer[0] == AnsiTest[0] && AnsiBuffer[1] == 0 && AnsiBuffer[2] == '%' && AnsiBuffer[TestLen] == '%');

	Reset();
	FPlatformString::Strncpy(WideBuffer, WideTest, 1);
	FPlatformString::Strncpy(Utf8Buffer, Utf8Test, 1);
	FPlatformString::Strncpy(AnsiBuffer, AnsiTest, 1);
	CHECK_MESSAGE(TEXT("WideStrncpyOneLen"),
		WideBuffer[0] == 0 && WideBuffer[1] == '%' && WideBuffer[TestLen] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncpyOneLen"),
		Utf8Buffer[0] == 0 && Utf8Buffer[1] == '%' && Utf8Buffer[TestLen] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncpyOneLen"),
		AnsiBuffer[0] == 0 && AnsiBuffer[1] == '%' && AnsiBuffer[TestLen] == '%');

	// ZeroLen Strncpy is undefined
}

TEST_CASE_NAMED(FPlatformStringTestStrcat, "System::Core::HAL::PlatformString::Strcat", "[ApplicationContextMask][EngineFilter]")
{
	constexpr int32 BufferLen = 32;
	WIDECHAR WideBuffer[BufferLen];
	UTF8CHAR Utf8Buffer[BufferLen];
	ANSICHAR AnsiBuffer[BufferLen];

	const WIDECHAR* WidePrefix = WIDETEXT("ABCD");
	const UTF8CHAR* Utf8Prefix = UTF8TEXT("ABCD");
	const ANSICHAR* AnsiPrefix = "ABCD";
	const WIDECHAR* WideTest = WIDETEXT("12345");
	const UTF8CHAR* Utf8Test = UTF8TEXT("12345");
	const ANSICHAR* AnsiTest = "12345";
	const WIDECHAR* WidePrefixPlusTest = WIDETEXT("ABCD12345");
	const UTF8CHAR* Utf8PrefixPlusTest = UTF8TEXT("ABCD12345");
	const ANSICHAR* AnsiPrefixPlusTest = "ABCD12345";
	constexpr int32 PrefixLen = 4;
	constexpr int32 TestLen = 5;
	constexpr int32 PrefixPlusTestLen = 9;

	auto Reset = [&WideBuffer, &Utf8Buffer, &AnsiBuffer, WidePrefix, Utf8Prefix, AnsiPrefix, BufferLen]()
		{
			int32 n = 0;
			for (; n < PrefixLen + 1; ++n)
			{
				WideBuffer[n] = WidePrefix[n];
				Utf8Buffer[n] = Utf8Prefix[n];
				AnsiBuffer[n] = AnsiPrefix[n];
			}
			for (; n < BufferLen; ++n)
			{
				WideBuffer[n] = '%';
				Utf8Buffer[n] = UTF8CHAR('%');
				AnsiBuffer[n] = '%';
			}
		};

	Reset();
	FPlatformString::Strcat(WideBuffer, WideTest);
	FPlatformString::Strcat(Utf8Buffer, Utf8Test);
	FPlatformString::Strcat(AnsiBuffer, AnsiTest);
	CHECK_MESSAGE(TEXT("WideStrcat"),
		WideBuffer[PrefixPlusTestLen] == 0 && FPlatformString::Strcmp(WidePrefixPlusTest, WideBuffer) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8Strcat"),
		Utf8Buffer[PrefixPlusTestLen] == 0 && FPlatformString::Strcmp(Utf8PrefixPlusTest, Utf8Buffer) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrcat"),
		AnsiBuffer[PrefixPlusTestLen] == 0 && FPlatformString::Strcmp(AnsiPrefixPlusTest, AnsiBuffer) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FPlatformString::Strncat(WideBuffer, WideTest, TestLen + 10);
	FPlatformString::Strncat(Utf8Buffer, Utf8Test, TestLen + 10);
	FPlatformString::Strncat(AnsiBuffer, AnsiTest, TestLen + 10);
	CHECK_MESSAGE(TEXT("WideStrncatTestLenPlus10"),
		WideBuffer[PrefixPlusTestLen] == 0 && FPlatformString::Strcmp(WidePrefixPlusTest, WideBuffer) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncatTestLenPlus10"),
		Utf8Buffer[PrefixPlusTestLen] == 0 && FPlatformString::Strcmp(Utf8PrefixPlusTest, Utf8Buffer) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncatTestLenPlus10"),
		AnsiBuffer[PrefixPlusTestLen] == 0 && FPlatformString::Strcmp(AnsiPrefixPlusTest, AnsiBuffer) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FPlatformString::Strncat(WideBuffer, WideTest, TestLen);
	FPlatformString::Strncat(Utf8Buffer, Utf8Test, TestLen);
	FPlatformString::Strncat(AnsiBuffer, AnsiTest, TestLen);
	CHECK_MESSAGE(TEXT("WideStrncatTestLenPlus1"),
		WideBuffer[PrefixPlusTestLen] == 0 && FPlatformString::Strcmp(WidePrefixPlusTest, WideBuffer) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncatTestLenPlus1"),
		Utf8Buffer[PrefixPlusTestLen] == 0 && FPlatformString::Strcmp(Utf8PrefixPlusTest, Utf8Buffer) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncatTestLenPlus1"),
		AnsiBuffer[PrefixPlusTestLen] == 0 && FPlatformString::Strcmp(AnsiPrefixPlusTest, AnsiBuffer) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FPlatformString::Strncat(WideBuffer, WideTest, TestLen - 1);
	FPlatformString::Strncat(Utf8Buffer, Utf8Test, TestLen - 1);
	FPlatformString::Strncat(AnsiBuffer, AnsiTest, TestLen - 1);
	CHECK_MESSAGE(TEXT("WideStrncatTestLenMinus1"),
		WideBuffer[PrefixPlusTestLen - 1] == 0 && WideBuffer[PrefixPlusTestLen] == '%' && FPlatformString::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixPlusTestLen - 1) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncatTestLenMinus1"),
		Utf8Buffer[PrefixPlusTestLen - 1] == 0 && Utf8Buffer[PrefixPlusTestLen] == '%' && FPlatformString::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixPlusTestLen - 1) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncatTestLenMinus1"),
		AnsiBuffer[PrefixPlusTestLen - 1] == 0 && AnsiBuffer[PrefixPlusTestLen] == '%' && FPlatformString::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixPlusTestLen - 1) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FPlatformString::Strncat(WideBuffer, WideTest, TestLen - 2);
	FPlatformString::Strncat(Utf8Buffer, Utf8Test, TestLen - 2);
	FPlatformString::Strncat(AnsiBuffer, AnsiTest, TestLen - 2);
	CHECK_MESSAGE(TEXT("WideStrncatTestLenMinus2"),
		WideBuffer[PrefixPlusTestLen - 2] == 0 && WideBuffer[PrefixPlusTestLen - 1] == '%' && FPlatformString::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixPlusTestLen - 2) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncatTestLenMinus2"),
		Utf8Buffer[PrefixPlusTestLen - 2] == 0 && Utf8Buffer[PrefixPlusTestLen - 1] == '%' && FPlatformString::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixPlusTestLen - 2) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncatTestLenMinus2"),
		AnsiBuffer[PrefixPlusTestLen - 2] == 0 && AnsiBuffer[PrefixPlusTestLen - 1] == '%' && FPlatformString::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixPlusTestLen - 2) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FPlatformString::Strncat(WideBuffer, WideTest, 1);
	FPlatformString::Strncat(Utf8Buffer, Utf8Test, 1);
	FPlatformString::Strncat(AnsiBuffer, AnsiTest, 1);
	CHECK_MESSAGE(TEXT("WideStrncatOneLen"),
		WideBuffer[PrefixLen + 1] == 0 && WideBuffer[PrefixLen + 2] == '%' && FPlatformString::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixLen + 1) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncatOneLen"),
		Utf8Buffer[PrefixLen + 1] == 0 && Utf8Buffer[PrefixLen + 2] == '%' && FPlatformString::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixLen + 1) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncatOneLen"),
		AnsiBuffer[PrefixLen + 1] == 0 && AnsiBuffer[PrefixLen + 2] == '%' && FPlatformString::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixLen + 1) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');

	Reset();
	FPlatformString::Strncat(WideBuffer, WideTest, 0);
	FPlatformString::Strncat(Utf8Buffer, Utf8Test, 0);
	FPlatformString::Strncat(AnsiBuffer, AnsiTest, 0);
	CHECK_MESSAGE(TEXT("WideStrncatZeroLen"),
		WideBuffer[PrefixLen] == 0 && WideBuffer[PrefixLen + 1] == '%' && FPlatformString::Strncmp(WidePrefixPlusTest, WideBuffer, PrefixLen) == 0 && WideBuffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("Utf8StrncatZeroLen"),
		Utf8Buffer[PrefixLen] == 0 && Utf8Buffer[PrefixLen + 1] == '%' && FPlatformString::Strncmp(Utf8PrefixPlusTest, Utf8Buffer, PrefixLen) == 0 && Utf8Buffer[PrefixPlusTestLen + 1] == '%');
	CHECK_MESSAGE(TEXT("AnsiStrncatZeroLen"),
		AnsiBuffer[PrefixLen] == 0 && AnsiBuffer[PrefixLen + 1] == '%' && FPlatformString::Strncmp(AnsiPrefixPlusTest, AnsiBuffer, PrefixLen) == 0 && AnsiBuffer[PrefixPlusTestLen + 1] == '%');
}
#endif //WITH_TESTS
