// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "HAL/PlatformString.h"
#include "Containers/Array.h"
#include "Templates/UnrealTemplate.h"

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FHALPlatformStringTest, "System::Core::HAL::PlatformString", "[Core]")
{
	SECTION("Strupr TCHAR null-termination")
	{
		// Whole TCHAR string
		const TCHAR TestLiteral[] = TEXT("the quick brown fox jumps over the lazy dog");

		TArray<TCHAR> TestFullString(TestLiteral, UE_ARRAY_COUNT(TestLiteral));

		FPlatformString::Strupr(TestFullString.GetData(), TestFullString.Num());
		REQUIRE(FPlatformString::Strncmp(TestFullString.GetData(), TEXT("THE QUICK BROWN FOX JUMPS OVER THE LAZY DOG"), TestFullString.Num()) == 0);

		// Partial TCHAR string
		TArray<TCHAR> TestPartialString(TestLiteral, UE_ARRAY_COUNT(TestLiteral));

		FPlatformString::Strupr(TestPartialString.GetData(), 19);
		REQUIRE(FPlatformString::Strncmp(TestPartialString.GetData(), TEXT("THE QUICK BROWN FOX jumps over the lazy dog"), TestPartialString.Num()) == 0);
	}

	SECTION("Strupr ANSICHAR null-termination")
	{
		// Whole ANSICHAR string
		const ANSICHAR TestLiteral[] = "the quick brown fox jumps over the lazy dog";

		TArray<ANSICHAR> TestFullString(TestLiteral, UE_ARRAY_COUNT(TestLiteral));

		FPlatformString::Strupr(TestFullString.GetData(), TestFullString.Num());
		REQUIRE(FPlatformString::Strncmp(TestFullString.GetData(), "THE QUICK BROWN FOX JUMPS OVER THE LAZY DOG", TestFullString.Num()) == 0);

		// Partial ANSICHAR string
		TArray<ANSICHAR> TestPartialString(TestLiteral, UE_ARRAY_COUNT(TestLiteral));

		FPlatformString::Strupr(TestPartialString.GetData(), 19);
		REQUIRE(FPlatformString::Strncmp(TestPartialString.GetData(), "THE QUICK BROWN FOX jumps over the lazy dog", TestPartialString.Num()) == 0);
	}

	SECTION("Strupr WIDECHAR null-termination")
	{
		// Whole WIDECHAR string
		const WIDECHAR TestLiteral[] = WIDETEXT("the quick brown fox jumps over the lazy dog");

		TArray<WIDECHAR> TestFullString(TestLiteral, UE_ARRAY_COUNT(TestLiteral));

		FPlatformString::Strupr(TestFullString.GetData(), TestFullString.Num());
		REQUIRE(FPlatformString::Strncmp(TestFullString.GetData(), WIDETEXT("THE QUICK BROWN FOX JUMPS OVER THE LAZY DOG"), TestFullString.Num()) == 0);

		// Partial WIDECHAR string
		TArray<WIDECHAR> TestPartialString(TestLiteral, UE_ARRAY_COUNT(TestLiteral));

		FPlatformString::Strupr(TestPartialString.GetData(), 19);
		REQUIRE(FPlatformString::Strncmp(TestPartialString.GetData(), WIDETEXT("THE QUICK BROWN FOX jumps over the lazy dog"), TestPartialString.Num()) == 0);
	}

	SECTION("Strupr UTF8CHAR null-termination")
	{
		// Whole UTF8CHAR string
		const UTF8CHAR* TestLiteral = UTF8TEXT("the quick brown fox jumps over the lazy dog");

		TArray<UTF8CHAR> TestFullString(TestLiteral, FPlatformString::Strlen(TestLiteral) + 1);

		FPlatformString::Strupr(TestFullString.GetData(), TestFullString.Num());
		REQUIRE(FPlatformString::Strncmp(TestFullString.GetData(), UTF8TEXT("THE QUICK BROWN FOX JUMPS OVER THE LAZY DOG"), TestFullString.Num()) == 0);

		// Partial UTF8CHAR string
		TArray<UTF8CHAR> TestPartialString(TestLiteral, FPlatformString::Strlen(TestLiteral) + 1);

		FPlatformString::Strupr(TestPartialString.GetData(), 19);
		REQUIRE(FPlatformString::Strncmp(TestPartialString.GetData(), UTF8TEXT("THE QUICK BROWN FOX jumps over the lazy dog"), TestPartialString.Num()) == 0);
	}
}

#endif //WITH_TESTS