// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PreprocessorHelpers.h"

#include "Misc/CString.h"
#include "Templates/UnrealTemplate.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_TESTS

TEST_CASE_NAMED(FPreprocessorHelperArgCountTest, "System::Core::HAL::UE_VA_ARG_COUNT", "[ApplicationContextMask][EngineFilter]")
{
	STATIC_CHECK(UE_VA_ARG_COUNT() == 0);
	STATIC_CHECK(UE_VA_ARG_COUNT(a) == 1);
	STATIC_CHECK(UE_VA_ARG_COUNT(a, b) == 2);
	STATIC_CHECK(UE_VA_ARG_COUNT(a, b, c) == 3);
	STATIC_CHECK(UE_VA_ARG_COUNT(a, b, c, d) == 4);
	STATIC_CHECK(UE_VA_ARG_COUNT(a, b, c, d, e) == 5);
	STATIC_CHECK(UE_VA_ARG_COUNT(a, b, c, d, e, f) == 6);
	STATIC_CHECK(UE_VA_ARG_COUNT(a, b, c, d, e, f, g) == 7);
	STATIC_CHECK(UE_VA_ARG_COUNT(a, b, c, d, e, f, g, h) == 8);
	STATIC_CHECK(UE_VA_ARG_COUNT(a, b, c, d, e, f, g, h, i) == 9);
	STATIC_CHECK(UE_VA_ARG_COUNT(a, b, c, d, e, f, g, h, i, j) == 10);
}

TEST_CASE_NAMED(FPreprocessorHelperJoinTest, "System::Core::HAL::UE_APPEND_VA_ARG_COUNT", "[ApplicationContextMask][EngineFilter]")
{
	#define TESTPREFIX_0 22
	#define TESTPREFIX_1 32
	#define TESTPREFIX_2 41
	#define TESTPREFIX_3 52
	#define TESTPREFIX_4 61
	#define TESTPREFIX_5 70
	#define TESTPREFIX_6 77
	#define TESTPREFIX_7 82
	#define TESTPREFIX_8 89
	#define TESTPREFIX_9 90
	#define TESTPREFIX_10 96

	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_) == TESTPREFIX_0);
	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_, a) == TESTPREFIX_1);
	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_, a, b) == TESTPREFIX_2);
	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_, a, b, c) == TESTPREFIX_3);
	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_, a, b, c, d) == TESTPREFIX_4);
	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_, a, b, c, d, e) == TESTPREFIX_5);
	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_, a, b, c, d, e, f) == TESTPREFIX_6);
	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_, a, b, c, d, e, f, g) == TESTPREFIX_7);
	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_, a, b, c, d, e, f, g, h) == TESTPREFIX_8);
	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_, a, b, c, d, e, f, g, h, i) == TESTPREFIX_9);
	STATIC_CHECK(UE_APPEND_VA_ARG_COUNT(TESTPREFIX_, a, b, c, d, e, f, g, h, i, j) == TESTPREFIX_10);

	#undef TESTPREFIX_10
	#undef TESTPREFIX_9
	#undef TESTPREFIX_8
	#undef TESTPREFIX_7
	#undef TESTPREFIX_6
	#undef TESTPREFIX_5
	#undef TESTPREFIX_4
	#undef TESTPREFIX_3
	#undef TESTPREFIX_2
	#undef TESTPREFIX_1
	#undef TESTPREFIX_0
}

TEST_CASE_NAMED(FPreprocessorHelperArgAppendCountTest, "System::Core::HAL::UE_JOIN", "[ApplicationContextMask][EngineFilter]")
{
	#define TESTMACRO_0 abc
	#define TESTMACRO_1 defgh
	#define TESTMACRO_2 ij
	#define TESTMACRO_3 klmn
	#define TESTMACRO_4 o
	#define TESTMACRO_5 pqrstuvwxyz

	CHECK(FCStringAnsi::Strcmp(UE_STRINGIZE(UE_JOIN(TESTMACRO_0, TESTMACRO_1)), "abcdefgh") == 0); //-V549
	CHECK(FCStringAnsi::Strcmp(UE_STRINGIZE(UE_JOIN(TESTMACRO_0, TESTMACRO_1, TESTMACRO_2)), "abcdefghij") == 0); //-V549
	CHECK(FCStringAnsi::Strcmp(UE_STRINGIZE(UE_JOIN(TESTMACRO_0, TESTMACRO_1, TESTMACRO_2, TESTMACRO_3)), "abcdefghijklmn") == 0); //-V549
	CHECK(FCStringAnsi::Strcmp(UE_STRINGIZE(UE_JOIN(TESTMACRO_0, TESTMACRO_1, TESTMACRO_2, TESTMACRO_3, TESTMACRO_4)), "abcdefghijklmno") == 0); //-V549
	CHECK(FCStringAnsi::Strcmp(UE_STRINGIZE(UE_JOIN(TESTMACRO_0, TESTMACRO_1, TESTMACRO_2, TESTMACRO_3, TESTMACRO_4, TESTMACRO_5)), "abcdefghijklmnopqrstuvwxyz") == 0); //-V549

	#undef TESTMACRO_5
	#undef TESTMACRO_4
	#undef TESTMACRO_3
	#undef TESTMACRO_2
	#undef TESTMACRO_1
	#undef TESTMACRO_0
}

TEST_CASE_NAMED(FPreprocessorHelperApplyTest, "System::Core::HAL::UE_FOR_EACH", "[ApplicationContextMask][EngineFilter]")
{
#define TESTMACRO(ARG1, ARG2) UE_STRINGIZE(UE_JOIN(ARG1, ARG2))
	const char* Strings[] = {
		UE_FOR_EACH(UE_STRINGIZE, (One), (Two), (Three)),
		UE_FOR_EACH(TESTMACRO, (One, 1), (Two, 2), (Three, 3))
	};
	static_assert(UE_ARRAY_COUNT(Strings) == 2);
	CHECK(FCStringAnsi::Strcmp(Strings[0], "OneTwoThree") == 0);
	CHECK(FCStringAnsi::Strcmp(Strings[1], "One1Two2Three3") == 0);

	const char* StringsSeparated[] = {
		UE_FOR_EACH_WITH_SEPARATOR(UE_STRINGIZE, UE_COMMA, (One), (Two), (Three)),
		UE_FOR_EACH_WITH_SEPARATOR(TESTMACRO, UE_COMMA, (One, 1), (Two, 2), (Three, 3))
	};
	static_assert(UE_ARRAY_COUNT(StringsSeparated) == 6);
	CHECK(FCStringAnsi::Strcmp(StringsSeparated[0], "One") == 0);
	CHECK(FCStringAnsi::Strcmp(StringsSeparated[1], "Two") == 0);
	CHECK(FCStringAnsi::Strcmp(StringsSeparated[2], "Three") == 0);
	CHECK(FCStringAnsi::Strcmp(StringsSeparated[3], "One1") == 0);
	CHECK(FCStringAnsi::Strcmp(StringsSeparated[4], "Two2") == 0);
	CHECK(FCStringAnsi::Strcmp(StringsSeparated[5], "Three3") == 0);

	// Test empty var-args
	const char* NoArgsStrings[] = {
		"Prefix",
		UE_FOR_EACH(UE_STRINGIZE)
		UE_FOR_EACH(TESTMACRO)
	};
	static_assert(UE_ARRAY_COUNT(NoArgsStrings) == 1);

#undef TESTMACRO
}

TEST_CASE_NAMED(FPreprocessorHelperCallTest, "System::Core::HAL::UE_EXPAND", "[ApplicationContextMask][EngineFilter]")
{
#define TESTMACRO(a, b, c, d) #a #b #c #d 
#define FIRST 1, 2
#define SECOND 3, 4

	const char* String = UE_EXPAND(TESTMACRO UE_LPAREN FIRST, SECOND UE_RPAREN);
	CHECK(FCStringAnsi::Strcmp(String, "1234") == 0);

	const char* String2 = UE_EXPAND(TESTMACRO UE_ADD_PARENS(FIRST, SECOND));
	CHECK(FCStringAnsi::Strcmp(String2, "1234") == 0);

#undef SECOND
#undef FIRST
#undef TESTMACRO
}
#endif //WITH_TESTS
