// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

	#include "HAL/Event.h"
	#include "Tasks/Task.h"
	#include "Tests/TestHarnessAdapter.h"

namespace UE
{
	TEST_CASE_NAMED(FSharedEventRefTest, "Core::HAL::FSharedEventRef", "[Core][HAL]")
	{
		SECTION("Default constructor (EEventMode::AutoReset)")
		{
			FSharedEventRef Event; // EEventMode::AutoReset (default)

			CHECK_FALSE(Event->IsManualReset());

			// Initial state: unsignaled
			CHECK_FALSE(Event->Wait(0));

			// Wait with timeout: returns false and respects the timeout duration
			const double PreWait = FPlatformTime::Seconds();
			CHECK_FALSE(Event->Wait(32));
			CHECK((FPlatformTime::Seconds() - PreWait) * 1000.0 >= 16.0);

			// After Trigger: first Wait returns true, then the signal is consumed (auto-reset)
			Event->Trigger();
			CHECK(Event->Wait(0));
			CHECK_FALSE(Event->Wait(0));

			// Cross-thread: signal arrives from a background task while blocked in Wait
			Tasks::Launch(UE_SOURCE_LOCATION, [&Event] {
				FPlatformProcess::SleepNoStats(0.02f);
				Event->Trigger();
			});
			const double CrossThreadPreWait = FPlatformTime::Seconds();
			CHECK(Event->Wait(5000));
			CHECK((FPlatformTime::Seconds() - CrossThreadPreWait) * 1000.0 >= 16.0);
			CHECK_FALSE(Event->Wait(0)); // auto-reset: signal was consumed by the Wait above
		}

		SECTION("Explicit constructor (EEventMode::AutoReset)")
		{
			FSharedEventRef Event(EEventMode::AutoReset);

			CHECK_FALSE(Event->IsManualReset());

			// Initial state: unsignaled
			CHECK_FALSE(Event->Wait(0));

			// Wait with timeout: returns false and respects the timeout duration
			const double PreWait = FPlatformTime::Seconds();
			CHECK_FALSE(Event->Wait(32));
			CHECK((FPlatformTime::Seconds() - PreWait) * 1000.0 >= 16.0);

			// After Trigger: first Wait returns true, then the signal is consumed (auto-reset)
			Event->Trigger();
			CHECK(Event->Wait(0));
			CHECK_FALSE(Event->Wait(0));

			// Cross-thread: signal arrives from a background task while blocked in Wait
			Tasks::Launch(UE_SOURCE_LOCATION, [&Event] {
				FPlatformProcess::SleepNoStats(0.02f);
				Event->Trigger();
			});
			const double CrossThreadPreWait = FPlatformTime::Seconds();
			CHECK(Event->Wait(5000));
			CHECK((FPlatformTime::Seconds() - CrossThreadPreWait) * 1000.0 >= 16.0);
			CHECK_FALSE(Event->Wait(0)); // auto-reset: signal was consumed by the Wait above
		}

		SECTION("Explicit constructor (EEventMode::ManualReset)")
		{
			FSharedEventRef Event(EEventMode::ManualReset);

			CHECK(Event->IsManualReset());

			// Initial state: unsignaled
			CHECK_FALSE(Event->Wait(0));

			// Wait with timeout: returns false and respects the timeout duration
			const double PreWait = FPlatformTime::Seconds();
			CHECK_FALSE(Event->Wait(32));
			CHECK((FPlatformTime::Seconds() - PreWait) * 1000.0 >= 16.0);

			// After Trigger: signal persists across multiple Wait calls (manual reset)
			Event->Trigger();
			CHECK(Event->Wait(0));
			CHECK(Event->Wait(0)); // still signaled
			CHECK(Event->Wait(0)); // still signaled

			// After Reset: unsignaled again
			Event->Reset();
			CHECK_FALSE(Event->Wait(0));

			// Cross-thread: signal arrives from a background task while blocked in Wait
			Tasks::Launch(UE_SOURCE_LOCATION, [&Event] {
				FPlatformProcess::SleepNoStats(0.02f);
				Event->Trigger();
			});
			const double CrossThreadPreWait = FPlatformTime::Seconds();
			CHECK(Event->Wait(5000));
			CHECK((FPlatformTime::Seconds() - CrossThreadPreWait) * 1000.0 >= 16.0);
			CHECK(Event->Wait(0)); // still signaled — manual reset does not consume on Wait

			// Cleanup
			Event->Reset();
		}
	}
} // namespace UE

#endif // WITH_TESTS
