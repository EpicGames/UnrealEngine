// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_TESTS

#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/ThreadSingleton.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Containers/Queue.h"
#include "Containers/StringConv.h"

namespace
{
	void TestIsJoinableAfterCreation()
	{
		FThread Thread(TEXT("Test.Thread.TestIsJoinableAfterCreation"), []() { /*NOOP*/ });
		CHECK_MESSAGE(TEXT("FThread must be joinable after construction"), Thread.IsJoinable());
		Thread.Join();
		UE_LOGF(LogTemp, Log, "%ls completed", StringCast<TCHAR>(__FUNCTION__).Get());
	}

	void TestIsJoinableAfterCompletion()
	{
		FThreadSafeBool bDone = false;
		FThread Thread(TEXT("Test.Thread.TestIsJoinableAfterCompletion"), [&bDone]() { bDone = true; });
		while (!bDone); // wait for completion //-V529
		CHECK_MESSAGE(TEXT("FThread must still be joinable after completion"), Thread.IsJoinable());
		Thread.Join();
		UE_LOGF(LogTemp, Log, "%ls completed", StringCast<TCHAR>(__FUNCTION__).Get());
	}

	void TestIsNotJoinableAfterJoining()
	{
		FThread Thread(TEXT("Test.Thread.TestIsNotJoinableAfterJoining"), []() { /*NOOP*/ });
		Thread.Join();
		CHECK_FALSE_MESSAGE(TEXT("FThread must not be joinable after joining"), Thread.IsJoinable());
		UE_LOGF(LogTemp, Log, "%ls completed", StringCast<TCHAR>(__FUNCTION__).Get());
	}

#if 0 // detaching is not implemented
	void TestIsNotJoinableAfterDetaching()
	{
		// two cases: it's either the calling thread detaches from the thread before the thread is completed
		{
			TAtomic<bool> bReady{ false };
			FThread Thread(TEXT("Test.Thread"), [&bReady]()
				{
					while (!bReady) {}
				});
			Thread.Detach();
			bReady = true; // make sure `Detach` is called before thread function exit
			CHECK_FALSE_MESSAGE(TEXT("FThread must not be joinable after detaching"), Thread.IsJoinable());
		}
		// or thread function is completed fast and `FThreadImpl` releases the reference to itself before
		// `Detach` call
		{
			TAtomic<bool> bReady{ false };
			FThread Thread(TEXT("Test.Thread"), [&bReady]() { /*NOOP*/});
			FPlatformProcess::Sleep(0.1); // let the thread exit before detaching
			Thread.Detach();
			bReady = true; // make sure `Detach` is called before thread function exit
			CHECK_FALSE_MESSAGE(TEXT("FThread must not be joinable after detaching"), Thread.IsJoinable());
		}
		UE_LOGF(LogTemp, Log, "%ls completed", StringCast<TCHAR>(__FUNCTION__).Get());
	}
#endif

	void TestAssertIfNotJoinedOrDetached()
	{
		// this does fails the `check`, but it seems there's no way to test this by UE4 Automation Testing, so commented out
		FThread Thread(TEXT("Test.Thread.TestAssertIfNotJoinedOrDetached"), []() { /*NOOP*/ });
		// should assert in the destructor
	}

	void TestDefaultConstruction()
	{
		{
			FThread Thread;
			CHECK_FALSE_MESSAGE(TEXT("Default-constructed FThread must be not joinable"), Thread.IsJoinable());
		}
		{	// check that default constructed thread can be "upgraded" to joinable thread
			FThread Thread;
			Thread = FThread(TEXT("Test.Thread.TestDefaultConstruction"), []() { /* NOOP */ });
			CHECK_MESSAGE(TEXT("Move-constructed FThread from joinable thread must be joinable"), Thread.IsJoinable());
			Thread.Join();
		}
		UE_LOGF(LogTemp, Log, "%ls completed", StringCast<TCHAR>(__FUNCTION__).Get());
	}

	void TestThreadSingleton()
	{
		{	// check that ThreadSingleton instances in different threads are isolated from each other
			class FThreadSingletonTest : public TThreadSingleton<FThreadSingletonTest>
			{
			public:
				void SetTestField(int NewValue) { TestField = NewValue; }
				int GetTestField() const { return TestField; }
			private:
				int TestField{ 0 };
			};
			FThreadSingletonTest::Get().SetTestField(1);
			FThread Thread;
			TAtomic<bool> bDefaultValuePass{ false };
			Thread = FThread(TEXT("Test.Thread.TestThreadSingleton"),
				[&bDefaultValuePass]()
				{
					bDefaultValuePass = FThreadSingletonTest::TryGet() == nullptr;
					bDefaultValuePass = bDefaultValuePass && (FThreadSingletonTest::Get().GetTestField() == 0);
					FThreadSingletonTest::Get().SetTestField(2);
				});
			Thread.Join();
			CHECK_MESSAGE(TEXT("Thread singleton should be uninitialized by default"), bDefaultValuePass);
			CHECK_MESSAGE(TEXT("Thread singletons in different threads should be isolated"), FThreadSingletonTest::Get().GetTestField() == 1);
		}
		{	// check that ThreadSingleton entries don't point to invalid memory after cleanup, and that destruction occurs in the reverse order of construction.
			class FThreadSingletonSecond : public TThreadSingleton<FThreadSingletonSecond>
			{
			};
			class FThreadSingletonFirst : public TThreadSingleton<FThreadSingletonFirst>
			{
				virtual ~FThreadSingletonFirst()
				{
					// By the time we reach this destructor, the second singleton's destructor should have been executed already.
					check(FThreadSingletonSecond::TryGet() == nullptr);
				}
			};
			FThread Thread;
			Thread = FThread(TEXT("Test.Thread.TestThreadSingleton"),
				[]()
				{
					FThreadSingletonFirst::Get();
					FThreadSingletonSecond::Get();
				});
			Thread.Join();
			CHECK_MESSAGE(TEXT("Thread singletons should be uninitialized by default"), FThreadSingletonFirst::TryGet() == nullptr);
			CHECK_MESSAGE(TEXT("Thread singletons should be uninitialized by default"), FThreadSingletonSecond::TryGet() == nullptr);
		}
		UE_LOGF(LogTemp, Log, "%ls completed", StringCast<TCHAR>(__FUNCTION__).Get());
	}

	void TestMovability()
	{
		{	// move constructor with default-constructed thread
			FThread Src;
			FThread Dst(MoveTemp(Src));
			CHECK_FALSE_MESSAGE(TEXT("Default-constructed thread must stay not joinable after moving out"), Src.IsJoinable());
			CHECK_FALSE_MESSAGE(TEXT("Move-constructed thread from not joinable thread must be not joinable"), Dst.IsJoinable());
		}
		{	// move constructor with joinable thread
			FThread Src(TEXT("Test.Thread.TestMovability.1"), []() { /* NOOP */ });
			FThread Dst(MoveTemp(Src));
			CHECK_FALSE_MESSAGE(TEXT("Moved out thread must be not joinable"), Src.IsJoinable());
			CHECK_MESSAGE(TEXT("Move-constructed thread from joinable thread must be joinable"), Dst.IsJoinable());
			Dst.Join();
		}
		{	// move assignment operator
			FThread Src(TEXT("Test.Thread.TestMovability.2"), []() { /* NOOP */ });
			FThread Dst;
			Dst = MoveTemp(Src);
			CHECK_FALSE_MESSAGE(TEXT("Moved out thread must be not joinable"), Src.IsJoinable());
			CHECK_MESSAGE(TEXT("Move-assigned thread from joinable thread must be joinable"), Dst.IsJoinable());
			Dst.Join();
		}
		{	// Failure test for move assignment operator of joinable thread
			//FThread Src(TEXT("Test.Thread"), []() { /* NOOP */ });
			//FThread Dst(TEXT("Test.Thread"), []() { /* NOOP */ });
			//Dst = MoveTemp(Src); // must assert that joinable thread wasn't joined before move-assignment, no way to test this
			//Dst.Join();
		}
		{	// Move assignment operator of thread that has been joined
			FThread Src(TEXT("Test.Thread.TestMovability.3"), []() { /* NOOP */ });
			FThread Dst(TEXT("Test.Thread.TestMovability.4"), []() { /* NOOP */ });
			Dst.Join();
			Dst = MoveTemp(Src);
			Dst.Join();
		}
		UE_LOGF(LogTemp, Log, "%ls completed", StringCast<TCHAR>(__FUNCTION__).Get());
	}

	// An example of possible implementation of Consumer/Producer idiom
	void TestTypicalUseCase()
	{
		FThreadSafeBool bQuitRequested = false;
		using FWork = uint32;
		TQueue<FWork> WorkQueue;
		FEvent* WorkQueuedEvent = FPlatformProcess::GetSynchEventFromPool();

		FThread WorkerThread(TEXT("Test.Thread.TestTypicalUseCase"), [&bQuitRequested, &WorkQueue, WorkQueuedEvent]() 
		{
			while (!bQuitRequested)
			{
				// get work
				FWork Work;
				if (!WorkQueue.Dequeue(Work))
				{
					WorkQueuedEvent->Wait();
					// first check if quitting requested
					continue;
				}

				// do work
				UE_LOGF(LogTemp, Log, "Work #%d consumed", Work);
			}

			UE_LOGF(LogTemp, Log, "Quit");
		});

		// produce work
		const int WorkNum = 3;
		for (FWork Work = 0; Work != WorkNum; ++Work)
		{
			WorkQueue.Enqueue(Work);
			WorkQueuedEvent->Trigger();
			UE_LOGF(LogTemp, Log, "Work #%d produced", Work);
		}

		UE_LOGF(LogTemp, Log, "Request to quit");
		bQuitRequested = true;
		// the thread can be blocked waiting for work, unblock it
		WorkQueuedEvent->Trigger();
		WorkerThread.Join();

		FPlatformProcess::ReturnSynchEventToPool(WorkQueuedEvent);

		// example of output:
		// Work #0 produced
		//	Work #0 consumed
		//	Work #1 produced
		//	Work #1 consumed
		//	Work #2 produced
		//	Work #2 consumed
		//	Request to quit
		//	The thread 0x96e0 has exited with code 0 (0x0).
		//	Quit
	
		UE_LOGF(LogTemp, Log, "%ls completed", StringCast<TCHAR>(__FUNCTION__).Get());
	}
}

TEST_CASE_NAMED(FThreadTest, "System::Core::HAL::Thread", "[ApplicationContextMask][EngineFilter]")
{
	UE_LOGF(LogTemp, Log, "%ls", StringCast<TCHAR>(__FUNCTION__).Get());

	TestIsJoinableAfterCreation();
	TestIsJoinableAfterCompletion();
	TestIsNotJoinableAfterJoining();
	
#if 0 // detaching is not implemented
	TestIsNotJoinableAfterDetaching();
#endif

	//TestAssertIfNotJoinedOrDetached();

	TestDefaultConstruction();
	TestMovability();

	TestTypicalUseCase();
	TestThreadSingleton();
}

#endif //WITH_TESTS
