// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Hash/Blake3.h"

#include "Misc/StringBuilder.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FHashBlake3Test, "System::Core::Hash::Blake3", "[Core][Hash][SmokeFilter]")
{
	struct FBlake3TestCase
	{
		uint32 InputLength;
		FStringView Hash;
	};

	constexpr FBlake3TestCase TestCases[] =
	{
		{     0u, TEXTVIEW("af1349b9f5f9a1a6a0404dea36dcc9499bcb25c9adc112b7cc9a93cae41f3262")},
		{     1u, TEXTVIEW("2d3adedff11b61f14c886e35afa036736dcd87a74d27b5c1510225d0f592e213")},
		{ 1'023u, TEXTVIEW("10108970eeda3eb932baac1428c7a2163b0e924c9a9e25b35bba72b28f70bd11")},
		{ 1'024u, TEXTVIEW("42214739f095a406f3fc83deb889744ac00df831c10daa55189b5d121c855af7")},
		{ 1'025u, TEXTVIEW("d00278ae47eb27b34faecf67b4fe263f82d5412916c1ffd97c8cb7fb814b8444")},
		{ 2'048u, TEXTVIEW("e776b6028c7cd22a4d0ba182a8bf62205d2ef576467e838ed6f2529b85fba24a")},
		{ 2'049u, TEXTVIEW("5f4d72f40d7a5f82b15ca2b2e44b1de3c2ef86c426c95c1af0b6879522563030")},
		{ 3'072u, TEXTVIEW("b98cb0ff3623be03326b373de6b9095218513e64f1ee2edd2525c7ad1e5cffd2")},
		{ 3'073u, TEXTVIEW("7124b49501012f81cc7f11ca069ec9226cecb8a2c850cfe644e327d22d3e1cd3")},
		{ 4'096u, TEXTVIEW("015094013f57a5277b59d8475c0501042c0b642e531b0a1c8f58d2163229e969")},
		{ 4'097u, TEXTVIEW("9b4052b38f1c5fc8b1f9ff7ac7b27cd242487b3d890d15c96a1c25b8aa0fb995")},
		{ 5'120u, TEXTVIEW("9cadc15fed8b5d854562b26a9536d9707cadeda9b143978f319ab34230535833")},
		{ 5'121u, TEXTVIEW("628bd2cb2004694adaab7bbd778a25df25c47b9d4155a55f8fbd79f2fe154cff")},
		{ 6'144u, TEXTVIEW("3e2e5b74e048f3add6d21faab3f83aa44d3b2278afb83b80b3c35164ebeca205")},
		{ 6'145u, TEXTVIEW("f1323a8631446cc50536a9f705ee5cb619424d46887f3c376c695b70e0f0507f")},
		{ 7'168u, TEXTVIEW("61da957ec2499a95d6b8023e2b0e604ec7f6b50e80a9678b89d2628e99ada77a")},
		{ 7'169u, TEXTVIEW("a003fc7a51754a9b3c7fae0367ab3d782dccf28855a03d435f8cfe74605e7817")},
		{ 8'192u, TEXTVIEW("aae792484c8efe4f19e2ca7d371d8c467ffb10748d8a5a1ae579948f718a2a63")},
		{ 8'193u, TEXTVIEW("bab6c09cb8ce8cf459261398d2e7aef35700bf488116ceb94a36d0f5f1b7bc3b")},
		{16'384u, TEXTVIEW("f875d6646de28985646f34ee13be9a576fd515f76b5b0a26bb324735041ddde4")},
		{31'744u, TEXTVIEW("62b6960e1a44bcc1eb1a611a8d6235b6b4b78f32e7abc4fb4c6cdcce94895c47")},
	};

	constexpr uint32 MaxInputLength = 31'744u;

	uint8 Input[MaxInputLength];
	for (uint32 Index = 0; Index < MaxInputLength; ++Index)
	{
		Input[Index] = uint8(Index % 251);
	}

	FBlake3 Hash;
	uint32 InputIndex = 0;
	for (const FBlake3TestCase& TestCase : TestCases)
	{
		REQUIRE(InputIndex <= TestCase.InputLength);
		REQUIRE(TestCase.InputLength <= MaxInputLength);
		Hash.Update(Input + InputIndex, TestCase.InputLength - InputIndex);
		InputIndex = TestCase.InputLength;
		TStringBuilder<70> HashHex;
		HashHex << Hash.Finalize();
		CHECK(HashHex.ToView() == TestCase.Hash);
	}
}

#endif //WITH_TESTS