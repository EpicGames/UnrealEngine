// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Misc/SecureHash.h"

#include "HAL/UnrealMemory.h"
#include "String/HexToBytes.h"
#include "Containers/StringView.h"
#include "Containers/StringConv.h"
#include "Containers/Array.h"
#include "Misc/StringBuilder.h"

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FHashSecureHashTest, "System::Core::Hash::SecureHash", "[Core][SHA1][SmokeFilter]")
{
	struct FSHA1TestCase
	{
		FAnsiStringView Hash;
		FAnsiStringView Input;
	};

	// https://csrc.nist.gov/CSRC/media/Projects/Cryptographic-Standards-and-Guidelines/documents/examples/SHA1.pdf
	SECTION("Examples")
	{
		static const FSHA1TestCase TestCases[] =
		{
			{ "a9993e364706816aba3e25717850c26c9cd0d89d", "abc" },
			{ "84983e441c3bd26ebaae4aa1f95129e5e54670f1", "abcdbcdecdefdefgefghfghighijhijkijkljklmklmnlmnomnopnopq" },
		};

		for (const auto& TestCase : TestCases)
		{
			FSHA1 Hash;
			Hash.Update((const uint8*)TestCase.Input.GetData(), TestCase.Input.Len());

			TStringBuilder<40> Hex;
			Hex << Hash.Finalize();

			REQUIRE(Hex.ToView() == StringCast<TCHAR>(TestCase.Hash.GetData()));
		}
	}

	// https://csrc.nist.gov/projects/cryptographic-algorithm-validation-program/secure-hashing#shavs
	SECTION("Validation")
	{
		static const FSHA1TestCase TestCases[] =
		{
			// from SHA1ShortMsg.rsp
			{ "da39a3ee5e6b4b0d3255bfef95601890afd80709", "" },
			{ "c1dfd96eea8cc2b62785275bca38ac261256e278", "36" },
			{ "0a1c2d555bbe431ad6288af5a54f93e0449c9232", "195a" },
			{ "bf36ed5d74727dfd5d7854ec6b1d49468d8ee8aa", "df4bd2" },
			{ "b78bae6d14338ffccfd5d5b5674a275f6ef9c717", "549e959e" },
			{ "60b7d5bb560a1acf6fa45721bd0abb419a841a89", "f7fb1be205" },
			{ "a6d338459780c08363090fd8fc7d28dc80e8e01f", "c0e5abeaea63" },
			{ "860328d80509500c1783169ebf0ba0c4b94da5e5", "63bfc1ed7f78ab" },
			{ "24a2c34b976305277ce58c2f42d5092031572520", "7e3d7b3eada98866" },
			{ "411ccee1f6e3677df12698411eb09d3ff580af97", "9e61e55d9ed37b1c20" },
			{ "05c915b5ed4e4c4afffc202961f3174371e90b5c", "9777cf90dd7c7e863506" },
			{ "af320b42d7785ca6c8dd220463be23a2d2cb5afc", "4eb08c9e683c94bea00dfa" },
			{ "9f4e66b6ceea40dcf4b9166c28f1c88474141da9", "0938f2e2ebb64f8af8bbfc91" },
			{ "e6c4363c0852951991057f40de27ec0890466f01", "74c9996d14e87d3e6cbea7029d" },
			{ "046a7b396c01379a684a894558779b07d8c7da20", "51dca5c0f8e5d49596f32d3eb874" },
			{ "d58a262ee7b6577c07228e71ae9b3e04c8abcda9", "3a36ea49684820a2adc7fc4175ba78" },
			{ "a150de927454202d94e656de4c7c0ca691de955d", "3552694cdf663fd94b224747ac406aaf" },
			{ "35a4b39fef560e7ea61246676e1b7e13d587be30", "f216a1cbde2446b1edf41e93481d33e2ed" },
			{ "7ce69b1acdce52ea7dbd382531fa1a83df13cae7", "a3cf714bf112647e727e8cfd46499acd35a6" },
			{ "b47be2c64124fa9a124a887af9551a74354ca411", "148de640f3c11591a6f8c5c48632c5fb79d3b7" },
			{ "8bb8c0d815a9c68a1d2910f39d942603d807fbcc", "63a3cc83fd1ec1b6680e9974a0514e1a9ecebb6a" },
			{ "b486f87fb833ebf0328393128646a6f6e660fcb1", "875a90909a8afc92fb7070047e9d081ec92f3d08b8" },
			{ "76159368f99dece30aadcfb9b7b41dab33688858", "444b25f9c9259dc217772cc4478c44b6feff62353673" },
			{ "dbc1cb575ce6aeb9dc4ebf0f843ba8aeb1451e89", "487351c8a5f440e4d03386483d5fe7bb669d41adcbfdb7" },
			{ "d7a98289679005eb930ab75efd8f650f991ee952", "46b061ef132b87f6d3b0ee2462f67d910977da20aed13705" },
			{ "fda26fa9b4874ab701ed0bb64d134f89b9c4cc50", "3842b6137bb9d27f3ca5bafe5bbb62858344fe4ba5c41589a5" },
			{ "c2ff7ccde143c8f0601f6974b1903eb8d5741b6e", "44d91d3d465a4111462ba0c7ec223da6735f4f5200453cf132c3" },
			{ "643c9dc20a929608f6caa9709d843ca6fa7a76f4", "cce73f2eabcb52f785d5a6df63c0a105f34a91ca237fe534ee399d" },
			{ "509ef787343d5b5a269229b961b96241864a3d74", "664e6e7946839203037a65a12174b244de8cbc6ec3f578967a84f9ce" },
			{ "b61ce538f1a1e6c90432b233d7af5b6524ebfbe3", "9597f714b2e45e3399a7f02aec44921bd78be0fefee0c5e9b499488f6e" },
			{ "5b7b94076b2fc20d6adb82479e6b28d07c902b75", "75c5ad1f3cbd22e8a95fc3b089526788fb4ebceed3e7d4443da6e081a35e" },
			{ "6066db99fc358952cf7fb0ec4d89cb0158ed91d7", "dd245bffe6a638806667768360a95d0574e1a0bd0d18329fdb915ca484ac0d" },
			{ "b89962c94d60f6a332fd60f6f07d4f032a586b76", "0321794b739418c24e7c2e565274791c4be749752ad234ed56cb0a6347430c6b" },
			{ "17bda899c13d35413d2546212bcd8a93ceb0657b", "4c3dcf95c2f0b5258c651fcd1d51bd10425d6203067d0748d37d1340d9ddda7db3" },
			{ "badcdd53fdc144b8bf2cc1e64d10f676eebe66ed", "b8d12582d25b45290a6e1bb95da429befcfdbf5b4dd41cdf3311d6988fa17cec0723" },
			{ "01b4646180f1f6d2e06bbe22c20e50030322673a", "6fda97527a662552be15efaeba32a3aea4ed449abb5c1ed8d9bfff544708a425d69b72" },
			{ "10016dc3a2719f9034ffcc689426d28292c42fc9", "09fa2792acbb2417e8ed269041cc03c77006466e6e7ae002cf3f1af551e8ce0bb506d705" },
			{ "9f42fa2bce6ef021d93c6b2d902273797e426535", "5efa2987da0baf0a54d8d728792bcfa707a15798dc66743754406914d1cfe3709b1374eaeb" },
			{ "cdf48bacbff6f6152515323f9b43a286e0cb8113", "2836de99c0f641cd55e89f5af76638947b8227377ef88bfba662e5682babc1ec96c6992bc9a0" },
			{ "b88fb75274b9b0fd57c0045988cfcef6c3ce6554", "42143a2b9e1d0b354df3264d08f7b602f54aad922a3d63006d097f683dc11b90178423bff2f7fe" },
			{ "c06d3a6a12d9e8db62e8cff40ca23820d61d8aa7", "eb60c28ad8aeda807d69ebc87552024ad8aca68204f1bcd29dc5a81dd228b591e2efb7c4df75ef03" },
			{ "6e40f9e83a4be93874bc97cdebb8da6889ae2c7a", "7de4ba85ec54747cdc42b1f23546b7e490e31280f066e52fac117fd3b0792e4de62d5843ee98c72015" },
			{ "3efc940c312ef0dfd4e1143812248db89542f6a5", "e70653637bc5e388ccd8dc44e5eace36f7398f2bac993042b9bc2f4fb3b0ee7e23a96439dc01134b8c7d" },
			{ "a0cf03f7badd0c3c3c4ea3717f5a4fb7e67b2e56", "dd37bc9f0b3a4788f9b54966f252174c8ce487cbe59c53c22b81bf77621a7ce7616dcb5b1e2ee63c2c309b" },
			{ "a544e06f1a07ceb175a51d6d9c0111b3e15e9859", "5f485c637ae30b1e30497f0fb7ec364e13c906e2813daa34161b7ac4a4fd7a1bddd79601bbd22cef1f57cbc7" },
			{ "199d986ed991b99a071f450c6b1121a727e8c735", "f6c237fb3cfe95ec8414cc16d203b4874e644cc9a543465cad2dc563488a659e8a2e7c981e2a9f22e5e868ffe1" },
			{ "33bac6104b0ad6128d091b5d5e2999099c9f05de", "da7ab3291553c659873c95913768953c6e526d3a26590898c0ade89ff56fbd110f1436af590b17fed49f8c4b2b1e" },
			{ "76d7db6e18c1f4ae225ce8ccc93c8f9a0dfeb969", "8cfa5fd56ee239ca47737591cba103e41a18acf8e8d257b0dbe8851134a81ff6b2e97104b39b76e19da256a17ce52d" },
			{ "f652f3b1549f16710c7402895911e2b86a9b2aee", "57e89659d878f360af6de45a9a5e372ef40c384988e82640a3d5e4b76d2ef181780b9a099ac06ef0f8a7f3f764209720" },
			{ "63faebb807f32be708cf00fc35519991dc4e7f68", "b91e64235dbd234eea2ae14a92a173ebe835347239cff8b02074416f55c6b60dc6ced06ae9f8d705505f0d617e4b29aef9" },
			{ "0e6730bc4a0e9322ea205f4edfff1fffda26af0a", "e42a67362a581e8cf3d847502215755d7ad425ca030c4360b0f7ef513e6980265f61c9fa18dd9ce668f38dbc2a1ef8f83cd6" },
			{ "b61a3a6f42e8e6604b93196c43c9e84d5359e6fe", "634db92c22010e1cbf1e1623923180406c515272209a8acc42de05cc2e96a1e94c1f9f6b93234b7f4c55de8b1961a3bf352259" },
			{ "32d979ca1b3ed0ed8c890d99ec6dd85e6c16abf4", "cc6ca3a8cb391cd8a5aff1faa7b3ffbdd21a5a3ce66cfaddbfe8b179e4c860be5ec66bd2c6de6a39a25622f9f2fcb3fc05af12b5" },
			{ "6f18190bd2d02fc93bce64756575cea36d08b1c3", "7c0e6a0d35f8ac854c7245ebc73693731bbbc3e6fab644466de27bb522fcb99307126ae718fe8f00742e6e5cb7a687c88447cbc961" },
			{ "68f525feea1d8dbe0117e417ca46708d18d7629a", "c5581d40b331e24003901bd6bf244aca9e9601b9d81252bb38048642731f1146b8a4c69f88e148b2c8f8c14f15e1d6da57b2daa9991e" },
			{ "a7272e2308622ff7a339460adc61efd0ea8dabdc", "ec6b4a88713df27c0f2d02e738b69db43abda3921317259c864c1c386e9a5a3f533dc05f3beeb2bec2aac8e06db4c6cb3cddcf697e03d5" },
			{ "aef843b86916c16f66c84d83a6005d23fd005c9e", "0321736beba578e90abc1a90aa56157d871618f6de0d764cc8c91e06c68ecd3b9de3824064503384db67beb7fe012232dacaef93a000fba7" },
			{ "be2cd6f380969be59cde2dff5e848a44e7880bd6", "d0a249a97b5f1486721a50d4c4ab3f5d674a0e29925d5bf2678ef6d8d521e456bd84aa755328c83fc890837726a8e7877b570dba39579aabdd" },
			{ "e5eb4543deee8f6a5287845af8b593a95a9749a1", "c32138531118f08c7dcc292428ad20b45ab27d9517a18445f38b8f0c2795bcdfe3ffe384e65ecbf74d2c9d0da88398575326074904c1709ba072" },
			{ "534c850448dd486787b62bdec2d4a0b140a1b170", "b0f4cfb939ea785eabb7e7ca7c476cdd9b227f015d905368ba00ae96b9aaf720297491b3921267576b72c8f58d577617e844f9f0759b399c6b064c" },
			{ "6fbfa6e4edce4cc85a845bf0d228dc39acefc2fa", "bd02e51b0cf2c2b8d204a026b41a66fbfc2ac37ee9411fc449c8d1194a0792a28ee731407dfc89b6dfc2b10faa27723a184afef8fd83def858a32d3f" },
			{ "018872691d9b04e8220e09187df5bc5fa6257cd9", "e33146b83e4bb671392218da9a77f8d9f5974147182fb95ba662cb66011989c16d9af104735d6f79841aa4d1df276615b50108df8a29dbc9de31f4260d" },
			{ "d98d512a35572f8bd20de62e9510cc21145c5bf4", "411c13c75073c1e2d4b1ecf13139ba9656cd35c14201f1c7c6f0eeb58d2dbfe35bfdeccc92c3961cfabb590bc1eb77eac15732fb0275798680e0c7292e50" },
			{ "9f3ea255f6af95c5454e55d7354cabb45352ea0b", "f2c76ef617fa2bfc8a4d6bcbb15fe88436fdc2165d3074629579079d4d5b86f5081ab177b4c3f530376c9c924cbd421a8daf8830d0940c4fb7589865830699" },
			{ "a70cfbfe7563dd0e665c7c6715a96a8d756950c0", "45927e32ddf801caf35e18e7b5078b7f5435278212ec6bb99df884f49b327c6486feae46ba187dc1cc9145121e1492e6b06e9007394dc33b7748f86ac3207cfe" },

			// first 5 from SHA1LongMsg.rsp
			{
				"d8fd6a91ef3b6ced05b98358a99107c1fac8c807",
				"7c9c67323a1df1adbfe5ceb415eaef0155ece2820f4d50c1ec22cba4928ac656"
				"c83fe585db6a78ce40bc42757aba7e5a3f582428d6ca68d0c3978336a6efb729"
				"613e8d9979016204bfd921322fdd5222183554447de5e6e9bbe6edf76d7b71e1"
				"8dc2e8d6dc89b7398364f652fafc734329aafa3dcd45d4f31e388e4fafd7fc64"
				"95f37ca5cbab7f54d586463da4bfeaa3bae09f7b8e9239d832b4f0a733aa609c"
				"c1f8d4"
			},
			{
				"4a75a406f4de5f9e1132069d66717fc424376388",
				"6cb70d19c096200f9249d2dbc04299b0085eb068257560be3a307dbd741a3378"
				"ebfa03fcca610883b07f7fea563a866571822472dade8a0bec4b98202d47a344"
				"312976a7bcb3964427eacb5b0525db22066599b81be41e5adaf157d925fac04b"
				"06eb6e01deb753babf33be16162b214e8db017212fafa512cdc8c0d0a15c10f6"
				"32e8f4f47792c64d3f026004d173df50cf0aa7976066a79a8d78deeeec951dab"
				"7cc90f68d16f786671feba0b7d269d92941c4f02f432aa5ce2aab6194dcc6fd3"
				"ae36c8433274ef6b1bd0d314636be47ba38d1948343a38bf9406523a0b2a8cd7"
				"8ed6266ee3c9b5c60620b308cc6b3a73c6060d5268a7d82b6a33b93a6fd6fe1d"
				"e55231d12c97"
			},
			{
				"a135e32581bb06289b8c83f040e9421ec79bbe01",
				"6487972d88d0dd390d8d09d134860f263f88df7a3412457adf510dcf164e6cf0"
				"41679b3a19fcc542af6a236ab03d66b2e8a155d1061ab7859f75732775fff682"
				"f8f4d5e50d3ab3770f4f66cb138155b4715d245b8069948ea016a45b7ef0fdde"
				"93188c57eef4717f3425181de5b9a5d4e0a2963f2a67a340eb1ae994b98a48ab"
				"19b90ab74391c50426d28287ac4f1eb93f5af1a68c7dae40876b8afaaf35a192"
				"93c1952e957978abee40ec32f2aa880c956c7eb72f117b397cefcfb4e75ace3b"
				"081776e46b13521e93559d453e32ab74ebc0859b9a8dd4d1d39000ebe95f984d"
				"80a3f5004dc91a051dfbdfe9194f4f9a483e4e7955577fb0933464c63eaec771"
				"044d59abc3029a079519f8460a693b25b4ce207ae9d9447fc4c5446e6dad234e"
				"9afdec0c562798cd0297318399e838be385845c6dd79ede66e2ae80afec6738d"
				"4d9bf44c8d9eddff6c5cd2c94e340e0ddac40384b9a1408c9a4b98c37a6081d5"
				"220fba92f1d03144db"
			},
			{
				"b22b87ea30f4050913f8f0241fc2ae2c319f52e7",
				"bd74e7f607cd7d905e90175d67650a6dc2f8a4e2d4ab1249ca88812bda7984de"
				"ccbbb6a1ba90a0e91434ddf5e6137ba85e39a598890a7f635d335242fce0e9e0"
				"37303b6c51e54aec06614ad5ccce06d9599c80016530d7fbb1da6eb548084b2b"
				"05babd7d553642443efda726a1fd71a8bc087c44f285e2bccf661ead475a7267"
				"3e4386fc4eea5197c4f13c0feb0a85bc8e67e28ab872684bbebdaa527f3c253d"
				"ebb2dc12c2693f8e9e2651b9345c0abed7a0fafa3e5d305386c95acb7a172e54"
				"13ef08e73b1bd4d0d6832e4c035bc8559f9b0cbd0caf037a30707641c0545356"
				"bee151a24068d70674ef1befe16f872aef4060faaad1a968c39c45dbd7595de8"
				"f472016b5ab812d77e545fca55000ee5ce773edaa129eac6473410c2499013b4"
				"be895f6c0f734becfe994306e776262d4528ed8577218e3cc5201f1d9e5f3f62"
				"230eb2caea014becfba60fcb1f3997aa5b3bb622b7205c714348ba155c30a79a"
				"2cea43b070cada807e630b4086b129051898e1d9e68d1d0ecc9429d20d6a1403"
				"e0035a442b37bf508eb87e8ea347a3e68427b6d48ed299ba65ecb37b38754f45"
				"47423eaea2aec403338db2dcfe61cff4a8d17c383656981e1838a23866b91d09"
				"698f39175d98af4175caed53"
			},
			{
				"d742931bc51d4d44ff938783be73dc7beccc980b",
				"a52638f0efb19bff5ec95fcde4ac9aabd95e14d2e5f84c551f43bc5376855e71"
				"519b6f877248739a20cd790b85baa00d5503da5cb056f02d4aacc760c91fe1fd"
				"6efb26def817e5a9c56616023bc9e2fe662765dae2c0b2edfcbe17db140da30c"
				"466de65c49c6f81496bbbd1acd81666455f23bb243dd987d7ea1362a20faac84"
				"1f1a36692cfcb4c3dbf5f6bb058c36296b8be64e9b56adc5187cacb7b58c054f"
				"422a9e6d6a61229fdc3b494da98f5a33ed1bee14b2d2f6ad1177ffe99a6bb553"
				"f7c4a6d0cb9e498ee0b63f388235d86c26c9d96e50fa7d1eb3bcb9279940c47a"
				"8510d7fb175b3279318d5fe45823baba5dbe31c33c7649fe447061db78b33baa"
				"3637b854163fe34915e931b9f3040807d9217d7b3fed62370dbe806c006b21cd"
				"5061d24490f366e4d5f23e201a7ec83ae31b46fe2108d1af56cc9d42f9117eca"
				"1cb5ab344c1fc334b9cf0d7f9739043bc3d413b3aa6e9d5067c240c52b4c5b89"
				"e25ccd8a136a002008a9273f30dec3f2c1736c04a1c7ce0087c9f25d5ec5bff2"
				"ea7ec0b0ad7c278f0ca712c9ae150e472521d958d0bd6da9ff0939725924b2ed"
				"7b410a0ce2fe3f6b0bf25884d885ec223605e318fdf6803218a9a06ce5103c62"
				"ded035087a98519b4eb180d778d7656b3d4811aaf11a128317d1acb3ca316639"
				"5c51c90a3cf164071d0d132c54b3810a8211ec7774d2288447abe7afd030375a"
				"3bed4c7cf1b28097c02e98ea36bf49e74d89fbe74ec6cc1def5cd8c8beb5b8ad"
				"c3cb48c56182ad337e3b9778e4a6c4"
			},
		};

		for (const auto& TestCase : TestCases)
		{
			TArray<uint8> Input;
			Input.AddUninitialized(TestCase.Input.Len() / 2);
			UE::String::HexToBytes(TestCase.Input, Input.GetData());

			FSHAHash Hash = FSHA1::HashBuffer(Input.GetData(), Input.Num());

			TStringBuilder<40> Hex;
			Hex << Hash;

			REQUIRE(Hex.ToView() == StringCast<TCHAR>(TestCase.Hash.GetData()));
		}
	}
}

#endif //WITH_TESTS
