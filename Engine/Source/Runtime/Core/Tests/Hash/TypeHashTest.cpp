// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"
#include "Tests/Assertions.h"

TEST_CASE_NAMED(FHashTest, "System::Core::Hash::TypeHash", "[Core][Hash][SmokeFilter]")
{
	const void* NullPtr = nullptr;
	const uint32 NullPtrHash = GetTypeHash(NullPtr);
	CHECK(NullPtrHash == 0);

	void* HighBitsPtr = (void*)0x00007ff000000000;
	const uint32 HighBitsPtrHash = GetTypeHash(HighBitsPtr);
	CHECK(HighBitsPtrHash != 0);
}

#endif //WITH_TESTS