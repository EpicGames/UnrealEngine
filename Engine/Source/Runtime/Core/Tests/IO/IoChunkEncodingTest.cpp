// Copyright Epic Games, Inc. All Rights Reserved.

#include "Memory/MemoryFwd.h"
#if WITH_LOW_LEVEL_TESTS

#include "Algo/Compare.h"
#include "IO/IoChunkEncoding.h"
#include "IO/IoDispatcher.h"
#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include <catch2/generators/catch_generators.hpp>

namespace UE::IO::ChunkEncodingTest
{

static FAES::FAESKey GenerateEncryptionKey()
{
	FAES::FAESKey Key;

	for (int32 Idx = 0; Idx < FAES::FAESKey::KeySize; ++Idx)
	{
		Key.Key[Idx] = uint8(Idx + 1);
	}

	return Key;
}

TArray<uint64> GenerateData(int32 N)
{
	TArray<uint64> Data;
	Data.SetNum(N);
	for (int32 Idx = 0; Idx < Data.Num(); ++Idx)
	{
		Data[Idx] = uint64(Idx);
	}
	return Data;
};

TEST_CASE("Core::IO::IoChunk", "[Core][IO]")
{
	SECTION("Encode/Decode with offset")
	{
		const auto UncompressAndValidate =[](
			FMemoryView EncodedData,
			FName CompressionMethod,
			const FAES::FAESKey& Key,
			const int32 OffsetCount,
			const int32 Count,
			const TConstArrayView<uint64> ExpectedValues)
		{
			FIoBuffer RawData(Count * sizeof(uint64));

			FMemoryView KeyView(Key.Key, Key.KeySize);
			const bool bDecoded = FIoChunkEncoding::Decode(EncodedData, CompressionMethod, KeyView, RawData.GetMutableView(), OffsetCount * sizeof(uint64));
			CHECK(bDecoded);

			TConstArrayView<uint64> DecodedValues(reinterpret_cast<const uint64*>(RawData.GetData()), int32(RawData.GetSize() / sizeof(uint64)));
			TConstArrayView<uint64> ExpectedRange = ExpectedValues.Mid(OffsetCount, Count);
			CHECK(Algo::Compare(DecodedValues, ExpectedRange));
		};

		const FAES::FAESKey Key = GenerateEncryptionKey();
		const FName Method = "Oodle";
		constexpr uint64 BlockSize = 64 * sizeof(uint64);
		constexpr int32 N = 5000;
		const TArray<uint64> ExpectedValues = GenerateData(N);
		
		FIoBuffer Encoded;
		const bool bEncoded = FIoChunkEncoding::Encode(
			FIoChunkEncodingParams{Method, MakeMemoryView(Key.Key, Key.KeySize), BlockSize},
			MakeMemoryView(ExpectedValues.GetData(), sizeof(uint64) * N),
			Encoded);
		
		CHECK(bEncoded);
		CHECK(Encoded.GetSize() > 0);

		UncompressAndValidate(Encoded.GetView(), Method, Key, 0, N, ExpectedValues);
		UncompressAndValidate(Encoded.GetView(), Method, Key, 1, N - 1, ExpectedValues);
		UncompressAndValidate(Encoded.GetView(), Method, Key, N - 1, 1, ExpectedValues);
		UncompressAndValidate(Encoded.GetView(), Method, Key, 0, 1, ExpectedValues);
		UncompressAndValidate(Encoded.GetView(), Method, Key, 2, 4, ExpectedValues);
		UncompressAndValidate(Encoded.GetView(), Method, Key, 0, 512, ExpectedValues);
		UncompressAndValidate(Encoded.GetView(), Method, Key, 3, 514, ExpectedValues);
		UncompressAndValidate(Encoded.GetView(), Method, Key, 256, 512, ExpectedValues);
		UncompressAndValidate(Encoded.GetView(), Method, Key, 512, 512, ExpectedValues);
		UncompressAndValidate(Encoded.GetView(), Method, Key, 512, 512, ExpectedValues);
		UncompressAndValidate(Encoded.GetView(), Method, Key, 4993, 4, ExpectedValues);
	}
}

} // namespace UE::IO::ChunkEncodingTest

#endif // WITH_LOW_LEVEL_TESTS
