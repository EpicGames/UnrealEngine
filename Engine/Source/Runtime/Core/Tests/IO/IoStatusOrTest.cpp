// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "IO/IoDispatcher.h"

#include "Tests/TestHarnessAdapter.h"

struct FIoStatusTestType
{
	FIoStatusTestType() { }
	FIoStatusTestType(const FIoStatusTestType& Other)
		: Text(Other.Text) { }
	FIoStatusTestType(FIoStatusTestType&&) = default;

	FIoStatusTestType(const FString& InText)
		: Text(InText) { }
	FIoStatusTestType(FString&& InText)
		: Text(MoveTemp(InText)) { }

	FIoStatusTestType& operator=(const FIoStatusTestType& Other) = default;
	FIoStatusTestType& operator=(FIoStatusTestType&& Other) = default;
	FIoStatusTestType& operator=(const FString& OtherText)
	{
		Text = OtherText;
		return *this;
	}

	FString Text;
};

TEST_CASE_NAMED(FIoStatusOrTest, "System::Core::IO::IoStatusOr", "[ApplicationContextMask][SmokeFilter]")
{
	// TestConstruct
	{
		TIoStatusOr<FIoStatusTestType> Result;
		CHECK_MESSAGE("Default IoStatus is Unknown", Result.Status() == FIoStatus::Unknown);
	}

	{
		const TIoStatusOr<FIoStatusTestType> Other;
		TIoStatusOr<FIoStatusTestType> Result(Other);
		CHECK_MESSAGE("Copy construct", Result.Status() == FIoStatus::Unknown);
	}

	{
		const FIoStatus IoStatus(EIoErrorCode::InvalidCode);
		TIoStatusOr<FIoStatusTestType> Result(IoStatus);
		CHECK_MESSAGE("Construct with status", Result.Status().GetErrorCode() == EIoErrorCode::InvalidCode);
	}

	{
		const FString ExpectedText("Unreal");
		const FIoStatusTestType Type(ExpectedText);
		TIoStatusOr<FIoStatusTestType> Result(Type);
		CHECK_MESSAGE("Construct with value", Result.ValueOrDie().Text == ExpectedText);
	}

	{
		const FString ExpectedText("Unreal");
		TIoStatusOr<FIoStatusTestType> Result(FIoStatusTestType("Unreal"));
		CHECK_MESSAGE("Construct with temporary value", Result.ValueOrDie().Text == ExpectedText);
	}

	{
		TIoStatusOr<FIoStatusTestType> Result(FString("Unreal"));
		CHECK_MESSAGE("Construct with value arguments", Result.ValueOrDie().Text == FString("Unreal"));
	}
	// TestAssignment
	{
		const EIoErrorCode ExpectedErrorCode = EIoErrorCode::InvalidCode;
		TIoStatusOr<FIoStatusTestType> Other = FIoStatus(ExpectedErrorCode);
		TIoStatusOr<FIoStatusTestType> Result;
		Result = Other;
		CHECK_MESSAGE("Assign IoStatusOr with status", Result.Status().GetErrorCode() == ExpectedErrorCode);
	}

	{
		const EIoErrorCode ExpectedErrorCode = EIoErrorCode::InvalidCode;
		TIoStatusOr<FIoStatusTestType> Result;
		Result = TIoStatusOr<FIoStatusTestType>(FIoStatus(ExpectedErrorCode));
		CHECK_MESSAGE("Assign temporary IoStatusOr with status", Result.Status().GetErrorCode() == ExpectedErrorCode);
	}

	{
		const FString ExpectedText("Unreal");
		TIoStatusOr<FIoStatusTestType> Other = FIoStatusTestType(ExpectedText);
		TIoStatusOr<FIoStatusTestType> Result;
		Result = Other;
		CHECK_MESSAGE("Assign IoStatusOr with value", Result.ValueOrDie().Text == ExpectedText);
	}

	{
		const FString ExpectedText("Unreal");
		TIoStatusOr<FIoStatusTestType> Result;
		Result = TIoStatusOr<FIoStatusTestType>(ExpectedText);
		CHECK_MESSAGE("Assign temporary IoStatusOr with value", Result.ValueOrDie().Text == ExpectedText);
	}

	{
		const EIoErrorCode ExpectedErrorCode = EIoErrorCode::InvalidCode;
		const FIoStatus IoStatus(ExpectedErrorCode);
		TIoStatusOr<FIoStatusTestType> Result;
		Result = IoStatus;
		CHECK_MESSAGE("Assign status", Result.Status().GetErrorCode() == ExpectedErrorCode);
	}

	{
		const EIoErrorCode ExpectedErrorCode = EIoErrorCode::InvalidCode;
		TIoStatusOr<FIoStatusTestType> Result;
		Result = FIoStatus(ExpectedErrorCode);
		CHECK_MESSAGE("Assign temporary status", Result.Status().GetErrorCode() == ExpectedErrorCode);
	}

	{
		const FString ExpectedText("Unreal");
		const FIoStatusTestType Value(ExpectedText);
		TIoStatusOr<FIoStatusTestType> Result;
		Result = Value;
		CHECK_MESSAGE("Assign value", Result.ValueOrDie().Text == ExpectedText);
	}

	{
		const FString ExpectedText("Unreal");
		TIoStatusOr<FIoStatusTestType> Result;
		Result = FIoStatusTestType(ExpectedText);
		CHECK_MESSAGE("Assign temporary value", Result.ValueOrDie().Text == ExpectedText);
	}
	// TestConsumeValue
	{
		const FString ExpectedText("Unreal");
		TIoStatusOr<FIoStatusTestType> Result = FIoStatusTestType(ExpectedText);
		FIoStatusTestType Value = Result.ConsumeValueOrDie();
		CHECK_MESSAGE("Consume value or die with valid value", Value.Text == ExpectedText);
	}
}

#endif //WITH_TESTS
