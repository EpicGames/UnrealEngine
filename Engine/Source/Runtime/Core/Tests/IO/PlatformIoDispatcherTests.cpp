// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Algo/RandomShuffle.h"
#include "IO/IoStatus.h"
#include "IO/PlatformIoDispatcherBase.h"
#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include <catch2/generators/catch_generators.hpp>

namespace UE
{

namespace PlatformIoDispatcherTest
{

FIoFileBlockRequestList RandomList(TArray<FIoFileBlockRequest>& Requests)
{
	TArray<int32> Indices;
	for (int32 Idx = 0; Idx < Requests.Num(); ++Idx)
	{
		Indices.Add(Idx);
	}
	Algo::RandomShuffle(Indices);

	FIoFileBlockRequestList Random;
	for (int32 Idx : Indices)
	{
		Random.AddTail(&Requests[Idx]);
	}

	return Random;
}

}

TEST_CASE("Core::IO::Platform::FileReadQueue", "[Core][IO]")
{
	using namespace PlatformIoDispatcherTest;

	SECTION("DequeueBySeqNo")
	{
		// Arrange
		const int32					ExpectedReadCount = 20;
		FPlatformIoDispatcherStats	Stats;
		FIoQueue					Queue(Stats);
		TArray<FIoFileBlockRequest> Enqueued;

		Enqueued.SetNum(ExpectedReadCount);
		FIoFileBlockRequestList ToEnqueue = RandomList(Enqueued);

		// Act
		Queue.Enqueue(MoveTemp(ToEnqueue));

		TArray<FIoFileBlockRequest*> Dequeued;
		while (FIoFileBlockRequest* Request = Queue.Dequeue())
		{
			Dequeued.Add(Request);
		}

		// Assert
		CHECK(Dequeued.Num() == ExpectedReadCount);
		for (int32 Idx = 0; Idx < Dequeued.Num(); ++Idx) 
		{
			CHECK(Dequeued[Idx]->SeqNo == Enqueued[Idx].SeqNo);
		}
	}

	SECTION("CancelledComesFirst")
	{
		// Arrange
		const int32						ExpectedReadCount = 5;
		FPlatformIoDispatcherStats		Stats;
		FIoQueue						Queue(Stats);
		TArray<FIoFileBlockRequest> 	Enqueued;
		FIoFileHandle					FileHandle(1);

		Enqueued.SetNum(ExpectedReadCount);
		FIoFileBlockRequestList ToEnqueue;
		for (FIoFileBlockRequest& Request : Enqueued)
		{
			ToEnqueue.AddTail(&Request);
		}

		// Act
		Queue.Enqueue(MoveTemp(ToEnqueue));
		Enqueued[3].ErrorCode = EIoErrorCode::Cancelled;
		Enqueued[4].ErrorCode = EIoErrorCode::Cancelled;
		Queue.ReprioritizeCancelled();

		TArray<FIoFileBlockRequest*> Dequeued;
		while (FIoFileBlockRequest* Request = Queue.Dequeue())
		{
			Dequeued.Add(Request);
		}

		// Assert
		CHECK(Dequeued.Num() == ExpectedReadCount);
		CHECK(Dequeued[0]->SeqNo == Enqueued[3].SeqNo);
		CHECK(Dequeued[1]->SeqNo == Enqueued[4].SeqNo);
		CHECK(Dequeued[2]->SeqNo == Enqueued[0].SeqNo);
		CHECK(Dequeued[3]->SeqNo == Enqueued[1].SeqNo);
		CHECK(Dequeued[4]->SeqNo == Enqueued[2].SeqNo);
	}

	SECTION("HigherPriorityComesFirst")
	{
		// Arrange
		const int32						ExpectedReadCount = 5;
		FPlatformIoDispatcherStats		Stats;
		FIoQueue						Queue(Stats);
		TArray<FIoFileBlockRequest> 	Enqueued;
		FIoFileHandle					FileHandle(1);

		Enqueued.SetNum(ExpectedReadCount);
		FIoFileBlockRequestList ToEnqueue;
		for (FIoFileBlockRequest& Request : Enqueued)
		{
			ToEnqueue.AddTail(&Request);
		}

		// Act
		Queue.Enqueue(MoveTemp(ToEnqueue));
		Enqueued[0].Priority = IoDispatcherPriority_Min;
		Enqueued[1].Priority = IoDispatcherPriority_Low;
		Enqueued[2].Priority = IoDispatcherPriority_Medium;
		Enqueued[3].Priority = IoDispatcherPriority_High;
		Enqueued[4].Priority = IoDispatcherPriority_Max;
		Queue.Reprioritize();

		TArray<FIoFileBlockRequest*> Dequeued;
		while (FIoFileBlockRequest* Request = Queue.Dequeue())
		{
			Dequeued.Add(Request);
		}

		// Assert
		CHECK(Dequeued.Num() == ExpectedReadCount);
		CHECK(Dequeued[0]->SeqNo == Enqueued[4].SeqNo);
		CHECK(Dequeued[1]->SeqNo == Enqueued[3].SeqNo);
		CHECK(Dequeued[2]->SeqNo == Enqueued[2].SeqNo);
		CHECK(Dequeued[3]->SeqNo == Enqueued[1].SeqNo);
		CHECK(Dequeued[4]->SeqNo == Enqueued[0].SeqNo);
	}

	SECTION("DequeueByOffset")
	{
		// Arrange
		const int32						ExpectedReadCount = 20;
		FPlatformIoDispatcherStats		Stats;
		FIoQueue						Queue(Stats);
		TArray<FIoFileBlockRequest> 	Enqueued;
		FIoFileHandle					FileHandle(1);
		uint64							FileBlockOffset = 0; 

		Queue.SortByOffset(true);

		for (int32 Idx = 0; Idx < ExpectedReadCount; ++Idx) 
		{
			FIoFileBlockRequest& Request = Enqueued.AddDefaulted_GetRef();
			Request.FileHandle = FileHandle;
			Request.FileOffset = FileBlockOffset++;
		}

		FIoFileBlockRequestList ToEnqueue;
		ToEnqueue.AddTail(&Enqueued[0]); // We need one successful read before sorting on offset works
		for (int32 Idx = Enqueued.Num() - 1; Idx >= 1; --Idx)
		{
			ToEnqueue.AddTail(&Enqueued[Idx]);
		}

		// Act
		Queue.Enqueue(MoveTemp(ToEnqueue));

		TArray<FIoFileBlockRequest*> Dequeued;
		while (FIoFileBlockRequest* Request = Queue.Dequeue())
		{
			Dequeued.Add(Request);
		}

		// Assert
		CHECK(Dequeued.Num() == Enqueued.Num());
		for (int32 Idx = 0; Idx < Dequeued.Num(); ++Idx) 
		{
			CHECK(Dequeued[Idx]->SeqNo == Enqueued[Idx].SeqNo);
		}
	}
}

} // namespace UE

#endif // WITH_LOW_LEVEL_TESTS
