// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/InternationalizationArchive.h"
#include "Internationalization/InternationalizationMetadata.h"

#include "Tests/TestHarnessAdapter.h"


TEST_CASE_NAMED(FArchiveTest, "System::Core::Misc::Internationalization Archive", "[EditorContext][ClientContext][SmokeFilter]")
{
	// Key metadata
	TSharedPtr< FLocMetadataObject > KeyMetadataA = MakeShareable(new FLocMetadataObject);
	TSharedPtr< FLocMetadataObject > KeyMetadataB = MakeShareable(new FLocMetadataObject);

	// Source metadata
	TSharedPtr< FLocMetadataObject > SourceMetadataA = MakeShareable(new FLocMetadataObject);
	TSharedPtr< FLocMetadataObject > SourceMetadataB = MakeShareable(new FLocMetadataObject);


	// Setup KeyMetadataA
	KeyMetadataA->SetStringField(TEXT("Gender"), TEXT("Masculine"));
	KeyMetadataA->SetStringField(TEXT("Plurality"), TEXT("Singular"));
	KeyMetadataA->SetStringField(TEXT("TargetGender"), TEXT("Masculine"));
	KeyMetadataA->SetStringField(TEXT("TargetPlurality"), TEXT("Singular"));

	// Setup KeyMetadataB
	KeyMetadataB->SetStringField(TEXT("Gender"), TEXT("Masculine"));
	KeyMetadataB->SetStringField(TEXT("Plurality"), TEXT("Singular"));
	KeyMetadataB->SetStringField(TEXT("TargetGender"), TEXT("Feminine"));
	KeyMetadataB->SetStringField(TEXT("TargetPlurality"), TEXT("Singular"));

	// Setup source metadata
	SourceMetadataA->SetBoolField(TEXT("*IsMature"), false);
	SourceMetadataB->SetBoolField(TEXT("*IsMature"), true);

	// Setup source item
	FLocItem SourceA(TEXT("TextA"));
	SourceA.MetadataObj = MakeShareable(new FLocMetadataObject(*SourceMetadataA));

	FLocItem Translation = SourceA;
	Translation.Text = TEXT("TranslatedTextA");

	FString TestNamespace = "TestNamespace";
	FString SourceAKey = "TextA";

	// Test entry add
	{
		bool TestOptionalTrue = true;
		bool TestOptionalFalse = false;

		// bIsOptional is not used as a key.  We ensure adding entries where the bIsOptional member is the only difference works as expected.
		FInternationalizationArchive TestArchive;
		TestArchive.AddEntry(TestNamespace, SourceAKey, SourceA, Translation, NULL, TestOptionalTrue);
		// Add duplicate entry that differs in bIsOptional flag.  This add should report success because we already have an entry with matching 
		//  namespace/source/keymetadata.  Differences in bIsOptional are not taken into consideration. 
		bool bResult = TestArchive.AddEntry(TestNamespace, SourceAKey, SourceA, Translation, NULL, TestOptionalFalse);
		CHECK_MESSAGE(TEXT("AddEntry result = true"), bResult);

		// We should only have one entry in the archive
		int32 EntryCount = 0;
		for (auto Iter = TestArchive.GetEntriesBySourceTextIterator(); Iter; ++Iter, ++EntryCount);
		CHECK_MESSAGE(TEXT("EntryCount == 1"), EntryCount == 1);

		// Make sure the original bIsOptional value is not overwritten by our second add.
		TSharedPtr< FArchiveEntry > FoundEntry = TestArchive.FindEntryByKey(TestNamespace, SourceAKey, NULL);
		if (!FoundEntry.IsValid())
		{
			FAIL_CHECK(TEXT("FArchiveEntry could not find entry using FindEntryByKey."));
		}
		else
		{
			CHECK_MESSAGE(TEXT("FoundEntry->Namespace == Namespace"), FoundEntry->bIsOptional == TestOptionalTrue);
		}
	}

	// Test lookup an entry
	{
		{
			FInternationalizationArchive TestArchive;
			TestArchive.AddEntry(TestNamespace, SourceAKey, SourceA, Translation, KeyMetadataA, false);

			TSharedPtr< FArchiveEntry > FoundEntry = TestArchive.FindEntryByKey(TestNamespace, SourceAKey, KeyMetadataA);
			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FArchiveEntry could not find entry using FindEntryByKey."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Namespace == Namespace"), FoundEntry->Namespace == TestNamespace);
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry->Source == SourceA);
				CHECK_MESSAGE(TEXT("FoundEntry->Translation == Translation"), FoundEntry->Translation == Translation);
				if (FoundEntry->KeyMetadataObj == KeyMetadataA)
				{
					FAIL_CHECK(TEXT("FArchiveEntry KeyMetadataObj is not a unique object."));
				}
				CHECK_MESSAGE(TEXT("FoundEntry->KeyMetadataObj == KeyMetadataA"), *(FoundEntry->KeyMetadataObj) == *(KeyMetadataA));
			}

			// Passing in a mismatched key metadata will fail to find the entry.  Any fallback logic is intended to happen at runtime
			FoundEntry = TestArchive.FindEntryByKey(TestNamespace, SourceAKey, NULL);
			CHECK_FALSE_MESSAGE(TEXT("!FoundEntry.IsValid()"), FoundEntry.IsValid());

			FoundEntry = TestArchive.FindEntryByKey(TestNamespace, SourceAKey, MakeShareable(new FLocMetadataObject()));
			CHECK_FALSE_MESSAGE(TEXT("!FoundEntry.IsValid()"), FoundEntry.IsValid());

			FoundEntry = TestArchive.FindEntryByKey(TestNamespace, SourceAKey, KeyMetadataB);
			CHECK_FALSE_MESSAGE(TEXT("!FoundEntry.IsValid()"), FoundEntry.IsValid());
		}

		// Ensure we can properly lookup entries with non-null but empty key metadata
		{
			FInternationalizationArchive TestArchive;
			TestArchive.AddEntry(TestNamespace, SourceAKey, SourceA, Translation, MakeShareable(new FLocMetadataObject()), false);

			TSharedPtr< FArchiveEntry > FoundEntry = TestArchive.FindEntryByKey(TestNamespace, SourceAKey, NULL);
			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FArchiveEntry could not find entry using FindEntryByKey."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Namespace == Namespace"), FoundEntry->Namespace == TestNamespace);
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry->Source == SourceA);
			}
		}

		// Ensure we can properly lookup entries with null key metadata
		{
			FInternationalizationArchive TestArchive;
			TestArchive.AddEntry(TestNamespace, SourceAKey, SourceA, Translation, NULL, false);

			TSharedPtr< FArchiveEntry > FoundEntry = TestArchive.FindEntryByKey(TestNamespace, SourceAKey, NULL);

			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FArchiveEntry could not find entry using FindEntryByKey."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Namespace == Namespace"), FoundEntry->Namespace == TestNamespace);
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry->Source == SourceA);
			}

			// Test lookup with non-null but empty key metadata
			FoundEntry = TestArchive.FindEntryByKey(TestNamespace, SourceAKey, MakeShareable(new FLocMetadataObject()));
			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FArchiveEntry could not find entry using FindEntryByKey."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Namespace == Namespace"), FoundEntry->Namespace == TestNamespace);
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry->Source == SourceA);
			}
		}

		// Ensure lookup where source metadata has * prefixed entries work as expected.  Note: The source metadata object
		//   supports a * prefix on metadata names which modifies the way we perform metadata comparison(ignores entry type and value, only name is checked)
		{
			FLocItem SourceCompare(TEXT("TextA"));
			SourceCompare.MetadataObj = MakeShareable(new FLocMetadataObject());
			SourceCompare.MetadataObj->SetStringField(TEXT("*IsMature"), TEXT(""));

			FInternationalizationArchive TestArchive;
			// Added entry with String *IsMature entry
			TestArchive.AddEntry(TestNamespace, SourceAKey, SourceCompare, Translation, KeyMetadataA, false);

			// Finding entry using a source that has Boolean *IsMature
			TSharedPtr< FArchiveEntry > FoundEntry = TestArchive.FindEntryByKey(TestNamespace, SourceAKey, KeyMetadataA);
			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FArchiveEntry could not find entry using FindEntryByKey."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Namespace == Namespace"), FoundEntry->Namespace == TestNamespace);
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry->Source == SourceA);
			}

			// Attempting to add an entry that only differs by a * prefexed source metadata entry value or type will result in success since
			//  a 'matching' entry already exists in the archive.  We should, however, only have one entry in the archive
			bool bResult = TestArchive.AddEntry(TestNamespace, SourceAKey, SourceA, Translation, KeyMetadataA, false);

			CHECK_MESSAGE(TEXT("AddEntry result = true"), bResult);

			// We should only have one entry in the archive
			int32 EntryCount = 0;
			for (auto Iter = TestArchive.GetEntriesBySourceTextIterator(); Iter; ++Iter, ++EntryCount);
			CHECK_MESSAGE(TEXT("EntryCount == 1"), EntryCount == 1);

			// Check to see that the original type/value of the * prefixed entry did not get modified
			FoundEntry = TestArchive.FindEntryByKey(TestNamespace, SourceAKey, KeyMetadataA);
			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FArchiveEntry could not find entry using FindEntryByKey."));
			}
			else
			{
				if (FoundEntry->Source.MetadataObj->HasTypedField< ELocMetadataType::String >(TEXT("*IsMature")))
				{
					CHECK_MESSAGE(TEXT("Metadata Type == String and Value == Empty string"), FoundEntry->Source.MetadataObj->GetStringField(TEXT("*IsMature")) == TEXT(""));
				}
				else
				{
					FAIL_CHECK(TEXT("FArchiveEntry * prefixed metadata entry on source object was modified unexpectedly."));
				}
			}
		}
	}
}

#endif //WITH_TESTS