// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"
#include "Internationalization/Text.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

class FDateTimeFormattingRulesTestClass {
public:
	void DateTimeFormattingRulesTest() {
		auto TestText = [this](const TCHAR* const Desc, const FText& A, const FText& B)
		{
			if (!A.EqualTo(B))
			{
				FAIL_CHECK(FString::Printf(TEXT("%s - A=%s B=%s"), Desc, *A.ToString(), *B.ToString()));
			}
		};
		auto TestStr = [this](const TCHAR* const Desc, const FString& A, const FString& B)
		{
			if (!A.Equals(B))
			{
				FAIL_CHECK(FString::Printf(TEXT("%s - A=%s B=%s"), Desc, *A, *B));
			}
		};

		FInternationalization& I18N = FInternationalization::Get();

#if UE_ENABLE_ICU
		{
			FInternationalization::FCultureStateSnapshot OriginalCultureState;
			I18N.BackupCultureState(OriginalCultureState);

			const FDateTime UnixEpoch = FDateTime::FromUnixTimestamp(0);
			const FDateTime UnixBillennium = FDateTime::FromUnixTimestamp(1000000000);
			const FDateTime UnixOnes = FDateTime::FromUnixTimestamp(1111111111);
			const FDateTime UnixDecimalSequence = FDateTime::FromUnixTimestamp(1234567890);
			const FDateTime YearOne(1, 1, 1, 00, 00, 00, 000);
			const FDateTime TestDateTime(1990, 6, 13, 12, 34, 56, 789);

			const FDateTime LocalTime = FDateTime::Now();
			const FDateTime UtcTime = FDateTime::UtcNow();

			if (I18N.SetCurrentCulture("en-US"))
			{
				// Unix Time Values via Date Time
				TestText(TEXT("Testing Unix Epoch"), FText::AsDateTime(UnixEpoch, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1/1/70, 12:00 AM")));
				TestText(TEXT("Testing Unix Epoch"), FText::AsDateTime(UnixEpoch, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Jan 1, 1970, 12:00:00 AM")));
				TestText(TEXT("Testing Unix Epoch"), FText::AsDateTime(UnixEpoch, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("January 1, 1970 at 12:00:00 AM GMT")));
				TestText(TEXT("Testing Unix Epoch"), FText::AsDateTime(UnixEpoch, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Thursday, January 1, 1970 at 12:00:00 AM GMT")));

				TestText(TEXT("Testing Unix Billennium"), FText::AsDateTime(UnixBillennium, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("9/9/01, 1:46 AM")));
				TestText(TEXT("Testing Unix Billennium"), FText::AsDateTime(UnixBillennium, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Sep 9, 2001, 1:46:40 AM")));
				TestText(TEXT("Testing Unix Billennium"), FText::AsDateTime(UnixBillennium, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("September 9, 2001 at 1:46:40 AM GMT")));
				TestText(TEXT("Testing Unix Billennium"), FText::AsDateTime(UnixBillennium, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Sunday, September 9, 2001 at 1:46:40 AM GMT")));

				TestText(TEXT("Testing Unix Ones"), FText::AsDateTime(UnixOnes, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("3/18/05, 1:58 AM")));
				TestText(TEXT("Testing Unix Ones"), FText::AsDateTime(UnixOnes, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Mar 18, 2005, 1:58:31 AM")));
				TestText(TEXT("Testing Unix Ones"), FText::AsDateTime(UnixOnes, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("March 18, 2005 at 1:58:31 AM GMT")));
				TestText(TEXT("Testing Unix Ones"), FText::AsDateTime(UnixOnes, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Friday, March 18, 2005 at 1:58:31 AM GMT")));

				TestText(TEXT("Testing Unix Decimal Sequence"), FText::AsDateTime(UnixDecimalSequence, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2/13/09, 11:31 PM")));
				TestText(TEXT("Testing Unix Decimal Sequence"), FText::AsDateTime(UnixDecimalSequence, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Feb 13, 2009, 11:31:30 PM")));
				TestText(TEXT("Testing Unix Decimal Sequence"), FText::AsDateTime(UnixDecimalSequence, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("February 13, 2009 at 11:31:30 PM GMT")));
				TestText(TEXT("Testing Unix Decimal Sequence"), FText::AsDateTime(UnixDecimalSequence, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Friday, February 13, 2009 at 11:31:30 PM GMT")));

				TestText(TEXT("Testing Year One"), FText::AsDateTime(YearOne, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1/1/01, 12:00 AM")));
				TestText(TEXT("Testing Year One"), FText::AsDateTime(YearOne, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Jan 1, 1, 12:00:00 AM")));
				TestText(TEXT("Testing Year One"), FText::AsDateTime(YearOne, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("January 1, 1 at 12:00:00 AM GMT")));
				TestText(TEXT("Testing Year One"), FText::AsDateTime(YearOne, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Saturday, January 1, 1 at 12:00:00 AM GMT")));

				// Date Time
				TestText(TEXT("Testing Date-Time"), FText::AsDateTime(TestDateTime, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("6/13/90, 12:34 PM")));
				TestText(TEXT("Testing Date-Time"), FText::AsDateTime(TestDateTime, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Jun 13, 1990, 12:34:56 PM")));
				TestText(TEXT("Testing Date-Time"), FText::AsDateTime(TestDateTime, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("June 13, 1990 at 12:34:56 PM GMT")));
				TestText(TEXT("Testing Date-Time"), FText::AsDateTime(TestDateTime, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("Wednesday, June 13, 1990 at 12:34:56 PM GMT")));

				// Checks that the default ICU timezone is set correctly (including DST)
				TestText(TEXT("Testing Local Time"), FText::AsDateTime(UtcTime, EDateTimeStyle::Short, EDateTimeStyle::Short), FText::AsDateTime(LocalTime, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()));
			}
			else
			{
				WARN(FString::Printf(TEXT("Internationalization data for %s missing - test is partially disabled."), TEXT("en-US")));
			}

			if (I18N.SetCurrentCulture("ja-JP"))
			{
				// Unix Time Values via Date Time
				TestText(TEXT("Testing Unix Epoch"), FText::AsDateTime(UnixEpoch, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1970/01/01 0:00")));
				TestText(TEXT("Testing Unix Epoch"), FText::AsDateTime(UnixEpoch, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1970/01/01 0:00:00")));
				TestText(TEXT("Testing Unix Epoch"), FText::AsDateTime(UnixEpoch, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1970\x5E74") TEXT("1\x6708") TEXT("1\x65E5") TEXT(" 0:00:00 GMT")));
				TestText(TEXT("Testing Unix Epoch"), FText::AsDateTime(UnixEpoch, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1970\x5E74") TEXT("1\x6708") TEXT("1\x65E5\x6728\x66DC\x65E5") TEXT(" ") TEXT("0\x6642") TEXT("00\x5206") TEXT("00\x79D2") TEXT(" GMT")));

				TestText(TEXT("Testing Unix Billennium"), FText::AsDateTime(UnixBillennium, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2001/09/09 1:46")));
				TestText(TEXT("Testing Unix Billennium"), FText::AsDateTime(UnixBillennium, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2001/09/09 1:46:40")));
				TestText(TEXT("Testing Unix Billennium"), FText::AsDateTime(UnixBillennium, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2001\x5E74") TEXT("9\x6708") TEXT("9\x65E5") TEXT(" 1:46:40 GMT")));
				TestText(TEXT("Testing Unix Billennium"), FText::AsDateTime(UnixBillennium, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2001\x5E74") TEXT("9\x6708") TEXT("9\x65E5\x65E5\x66DC\x65E5") TEXT(" ") TEXT("1\x6642") TEXT("46\x5206") TEXT("40\x79D2") TEXT(" GMT")));

				TestText(TEXT("Testing Unix Ones"), FText::AsDateTime(UnixOnes, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2005/03/18 1:58")));
				TestText(TEXT("Testing Unix Ones"), FText::AsDateTime(UnixOnes, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2005/03/18 1:58:31")));
				TestText(TEXT("Testing Unix Ones"), FText::AsDateTime(UnixOnes, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2005\x5E74") TEXT("3\x6708") TEXT("18\x65E5") TEXT(" 1:58:31 GMT")));
				TestText(TEXT("Testing Unix Ones"), FText::AsDateTime(UnixOnes, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2005\x5E74") TEXT("3\x6708") TEXT("18\x65E5\x91D1\x66DC\x65E5") TEXT(" ") TEXT("1\x6642") TEXT("58\x5206") TEXT("31\x79D2") TEXT(" GMT")));

				TestText(TEXT("Testing Unix Decimal Sequence"), FText::AsDateTime(UnixDecimalSequence, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2009/02/13 23:31")));
				TestText(TEXT("Testing Unix Decimal Sequence"), FText::AsDateTime(UnixDecimalSequence, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2009/02/13 23:31:30")));
				TestText(TEXT("Testing Unix Decimal Sequence"), FText::AsDateTime(UnixDecimalSequence, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2009\x5E74") TEXT("2\x6708") TEXT("13\x65E5 23:31:30 GMT")));
				TestText(TEXT("Testing Unix Decimal Sequence"), FText::AsDateTime(UnixDecimalSequence, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("2009\x5E74") TEXT("2\x6708") TEXT("13\x65E5\x91D1\x66DC\x65E5") TEXT(" ") TEXT("23\x6642") TEXT("31\x5206") TEXT("30\x79D2") TEXT(" GMT")));

				TestText(TEXT("Testing Year One"), FText::AsDateTime(YearOne, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1/01/01 0:00")));
				TestText(TEXT("Testing Year One"), FText::AsDateTime(YearOne, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1/01/01 0:00:00")));
				TestText(TEXT("Testing Year One"), FText::AsDateTime(YearOne, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1\x5E74") TEXT("1\x6708") TEXT("1\x65E5") TEXT(" 0:00:00 GMT")));
				TestText(TEXT("Testing Year One"), FText::AsDateTime(YearOne, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1\x5E74") TEXT("1\x6708") TEXT("1\x65E5\x571F\x66DC\x65E5") TEXT(" ") TEXT("0\x6642") TEXT("00\x5206") TEXT("00\x79D2 GMT")));

				// Date Time
				TestText(TEXT("Testing Date-Time"), FText::AsDateTime(TestDateTime, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1990/06/13 12:34")));
				TestText(TEXT("Testing Date-Time"), FText::AsDateTime(TestDateTime, EDateTimeStyle::Medium, EDateTimeStyle::Medium, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1990/06/13 12:34:56")));
				TestText(TEXT("Testing Date-Time"), FText::AsDateTime(TestDateTime, EDateTimeStyle::Long, EDateTimeStyle::Long, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1990\x5E74") TEXT("6\x6708") TEXT("13\x65E5") TEXT(" 12:34:56 GMT")));
				TestText(TEXT("Testing Date-Time"), FText::AsDateTime(TestDateTime, EDateTimeStyle::Full, EDateTimeStyle::Full, FText::GetInvariantTimeZone()), FText::FromString(TEXT("1990\x5E74") TEXT("6\x6708") TEXT("13\x65E5\x6C34\x66DC\x65E5") TEXT(" ") TEXT("12\x6642") TEXT("34\x5206") TEXT("56\x79D2 GMT")));

				// Checks that the default ICU timezone is set correctly (including DST)
				TestText(TEXT("Testing Local Time"), FText::AsDateTime(UtcTime, EDateTimeStyle::Short, EDateTimeStyle::Short), FText::AsDateTime(LocalTime, EDateTimeStyle::Short, EDateTimeStyle::Short, FText::GetInvariantTimeZone()));
			}
			else
			{
				WARN(FString::Printf(TEXT("Internationalization data for %s missing - test is partially disabled."), TEXT("ja-JP")));
			}

			I18N.RestoreCultureState(OriginalCultureState);
		}
#else
		WARN("ICU is disabled thus locale-aware date/time formatting is disabled.");
#endif

		{
			FCultureRef InvariantCulture = I18N.GetInvariantCulture();
			const FDateTime TestDateTime = FDateTime(1991, 6, 21, 9, 30);

			TestStr(TEXT("Testing Custom Date-Time Pattern (FDateTime) - %m/%d/%Y"), TestDateTime.ToFormattedString(TEXT("%m/%d/%Y")), TEXT("06/21/1991"));
			TestStr(TEXT("Testing Custom Date-Time Pattern (FDateTime) - %A, %B %e, %Y"), TestDateTime.ToFormattedString(TEXT("%A, %B %e, %Y")), TEXT("Friday, June 21, 1991"));
			TestStr(TEXT("Testing Custom Date-Time Pattern (FDateTime) - %a, %b %e"), TestDateTime.ToFormattedString(TEXT("%a, %b %e")), TEXT("Fri, Jun 21"));
			TestStr(TEXT("Testing Custom Date-Time Pattern (FDateTime) - %H:%M"), TestDateTime.ToFormattedString(TEXT("%H:%M")), TEXT("09:30"));
			TestStr(TEXT("Testing Custom Date-Time Pattern (FDateTime) - %l:%M %p"), TestDateTime.ToFormattedString(TEXT("%l:%M %p")), TEXT("9:30 AM"));

			TestStr(TEXT("Testing Custom Date-Time Pattern (FText) - %m/%d/%Y"), FText::AsDateTime(TestDateTime, TEXT("%m/%d/%Y"), FText::GetInvariantTimeZone(), InvariantCulture).ToString(), TEXT("06/21/1991"));
			TestStr(TEXT("Testing Custom Date-Time Pattern (FText) - %A, %B %e, %Y"), FText::AsDateTime(TestDateTime, TEXT("%A, %B %e, %Y"), FText::GetInvariantTimeZone(), InvariantCulture).ToString(), TEXT("Friday, June 21, 1991"));
			TestStr(TEXT("Testing Custom Date-Time Pattern (FText) - %a, %b %e"), FText::AsDateTime(TestDateTime, TEXT("%a, %b %e"), FText::GetInvariantTimeZone(), InvariantCulture).ToString(), TEXT("Fri, Jun 21"));
			TestStr(TEXT("Testing Custom Date-Time Pattern (FText) - %H:%M"), FText::AsDateTime(TestDateTime, TEXT("%H:%M"), FText::GetInvariantTimeZone(), InvariantCulture).ToString(), TEXT("09:30"));
			TestStr(TEXT("Testing Custom Date-Time Pattern (FText) - %l:%M %p"), FText::AsDateTime(TestDateTime, TEXT("%l:%M %p"), FText::GetInvariantTimeZone(), InvariantCulture).ToString(), TEXT("9:30 AM"));
		}
	}

};

TEST_CASE_NAMED(FDateTimeFormattingRulesTest, "System::Core::Misc::DateTime Formatting Rules", "[.][EditorContext][ClientContext][EngineFilter]")
{
	FDateTimeFormattingRulesTestClass Instance;
	Instance.DateTimeFormattingRulesTest();
}

#endif //WITH_TESTS