// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/LocTesting.h"
#include "Internationalization/PolyglotTextData.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Cultures/KeysCulture.h"

#if WITH_TESTS && ENABLE_LOC_TESTING

#include "Tests/TestHarnessAdapter.h"

#define LOCTEXT_NAMESPACE "Core.Tests.KeysCultureTests"

/** 
* A utility struct to ensure that the internationalization system is reverted
* to its original state by the end of a test. Declare this at the start of tests that involve culture changes.
*/
struct FRestoreCultureStateGuard
{
	FRestoreCultureStateGuard()
	{
		FInternationalization::Get().BackupCultureState(OriginalCultureState);
	}
	~FRestoreCultureStateGuard()
	{
		FInternationalization::Get().RestoreCultureState(OriginalCultureState);
	}
private:
	FInternationalization::FCultureStateSnapshot OriginalCultureState;
};
/**
* A helper function that generates the expected output string when the keys culture is active.
* If you change the behavior of what gets displayed by the keys culture, be sure to update this function. FTextLocalizationManager::KeyifyAllDisplayStrings is currently the place that is done.
*/
FString CreateExpectedOutputFromPolyglotData(const FPolyglotTextData& InData)
{
	FString Key;
	FString Namespace;
	InData.GetIdentity(Namespace, Key);
	// Right now, we display it as key, namespace
	return FString::Printf(TEXT("%s, %s"), *Key, *Namespace);
}

TEST_CASE_NAMED(FKeysCultureTest, "Development::System::Core::Misc::KeysCulture", "[.][EditorContext][ClientContext][EngineFilter]")
{
	FRestoreCultureStateGuard RestoreCultureStateGuard;
	
	static const FString NativeCulture = "en";
	static const FString TargetCulture = "ja";

	// Test Overview
	// 1. We set up the test environment and change the culture to the native culture
	// 2. We register text data that as a native string and a target culture translation.
	// 3. We register text data that as a native string and no target culture translation. 
	// 4. We change to the target culture. The text with translation should have its display string the same as the localized string in the target culture. The display string for the text with no translation should match the native string.
	// 5. We change to the keys culture. Both text with and without translations in a target culture should have their display strings show the localization key and namespace of the text.
	// 6. We revert back to the native culture. Both text with and without translations should have their display strings the same as their native strings with no change.
	
	FInternationalization& I18N = FInternationalization::Get();
	// We initialize the test environment to the native culture to start from a clean slate.
	I18N.SetCurrentCulture(NativeCulture);
	// somehow failed to initialize the test environment to the right culture. Early out.
	REQUIRE(I18N.GetCurrentCulture()->GetName() == NativeCulture);

#define CREATE_TEST_POLYGLOTDATA(InKey, InSourceString) FPolyglotTextData(ELocalizedTextSourceCategory::Editor, TEXT(LOCTEXT_NAMESPACE), TEXT(InKey), TEXT(InSourceString), NativeCulture)
	// We use polyglot text data here to patch in our test strings so we don't have to create a separte loclization resource file 
	// Test1 is for text with a native string and a non-empty translation in a target language
	// Simulates text with a valid translation editor/game
	FPolyglotTextData Test1 = CREATE_TEST_POLYGLOTDATA("Test1Key", "Test1 Native");
	Test1.AddLocalizedString(TargetCulture, TEXT("Test1 Target"));

	// Test2 is for text with a valid native string but a missing translation in a target culture
	// Simulates text with missing translation in editor/game
	FPolyglotTextData Test2 = CREATE_TEST_POLYGLOTDATA("Test2Key", "Test2 Native");
#undef CREATE_TEST_POLYGLOTDATA

	// GetText() already handles registration of the polyglot text data with the FTextLocalizationManager internally and adds these strings to the display string table.
	FText Test1Text = Test1.GetText();
	FText Test2Text = Test2.GetText();
	
	FString Test1NativeCultureDisplayString = Test1Text.ToString();
	// Test1 display string should have the same value as the native string. Otherwise it could be a problem with registering the text. The set up of the test failed. Early out.
	REQUIRE(Test1NativeCultureDisplayString == Test1.GetNativeString());

	FString Test2NativeCultureDisplayString = Test2Text.ToString();
	// Same as above.
	REQUIRE(Test2NativeCultureDisplayString == Test2.GetNativeString());

		// Change to target culture. 
	I18N.SetCurrentCulture(TargetCulture);
	// Test 1 - We should be able to change from the native culture to the target culture 
	// just early out. Doesn't make sense to keep testing.
	REQUIRE_MESSAGE(TEXT("Changing from native culture to target culture."), I18N.GetCurrentCulture()->GetName() == TargetCulture);

	// Test 2 - The display string for the text with translation should match the localized string in the target culture 
	FString Test1TargetCultureDisplayString = Test1Text.ToString();
	FString Test1TargetCultureString;
	Test1.GetLocalizedString(TargetCulture, Test1TargetCultureString);
	CHECK_MESSAGE(TEXT("Text with translation display is the same as the localized string in target culture."), Test1TargetCultureDisplayString == Test1TargetCultureString);

	// Test 3 - The text with no translation should have no retrievable text from it spolyglot data in the target culture.
	// Test 4 - The display string for the text with no translation should match the native string while in the target culture. Text with no translations for a culture should fall back to the native string. 
	FString Test2TargetCultureDisplayString = Test2Text.ToString();
	FString Test2TargetCultureString;
	Test2.GetLocalizedString(TargetCulture, Test2TargetCultureString);
	CHECK_MESSAGE(TEXT("Text without translation should not have a localized string in the target culture."), Test2TargetCultureString.IsEmpty());
	CHECK_MESSAGE(TEXT("Text without translation display string is the same as the native string in target culture."), Test2TargetCultureDisplayString == Test2.GetNativeString());
	
	// Change to the keys culture
	// This culture should output all the display strings as the localization key and namespace of the text.
	// There will be logs that say the localization resource files associated with the keys culture can't be read. That is normal and can be ignored.
	I18N.SetCurrentCulture(FKeysCulture::StaticGetName());
	// Test 5 - Can we successfully change from the native culture to the keys culture 
	// just early out. Doesn't make sense to keep testing.
	REQUIRE_MESSAGE(TEXT("Changing from native culture to keys culture."), I18N.GetCurrentCulture()->GetName() == FKeysCulture::StaticGetName());

	// Test 6 - The display string for the text with translation should be the localization key and localization namespace for the text in the keys culture 
	FString Test1KeysCultureDisplayString = Test1Text.ToString();
	CHECK_MESSAGE(TEXT("Text with translation now has display string that matches its localization key and namespace in keys culture."), Test1KeysCultureDisplayString == CreateExpectedOutputFromPolyglotData(Test1));
	
	// Test 7 - The display string for the text with no translation should be its localization key and localization namespace in the keys culture.
	FString Test2KeysCultureDisplayString = Test2Text.ToString();
	CHECK_MESSAGE(TEXT("Text with blank translation has display string that matches its localization key and namespace in keys culture."), Test2KeysCultureDisplayString == CreateExpectedOutputFromPolyglotData(Test2));

	// Revert to native culture. All of the text should still show up as the originals.
	I18N.SetCurrentCulture(NativeCulture);
	// Test 8 - Can we change from the keys culture back to the native culture 
	// early out same as above 
	REQUIRE_MESSAGE(TEXT("Able to revert from keys culture to native culture.."), I18N.GetCurrentCulture()->GetName() == NativeCulture);

	// Test 9 - The display string for the text with translation should be exactly the same as its original native string before changing to the keys culture
	FString Test1NativeCultureRevertedDisplayString = Test1Text.ToString();
	CHECK_MESSAGE(TEXT("Text with translation has display string that is the same as the original native string in native culture."), Test1NativeCultureRevertedDisplayString == Test1.GetNativeString());

	// Test 10 - The display string for the text with no translation should be exactly the same as its original native string before changing to the keys culture
	FString Test2NativeCultureRevertedDisplayString = Test2Text.ToString();
	CHECK_MESSAGE(TEXT("Text without translation has display string the same as its native string in native culture."), Test2NativeCultureRevertedDisplayString == Test2.GetNativeString());

// @TODOLocalization: Unregister the polyglot data once the functionality is implemented.
}

#undef LOCTEXT_NAMESPACE 
#endif // WITH_TESTS && ENABLE_LOC_TESTING
