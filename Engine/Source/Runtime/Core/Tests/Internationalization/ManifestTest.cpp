// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/InternationalizationMetadata.h"

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

// Helper function to count the number of entries in a manifest
int32 CountManifestEntries( const FInternationalizationManifest& Manifest )
{
	return Manifest.GetNumEntriesByKey();
}

TEST_CASE_NAMED(FLocContextTest, "System::Core::Misc::Internationalization Context", "[EditorContext][ClientContext][SmokeFilter]")
{
	// Key metadata
	TSharedPtr< FLocMetadataObject > KeyMetadataA = MakeShareable( new FLocMetadataObject );
	TSharedPtr< FLocMetadataObject > KeyMetadataB = MakeShareable( new FLocMetadataObject );

	// Info metadata
	TSharedPtr< FLocMetadataObject > InfoMetadataA = MakeShareable( new FLocMetadataObject );
	TSharedPtr< FLocMetadataObject > InfoMetadataB = MakeShareable( new FLocMetadataObject );

	// Source metadata
	TSharedPtr< FLocMetadataObject > SourceMetadataA = MakeShareable( new FLocMetadataObject );
	TSharedPtr< FLocMetadataObject > SourceMetadataB = MakeShareable( new FLocMetadataObject );


	// Setup KeyMetadataA
	KeyMetadataA->SetStringField( TEXT("Gender"			),	TEXT("Masculine")	);
	KeyMetadataA->SetStringField( TEXT("Plurality"		),	TEXT("Singular")	);
	KeyMetadataA->SetStringField( TEXT("TargetGender"	),	TEXT("Masculine")	);
	KeyMetadataA->SetStringField( TEXT("TargetPlurality"),	TEXT("Singular")	);

	// Setup KeyMetadataB
	KeyMetadataB->SetStringField( TEXT("Gender"			),	TEXT("Masculine")	);
	KeyMetadataB->SetStringField( TEXT("Plurality"		),	TEXT("Singular")	);
	KeyMetadataB->SetStringField( TEXT("TargetGender"	),	TEXT("Feminine")	);
	KeyMetadataB->SetStringField( TEXT("TargetPlurality"),	TEXT("Singular")	);

	// Setup source metadata
	SourceMetadataA->SetBoolField( TEXT("*IsMature"), false );
	SourceMetadataB->SetBoolField( TEXT("*IsMature"), true );

	// Set InfoMetadataA
	InfoMetadataA->SetStringField( TEXT("VoiceActorDirection"), TEXT("Go big or go home!") );

	// Test FManifestContext
	{
		FManifestContext ContextA;
		ContextA.Key			= TEXT("KeyA");
		ContextA.SourceLocation = TEXT("SourceLocationA");
		ContextA.InfoMetadataObj = MakeShareable( new FLocMetadataObject(*InfoMetadataA) );
		ContextA.KeyMetadataObj = MakeShareable( new FLocMetadataObject(*KeyMetadataA) );

		FManifestContext ContextB;
		ContextB.Key			= TEXT("KeyB");
		ContextB.SourceLocation = TEXT("SourceLocationB");
		ContextB.InfoMetadataObj = MakeShareable( new FLocMetadataObject(*InfoMetadataB) );
		ContextB.KeyMetadataObj = MakeShareable( new FLocMetadataObject(*KeyMetadataB) );


		// Test copy ctor
		{
			FManifestContext ContextAClone = ContextA;

			if (ContextAClone.InfoMetadataObj == ContextA.InfoMetadataObj)
			{
				FAIL_CHECK(TEXT("FManifestContext InfoMetadataObj and its Clone are not unique objects."));
			}
			if (ContextAClone.KeyMetadataObj == ContextA.KeyMetadataObj)
			{
				FAIL_CHECK(TEXT("FManifestContext KeyMetadataObj and its Clone are not unique objects."));
			}

			CHECK_MESSAGE(TEXT("ContextAClone.Key == ContextA.Key"), ContextAClone.Key == ContextA.Key );
			CHECK_MESSAGE(TEXT("ContextAClone.SourceLocation == ContextA.SourceLocation"), ContextAClone.SourceLocation == ContextA.SourceLocation );
			CHECK_MESSAGE(TEXT("ContextAClone.bIsOptional == ContextA.bIsOptional"), ContextAClone.bIsOptional == ContextA.bIsOptional );
			CHECK_MESSAGE(TEXT("ContextAClone.InfoMetadataObj == ContextA.InfoMetadataObj"), *(ContextAClone.InfoMetadataObj) == *(ContextA.InfoMetadataObj) );
			CHECK_MESSAGE(TEXT("ContextAClone.KeyMetadataObj == ContextA.KeyMetadataObj"), *(ContextAClone.KeyMetadataObj) == *(ContextA.KeyMetadataObj) );

			CHECK_MESSAGE(TEXT("ContextAClone == ContextA"), ContextAClone == ContextA );
			CHECK_FALSE_MESSAGE(TEXT("ContextAClone < ContextA"), ContextAClone < ContextA );

		}

		// Test assignment operator
		{
			FManifestContext ContextAClone = ContextA;

			if (ContextAClone.InfoMetadataObj == ContextA.InfoMetadataObj)
			{
				FAIL_CHECK(TEXT("FManifestContext InfoMetadataObj and its Clone are not unique objects."));
			}
			if (ContextAClone.KeyMetadataObj == ContextA.KeyMetadataObj)
			{
				FAIL_CHECK(TEXT("FManifestContext KeyMetadataObj and its Clone are not unique objects."));
			}

			CHECK_MESSAGE(TEXT("ContextAClone.Key == ContextA.Key"), ContextAClone.Key == ContextA.Key );
			CHECK_MESSAGE(TEXT("ContextAClone.SourceLocation == ContextA.SourceLocation"), ContextAClone.SourceLocation == ContextA.SourceLocation );
			CHECK_MESSAGE(TEXT("ContextAClone.bIsOptional == ContextA.bIsOptional"), ContextAClone.bIsOptional == ContextA.bIsOptional );
			CHECK_MESSAGE(TEXT("ContextAClone.InfoMetadataObj == ContextA.InfoMetadataObj"), *(ContextAClone.InfoMetadataObj) == *(ContextA.InfoMetadataObj) );
			CHECK_MESSAGE(TEXT("ContextAClone.KeyMetadataObj == ContextA.KeyMetadataObj"), *(ContextAClone.KeyMetadataObj) == *(ContextA.KeyMetadataObj) );

			CHECK_MESSAGE(TEXT("ContextAClone == ContextA"), ContextAClone == ContextA );
			CHECK_FALSE_MESSAGE(TEXT("ContextAClone < ContextA"), ContextAClone < ContextA );
		}

		// Test comparison operator
		{
			// Key and KeyMetadataObj members should be the only items that are taken into account when comparing
			FManifestContext ContextAClone = ContextA;
			CHECK_MESSAGE(TEXT("ContextAClone == ContextA"), ContextAClone == ContextA );

			// Arbitrarily change all the non-important members
			ContextAClone.SourceLocation = ContextA.SourceLocation + TEXT("New");
			ContextAClone.bIsOptional = !ContextA.bIsOptional;
			ContextAClone.InfoMetadataObj.Reset();
			ContextAClone.InfoMetadataObj = MakeShareable( new FLocMetadataObject( *InfoMetadataB ) );
			CHECK_MESSAGE(TEXT("ContextAClone == ContextA"), ContextAClone == ContextA );

			// Changing the key in any way will cause comparison to fail
			ContextAClone.Key = ContextAClone.Key.GetString() + TEXT("New");
			CHECK_FALSE_MESSAGE(TEXT("ContextAClone != ContextA"), ContextAClone == ContextA );

			// Reset and test KeyMetadataObj change to one of the value entries
			ContextAClone = ContextA;
			ContextAClone.KeyMetadataObj->SetStringField( TEXT("TargetPlurality"), TEXT("Plural") );
			CHECK_FALSE_MESSAGE(TEXT("ContextAClone != ContextA"), ContextAClone == ContextA );

			// Reset and test addition of entry to KeyMetadataObj
			ContextAClone = ContextA;
			ContextAClone.KeyMetadataObj->SetStringField( TEXT("NewField"), TEXT("NewFieldValue") );
			CHECK_FALSE_MESSAGE(TEXT("ContextAClone != ContextA"), ContextAClone == ContextA );

			// Reset and test removal of entry from KeyMetadataObj
			ContextAClone = ContextA;
			ContextAClone.KeyMetadataObj->RemoveField( TEXT("TargetPlurality") );
			CHECK_FALSE_MESSAGE(TEXT("ContextAClone != ContextA"), ContextAClone == ContextA );

			// Context with valid but empty KeyMetadataObject should be equivalent to Context with null KeyMetadataObject
			FManifestContext ContextEmptyA;
			FManifestContext ContextEmptyB;
			ContextEmptyB.KeyMetadataObj = MakeShareable( new FLocMetadataObject );
			CHECK_MESSAGE(TEXT("ContextEmptyA == ContextEmptyB"), ContextEmptyA == ContextEmptyB );
		}

		// Testing less than operator
		{
			CHECK_MESSAGE(TEXT("ContextA < ContextB"), ContextA < ContextB );

			FManifestContext ContextAClone = ContextA;

			// Differences in Key
			CHECK_FALSE_MESSAGE(TEXT("ContextA < ContextAClone"), ContextA < ContextAClone );
			ContextAClone.Key = ContextAClone.Key.GetString() + TEXT("A");
			//currently failing CHECK_MESSAGE( TEXT("ContextA < ContextAClone"), ContextA < ContextAClone );

			// Adding new key metadata entry that will appear before other entries
			ContextAClone = ContextA;
			ContextAClone.KeyMetadataObj->SetStringField( TEXT("ANewKey"), TEXT("ANewValue") );
			CHECK_MESSAGE(TEXT("ContextAClone < ContextA"), ContextAClone < ContextA );

			// Adding new key metadata entry that will appear after other entries
			ContextAClone = ContextA;
			ContextAClone.KeyMetadataObj->SetStringField( TEXT("ZNewKey"), TEXT("ZNewValue") );
			//currently failing CHECK_MESSAGE( TEXT("ContextA < ContextAClone"), ContextA < ContextAClone );

			// Removing a key metadata entry
			ContextAClone = ContextA;
			ContextAClone.KeyMetadataObj->RemoveField( TEXT("TargetPlurality") ) ;
			CHECK_MESSAGE(TEXT("ContextAClone < ContextA"), ContextAClone < ContextA );

			// Changing a key metadata entry value
			ContextAClone = ContextA;
			ContextAClone.KeyMetadataObj->SetStringField( TEXT("TargetPlurality"), TEXT("A") ) ;
			CHECK_MESSAGE(TEXT("ContextAClone < ContextA"), ContextAClone < ContextA );

			FManifestContext ContextEmptyA;
			FManifestContext ContextEmptyB;
			CHECK_FALSE_MESSAGE( TEXT("ContextEmptyA < ContextEmptyB"), ContextEmptyA < ContextEmptyB );
			ContextEmptyB.KeyMetadataObj = MakeShareable( new FLocMetadataObject );
			CHECK_FALSE_MESSAGE( TEXT("ContextEmptyA < ContextEmptyB"), ContextEmptyA < ContextEmptyB );
			CHECK_FALSE_MESSAGE( TEXT("ContextEmptyB < ContextEmptyA"), ContextEmptyB < ContextEmptyA );
			ContextEmptyB.KeyMetadataObj->SetStringField(TEXT("AMetadataKey"), TEXT("AMetadataValue"));
			CHECK_MESSAGE( TEXT("ContextEmptyA < ContextEmptyB"), ContextEmptyA < ContextEmptyB );
		}
	}
}

TEST_CASE_NAMED(FLocItemTest, "System::Core::Misc::Internationalization LocItem", "[EditorContext][ClientContext][SmokeFilter]")
{
	// Key metadata
	TSharedPtr< FLocMetadataObject > KeyMetadataA = MakeShareable( new FLocMetadataObject );
	TSharedPtr< FLocMetadataObject > KeyMetadataB = MakeShareable( new FLocMetadataObject );

	// Info metadata
	TSharedPtr< FLocMetadataObject > InfoMetadataA = MakeShareable( new FLocMetadataObject );
	TSharedPtr< FLocMetadataObject > InfoMetadataB = MakeShareable( new FLocMetadataObject );

	// Source metadata
	TSharedPtr< FLocMetadataObject > SourceMetadataA = MakeShareable( new FLocMetadataObject );
	TSharedPtr< FLocMetadataObject > SourceMetadataB = MakeShareable( new FLocMetadataObject );


	// Setup KeyMetadataA
	KeyMetadataA->SetStringField( TEXT("Gender"			),	TEXT("Masculine")	);
	KeyMetadataA->SetStringField( TEXT("Plurality"		),	TEXT("Singular")	);
	KeyMetadataA->SetStringField( TEXT("TargetGender"	),	TEXT("Masculine")	);
	KeyMetadataA->SetStringField( TEXT("TargetPlurality"),	TEXT("Singular")	);

	// Setup KeyMetadataB
	KeyMetadataB->SetStringField( TEXT("Gender"			),	TEXT("Masculine")	);
	KeyMetadataB->SetStringField( TEXT("Plurality"		),	TEXT("Singular")	);
	KeyMetadataB->SetStringField( TEXT("TargetGender"	),	TEXT("Feminine")	);
	KeyMetadataB->SetStringField( TEXT("TargetPlurality"),	TEXT("Singular")	);

	// Setup source metadata
	SourceMetadataA->SetBoolField( TEXT("*IsMature"), false );
	SourceMetadataB->SetBoolField( TEXT("*IsMature"), true );

	// Set InfoMetadataA
	InfoMetadataA->SetStringField( TEXT("VoiceActorDirection"), TEXT("Go big or go home!") );


	// Test FLocItem
	{
		FLocItem LocItemA( TEXT("TextA") );
		LocItemA.MetadataObj = MakeShareable( new FLocMetadataObject(*SourceMetadataA) );

		FLocItem LocItemB( TEXT("TextB") );
		LocItemB.MetadataObj = MakeShareable( new FLocMetadataObject(*SourceMetadataB) );

		// Test copy ctor
		{
			FLocItem LocItemAClone = LocItemA;

			if (LocItemAClone.MetadataObj == LocItemA.MetadataObj)
			{
				FAIL_CHECK(TEXT("FLocItem MetadataObj and its Clone are not unique objects."));
			}
			CHECK_MESSAGE(TEXT("LocItemAClone.Text == LocItemA.Text"), LocItemAClone.Text == LocItemA.Text );
			CHECK_MESSAGE(TEXT("LocItemAClone.MetadataObj == LocItemA.MetadataObj"), *(LocItemAClone.MetadataObj) == *(LocItemA.MetadataObj) );

			CHECK_MESSAGE(TEXT("LocItemAClone == LocItemA"), LocItemAClone == LocItemA );
			CHECK_FALSE_MESSAGE(TEXT("LocItemAClone < LocItemA"), LocItemAClone < LocItemA );

		}

		// Test assignment operator
		{
			FLocItem LocItemAClone( TEXT("New") );

			LocItemAClone = LocItemA;

			if (LocItemAClone.MetadataObj == LocItemA.MetadataObj)
			{
				FAIL_CHECK(TEXT("FLocItem MetadataObj and its Clone are not unique objects."));
			}
			CHECK_MESSAGE (TEXT("LocItemAClone.Text == LocItemA.Text"), LocItemAClone.Text == LocItemA.Text );
			CHECK_MESSAGE(TEXT("LocItemAClone.MetadataObj == LocItemA.MetadataObj"), *(LocItemAClone.MetadataObj) == *(LocItemA.MetadataObj) );

			CHECK_MESSAGE(TEXT("LocItemAClone == LocItemA"), LocItemAClone == LocItemA );
			CHECK_FALSE_MESSAGE(TEXT("LocItemAClone < LocItemA"), LocItemAClone < LocItemA );
			CHECK_FALSE_MESSAGE(TEXT("LocItemA < LocItemAClone"), LocItemA < LocItemAClone );
		}

		// Test comparison operator
		{
			// Text and MetadataObj members should both be taken into account when comparing.  Note, Metadata supports a special * name prefix that causes the type 
			//   and value of the metadata to be ignored when performing comparisons
			FLocItem LocItemAClone = LocItemA;
			CHECK_MESSAGE(TEXT("LocItemAClone == LocItemA"), LocItemAClone == LocItemA );

			// Metadata with * prefix does not impact comparison but both FLocItems need a metadata where the name matches( type and value can be different )
			FLocItem LocItemAClone2 = LocItemA;
			LocItemAClone.MetadataObj->SetStringField( TEXT("*NewNonCompare"), TEXT("NewNonCompareValue") );
			CHECK_FALSE_MESSAGE(TEXT("LocItemAClone != LocItemAClone2"), LocItemAClone == LocItemAClone2 );

			LocItemAClone2.MetadataObj->SetStringField( TEXT("*NewNonCompare"), TEXT("NewNonCompareValue2") );
			CHECK_MESSAGE(TEXT("LocItemAClone == LocItemAClone2"), LocItemAClone == LocItemAClone2 );

			// Changing the text in any way will cause comparison to fail
			LocItemAClone = LocItemA;
			LocItemAClone.Text = LocItemAClone.Text + TEXT("New");
			CHECK_FALSE_MESSAGE(TEXT("LocItemAClone != LocItemA"), LocItemAClone == LocItemA );

			// LocItem with valid but empty KeyMetadataObject should be equivalent to LocItem with null KeyMetadataObject
			FLocItem LocItemEmptyA(TEXT("TestText"));
			FLocItem LocItemEmptyB(TEXT("TestText"));
			LocItemEmptyB.MetadataObj = MakeShareable( new FLocMetadataObject );
			CHECK_MESSAGE(TEXT("LocItemEmptyA == LocItemEmptyB"), LocItemEmptyA == LocItemEmptyB );
		}

		// Testing less than operator
		{
			CHECK_MESSAGE(TEXT("LocItemA < LocItemB"), LocItemA < LocItemB );
			CHECK_FALSE_MESSAGE(TEXT("LocItemB < LocItemA"), LocItemB < LocItemA );

			FLocItem LocItemAClone = LocItemA;

			// Differences in Text
			CHECK_FALSE_MESSAGE(TEXT("LocItemA < LocItemAClone"), LocItemA < LocItemAClone );
			LocItemAClone.Text = LocItemAClone.Text + TEXT("A");
			CHECK_FALSE_MESSAGE(TEXT("LocItemAClone < LocItemA"), LocItemAClone < LocItemA );
			//currently failing CHECK_MESSAGE( TEXT("LocItemA < LocItemAClone"), LocItemA < LocItemAClone );

			// Adding new key metadata entry
			LocItemAClone = LocItemA;
			LocItemAClone.MetadataObj->SetStringField( TEXT("ANewKey"), TEXT("ANewValue") );
			//currently failing CHECK_MESSAGE( TEXT("LocItemA < LocItemAClone"), LocItemA < LocItemAClone );
			CHECK_FALSE_MESSAGE(TEXT("LocItemAClone < LocItemA"), LocItemAClone < LocItemA );

			// Removing a key metadata entry
			LocItemAClone = LocItemA;
			LocItemAClone.MetadataObj->RemoveField( TEXT("*IsMature") );
			CHECK_MESSAGE(TEXT("LocItemAClone < LocItemA"), LocItemAClone < LocItemA );
			CHECK_FALSE_MESSAGE(TEXT("LocItemA < LocItemAClone"), LocItemA < LocItemAClone );

			// Changing a key metadata entry value
			LocItemAClone = LocItemA;
			LocItemAClone.MetadataObj->SetBoolField( TEXT("*IsMature"),true );
			//currently failing CHECK_MESSAGE( TEXT("LocItemA < LocItemAClone"), LocItemA < LocItemAClone );
			CHECK_FALSE_MESSAGE(TEXT("LocItemAClone < LocItemA"), LocItemAClone < LocItemA );

			// Test null and non-null but empty Metadata
			FLocItem LocItemEmptyA(TEXT("SameText") );
			FLocItem LocItemEmptyB(TEXT("SameText") );
			CHECK_FALSE_MESSAGE(TEXT("LocItemEmptyA < LocItemEmptyB"), LocItemEmptyA < LocItemEmptyB );
			LocItemEmptyB.MetadataObj = MakeShareable( new FLocMetadataObject );
			CHECK_FALSE_MESSAGE(TEXT("LocItemEmptyA < LocItemEmptyB"), LocItemEmptyA < LocItemEmptyB );
			CHECK_FALSE_MESSAGE(TEXT("LocItemEmptyB < LocItemEmptyA"), LocItemEmptyB < LocItemEmptyA );
			LocItemEmptyB.MetadataObj->SetStringField(TEXT("AMetadataKey"), TEXT("AMetadataValue"));
			CHECK_MESSAGE(TEXT("LocItemEmptyA < LocItemEmptyB"), LocItemEmptyA < LocItemEmptyB );
		}
	}
}

TEST_CASE_NAMED(FManifestTest, "System::Core::Misc::Internationalization Manifest", "[EditorContext][ClientContext][SmokeFilter]")
{
	// Key metadata
	TSharedPtr< FLocMetadataObject > KeyMetadataA = MakeShareable( new FLocMetadataObject );
	TSharedPtr< FLocMetadataObject > KeyMetadataB = MakeShareable( new FLocMetadataObject );

	// Info metadata
	TSharedPtr< FLocMetadataObject > InfoMetadataA = MakeShareable( new FLocMetadataObject );
	TSharedPtr< FLocMetadataObject > InfoMetadataB = MakeShareable( new FLocMetadataObject );

	// Source metadata
	TSharedPtr< FLocMetadataObject > SourceMetadataA = MakeShareable( new FLocMetadataObject );
	TSharedPtr< FLocMetadataObject > SourceMetadataB = MakeShareable( new FLocMetadataObject );


	// Setup KeyMetadataA
	KeyMetadataA->SetStringField( TEXT("Gender"			),	TEXT("Masculine")	);
	KeyMetadataA->SetStringField( TEXT("Plurality"		),	TEXT("Singular")	);
	KeyMetadataA->SetStringField( TEXT("TargetGender"	),	TEXT("Masculine")	);
	KeyMetadataA->SetStringField( TEXT("TargetPlurality"),	TEXT("Singular")	);

	// Setup KeyMetadataB
	KeyMetadataB->SetStringField( TEXT("Gender"			),	TEXT("Masculine")	);
	KeyMetadataB->SetStringField( TEXT("Plurality"		),	TEXT("Singular")	);
	KeyMetadataB->SetStringField( TEXT("TargetGender"	),	TEXT("Feminine")	);
	KeyMetadataB->SetStringField( TEXT("TargetPlurality"),	TEXT("Singular")	);

	// Setup source metadata
	SourceMetadataA->SetBoolField( TEXT("*IsMature"), false );
	SourceMetadataB->SetBoolField( TEXT("*IsMature"), true );

	// Set InfoMetadataA
	InfoMetadataA->SetStringField( TEXT("VoiceActorDirection"), TEXT("Go big or go home!") );

	// Test FInternationalizationManifest
	{
		FManifestContext ContextA;
		ContextA.Key			= TEXT("KeyA");
		ContextA.SourceLocation = TEXT("SourceLocationA");
		ContextA.InfoMetadataObj = MakeShareable( new FLocMetadataObject(*InfoMetadataA) );
		ContextA.KeyMetadataObj = MakeShareable( new FLocMetadataObject(*KeyMetadataA) );

		FManifestContext ContextB;
		ContextB.Key			= TEXT("KeyB");
		ContextB.SourceLocation = TEXT("SourceLocationB");
		ContextB.InfoMetadataObj = MakeShareable( new FLocMetadataObject(*InfoMetadataB) );
		ContextB.KeyMetadataObj = MakeShareable( new FLocMetadataObject(*KeyMetadataB) );

		FLocItem Source( TEXT("TestText") );
		Source.MetadataObj = MakeShareable( new FLocMetadataObject(*SourceMetadataA) );

		FString TestNamespace = "TestNamespace";


		// Adding entries with exactly matching Source and matching Context
		{
			FInternationalizationManifest TestManifest;

			TestManifest.AddSource( TestNamespace, Source, ContextA );
			bool bResult = TestManifest.AddSource( TestNamespace, Source, ContextA );

			// Adding a duplicate entry reports success but our entry count is not increased after the first entry is added
			CHECK_MESSAGE(TEXT("AddSource result == true"), bResult);
			CHECK_MESSAGE(TEXT("ManifestCount == 1"), CountManifestEntries(TestManifest) == 1 );
		}

		// Adding entries with exactly matching Source but different Contexts
		{
			FInternationalizationManifest TestManifest;

			TestManifest.AddSource( TestNamespace, Source, ContextA );
			TestManifest.AddSource( TestNamespace, Source, ContextB );

			CHECK_MESSAGE(TEXT("ManifestCount == 2"), CountManifestEntries(TestManifest) == 2 );

			// Test find by context
			TSharedPtr< FManifestEntry > FoundEntry1 = TestManifest.FindEntryByContext( TestNamespace, ContextA );
			if (!FoundEntry1.IsValid())
			{
				FAIL_CHECK(TEXT("FManifestEntry could not find entry using FindEntryByContext."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry1->Source == Source);
				CHECK_MESSAGE(TEXT("FoundEntry->Context.Num() == 2"), FoundEntry1->Contexts.Num() == 2);
			}

			TSharedPtr< FManifestEntry > FoundEntry2 = TestManifest.FindEntryByContext( TestNamespace, ContextB );
			if (!FoundEntry2.IsValid())
			{
				FAIL_CHECK(TEXT("FManifestEntry could not find entry using FindEntryByContext."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry2->Source == Source);
				CHECK_MESSAGE(TEXT("FoundEntry->Context.Num() == 2"), FoundEntry2->Contexts.Num() == 2);
			}

			// Test find by source
			TSharedPtr< FManifestEntry > FoundEntry3 = TestManifest.FindEntryBySource( TestNamespace, Source );
			if (!FoundEntry3.IsValid())
			{
				FAIL_CHECK(TEXT("FManifestEntry could not find entry using FindEntryBySource."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry3->Source == Source);
				CHECK_MESSAGE(TEXT("FoundEntry->Context.Num() == 2"), FoundEntry3->Contexts.Num() == 2);
			}

			CHECK_MESSAGE(TEXT("FoundEntry1 == FoundEntry2 == FoundEntry3"), (FoundEntry1 == FoundEntry2) && (FoundEntry1 == FoundEntry3) );

		}

		// Adding entries with Source that is NOT an exact match and matching context
		{
			// Source mismatched by Source Text.
			{
				FInternationalizationManifest TestManifest;

				FLocItem ConflictingSourceA( TEXT("Conflicting TestTextA") );
				FLocItem ConflictingSourceB( TEXT("Conflicting TestTextB") );

				TestManifest.AddSource( TestNamespace, ConflictingSourceA, ContextA );
				bool bResult = TestManifest.AddSource( TestNamespace, ConflictingSourceB, ContextA );

				// Adding the second entry reports failure and entry count is not increased
				CHECK_FALSE_MESSAGE(TEXT("AddSource result == false"), bResult);
				CHECK_MESSAGE(TEXT("ManifestCount == 1"), CountManifestEntries(TestManifest) == 1 );	
			}

			// Source mismatched by standard metadata type (not * prefixed) 
			{
				FInternationalizationManifest ManifestTypeConflict;

				FLocItem ConflictingSourceMetadataTypeA = Source;
				FLocItem ConflictingSourceMetadataTypeB = Source;

				// Set metadata with the same name but different type
				ConflictingSourceMetadataTypeA.MetadataObj->SetBoolField( TEXT("ConflictingType"), true );
				ConflictingSourceMetadataTypeB.MetadataObj->SetStringField( TEXT("ConflictingType"), TEXT("true") );

				ManifestTypeConflict.AddSource( TestNamespace, ConflictingSourceMetadataTypeA, ContextA );
				bool bResult = ManifestTypeConflict.AddSource( TestNamespace, ConflictingSourceMetadataTypeB, ContextA );

				CHECK_FALSE_MESSAGE(TEXT("AddSource result == false"), bResult);
				CHECK_MESSAGE(TEXT("ManifestCount == 1"), CountManifestEntries(ManifestTypeConflict) == 1 );

				// Should not find a match when searching with the second Source we tried to add
				TSharedPtr< FManifestEntry > FoundEntry = ManifestTypeConflict.FindEntryBySource( TestNamespace, ConflictingSourceMetadataTypeB );
				CHECK_FALSE_MESSAGE(TEXT("FoundEntry is not valid"), FoundEntry.IsValid());

			}

			// Source mismatched by standard metadata value (not * prefixed)
			{
				FInternationalizationManifest TestManifest;

				FLocItem ConflictingSourceMetadataValueA = Source;
				FLocItem ConflictingSourceMetadataValueB = Source;

				// Set metadata with the same name and type but different value
				ConflictingSourceMetadataValueA.MetadataObj->SetStringField( TEXT("ConflictingValue"), TEXT("A") );
				ConflictingSourceMetadataValueB.MetadataObj->SetStringField( TEXT("ConflictingValue"), TEXT("B") );

				TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataValueA, ContextA );
				bool bResult = TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataValueB, ContextA );

				CHECK_FALSE_MESSAGE(TEXT("AddSource result == false"), bResult);
				CHECK_MESSAGE(TEXT("ManifestCount == 1"), CountManifestEntries(TestManifest) == 1 );

				// Should not find a match when searching with the second Source we tried to add
				TSharedPtr< FManifestEntry > FoundEntry = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataValueB );
				CHECK_FALSE_MESSAGE(TEXT("FoundEntry is not valid"), FoundEntry.IsValid());

			}

			// Source mismatched by * prefixed metadata type
			{
				FInternationalizationManifest TestManifest;

				FLocItem ConflictingSourceMetadataTypeA = Source;
				FLocItem ConflictingSourceMetadataTypeB = Source;

				// Set metadata with the same name but different type
				ConflictingSourceMetadataTypeA.MetadataObj->SetBoolField( TEXT("*ConflictingType"), true );
				ConflictingSourceMetadataTypeB.MetadataObj->SetStringField( TEXT("*ConflictingType"), TEXT("true") );

				TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataTypeA, ContextA );

				// Though the Source items are considered to be a match(they are equal) in this case, the manifest reports this as a conflict
				//  and should not add an entry.  The reason is that AddSource does an 'exact' match check on the metadata object
				bool bResult = TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataTypeB, ContextA );

				CHECK_FALSE_MESSAGE(TEXT("AddSource result == false"), bResult);
				CHECK_MESSAGE(TEXT("ManifestCount == 1"), CountManifestEntries(TestManifest) == 1 );

				// We should be able to find the entry using either Source FLocItem
				TSharedPtr< FManifestEntry > FoundEntry1 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataTypeA );
				CHECK_MESSAGE(TEXT("FoundEntry1 is valid"), FoundEntry1.IsValid());

				TSharedPtr< FManifestEntry > FoundEntry2 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataTypeB );
				CHECK_MESSAGE(TEXT("FoundEntry2 is valid"), FoundEntry2.IsValid());

			}

			// Source mismatched by * prefixed metadata value
			{
				FInternationalizationManifest TestManifest;

				FLocItem ConflictingSourceMetadataValueA = Source;
				FLocItem ConflictingSourceMetadataValueB = Source;

				// Set metadata with the same name and type but different value
				ConflictingSourceMetadataValueA.MetadataObj->SetStringField( TEXT("*ConflictingValue"), TEXT("A") );
				ConflictingSourceMetadataValueB.MetadataObj->SetStringField( TEXT("*ConflictingValue"), TEXT("B") );

				TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataValueA, ContextA );

				// Thought the Source items are considered to be a match(they are equal) in this case, the manifest reports this as a conflict
				//  and should not add an entry.  The reason is that AddSource does an 'exact' match check on the metadata object
				bool bResult = TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataValueB, ContextA );

				CHECK_FALSE_MESSAGE(TEXT("AddSource result == false"), bResult);
				CHECK_MESSAGE(TEXT("ManifestCount == 1"), CountManifestEntries(TestManifest) == 1 );

				// We should be able to find the entry using either Source FLocItem
				TSharedPtr< FManifestEntry > FoundEntry1 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataValueA );
				CHECK_MESSAGE(TEXT("FoundEntry1 is valid"), FoundEntry1.IsValid());

				TSharedPtr< FManifestEntry > FoundEntry2 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataValueB );
				CHECK_MESSAGE(TEXT("FoundEntry2 is valid"), FoundEntry2.IsValid());

			}

		}

		// Adding entries with Source that is NOT an exact match and different context
		{
			// Source mismatched by Source Text.
			{
				FInternationalizationManifest TestManifest;

				FLocItem ConflictingSourceA( TEXT("Conflicting TestTextA") );
				FLocItem ConflictingSourceB( TEXT("Conflicting TestTextB") );

				TestManifest.AddSource( TestNamespace, ConflictingSourceA, ContextA );
				bool bResult = TestManifest.AddSource( TestNamespace, ConflictingSourceB, ContextB );

				CHECK_MESSAGE(TEXT("AddSource result == true"), bResult);
				CHECK_MESSAGE(TEXT("ManifestCount == 2"), CountManifestEntries(TestManifest) == 2 );

				// We should be able to find two unique entries by source
				TSharedPtr< FManifestEntry > FoundEntry1 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceA );
				CHECK_MESSAGE(TEXT("FoundEntry1 is valid"), FoundEntry1.IsValid());

				TSharedPtr< FManifestEntry > FoundEntry2 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceB );
				CHECK_MESSAGE(TEXT("FoundEntry2 is valid"), FoundEntry2.IsValid());

				CHECK_MESSAGE(TEXT("FoundEntry1 != FoundEntry2"), FoundEntry1 != FoundEntry2 );

				// We should be able to find two unique entries by context
				TSharedPtr< FManifestEntry > FoundEntry3 = TestManifest.FindEntryByContext( TestNamespace, ContextA );
				CHECK_MESSAGE(TEXT("FoundEntry3 is valid"), FoundEntry3.IsValid());

				TSharedPtr< FManifestEntry > FoundEntry4 = TestManifest.FindEntryByContext( TestNamespace, ContextB );
				CHECK_MESSAGE(TEXT("FoundEntry3 is valid"), FoundEntry4.IsValid());

				CHECK_MESSAGE(TEXT("FoundEntry3 != FoundEntry4"), FoundEntry3 != FoundEntry4 );

				// Found entry looked up by source A should match entry looked up by context A
				CHECK_MESSAGE(TEXT("FoundEntry1 == FoundEntry3"), FoundEntry1 == FoundEntry3 );

				// Found entry looked up by source B should match entry looked up by context B
				CHECK_MESSAGE(TEXT("FoundEntry2 == FoundEntry4"), FoundEntry2 == FoundEntry4 );

			}

			// Source mismatched by standard metadata type (not * prefixed)
			{
				FInternationalizationManifest TestManifest;

				FLocItem ConflictingSourceMetadataTypeA = Source;
				FLocItem ConflictingSourceMetadataTypeB = Source;

				// Set metadata with the same name but different type
				ConflictingSourceMetadataTypeA.MetadataObj->SetBoolField( TEXT("ConflictingType"), true );
				ConflictingSourceMetadataTypeB.MetadataObj->SetStringField( TEXT("ConflictingType"), TEXT("true") );

				TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataTypeA, ContextA );
				bool bResult = TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataTypeB, ContextB );

				CHECK_MESSAGE(TEXT("AddSource result == true"), bResult);
				CHECK_MESSAGE(TEXT("ManifestCount == 2"), CountManifestEntries(TestManifest) == 2 );

				// We should be able to find two unique entries by source
				TSharedPtr< FManifestEntry > FoundEntry1 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataTypeA );
				CHECK_MESSAGE(TEXT("FoundEntry1 is valid"), FoundEntry1.IsValid());

				TSharedPtr< FManifestEntry > FoundEntry2 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataTypeB );
				CHECK_MESSAGE(TEXT("FoundEntry2 is valid"), FoundEntry2.IsValid());

				CHECK_MESSAGE(TEXT("FoundEntry1 != FoundEntry2"), FoundEntry1 != FoundEntry2 );

				// We should be able to find two unique entries by context
				TSharedPtr< FManifestEntry > FoundEntry3 = TestManifest.FindEntryByContext( TestNamespace, ContextA );
				CHECK_MESSAGE(TEXT("FoundEntry3 is valid"), FoundEntry3.IsValid());

				TSharedPtr< FManifestEntry > FoundEntry4 = TestManifest.FindEntryByContext( TestNamespace, ContextB );
				CHECK_MESSAGE(TEXT("FoundEntry3 is valid"), FoundEntry4.IsValid());

				CHECK_MESSAGE(TEXT("FoundEntry3 != FoundEntry4"), FoundEntry3 != FoundEntry4 );

				// Found entry looked up by source A should match entry looked up by context A
				CHECK_MESSAGE(TEXT("FoundEntry1 == FoundEntry3"), FoundEntry1 == FoundEntry3 );

				// Found entry looked up by source B should match entry looked up by context B
				CHECK_MESSAGE(TEXT("FoundEntry2 == FoundEntry4"), FoundEntry2 == FoundEntry4 );

			}

			// Source mismatched by standard metadata value (not * prefixed)
			{
				FInternationalizationManifest TestManifest;

				FLocItem ConflictingSourceMetadataValueA = Source;
				FLocItem ConflictingSourceMetadataValueB = Source;

				// Set metadata with the same name and type but different value
				ConflictingSourceMetadataValueA.MetadataObj->SetStringField( TEXT("ConflictingValue"), TEXT("A") );
				ConflictingSourceMetadataValueB.MetadataObj->SetStringField( TEXT("ConflictingValue"), TEXT("B") );

				TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataValueA, ContextA );
				bool bResult = TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataValueB, ContextB );

				CHECK_MESSAGE(TEXT("AddSource result == true"), bResult);
				CHECK_MESSAGE(TEXT("ManifestCount == 2"), CountManifestEntries(TestManifest) == 2 );

				// We should be able to find two unique entries by source
				TSharedPtr< FManifestEntry > FoundEntry1 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataValueA );
				CHECK_MESSAGE(TEXT("FoundEntry1 is valid"), FoundEntry1.IsValid());

				TSharedPtr< FManifestEntry > FoundEntry2 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataValueB );
				CHECK_MESSAGE(TEXT("FoundEntry2 is valid"), FoundEntry2.IsValid());

				CHECK_MESSAGE(TEXT("FoundEntry1 != FoundEntry2"), FoundEntry1 != FoundEntry2 );

				// We should be able to find two unique entries by context
				TSharedPtr< FManifestEntry > FoundEntry3 = TestManifest.FindEntryByContext( TestNamespace, ContextA );
				CHECK_MESSAGE(TEXT("FoundEntry3 is valid"), FoundEntry3.IsValid());

				TSharedPtr< FManifestEntry > FoundEntry4 = TestManifest.FindEntryByContext( TestNamespace, ContextB );
				CHECK_MESSAGE(TEXT("FoundEntry3 is valid"), FoundEntry4.IsValid());

				CHECK_MESSAGE(TEXT("FoundEntry3 != FoundEntry4"), FoundEntry3 != FoundEntry4 );

				// Found entry looked up by source A should match entry looked up by context A
				CHECK_MESSAGE(TEXT("FoundEntry1 == FoundEntry3"), FoundEntry1 == FoundEntry3 );

				// Found entry looked up by source B should match entry looked up by context B
				CHECK_MESSAGE(TEXT("FoundEntry2 == FoundEntry4"), FoundEntry2 == FoundEntry4 );

			}

			// Source mismatched by * prefixed metadata type
			{
				FInternationalizationManifest TestManifest;

				FLocItem ConflictingSourceMetadataTypeA = Source;
				FLocItem ConflictingSourceMetadataTypeB = Source;

				// Set metadata with the same name but different type
				ConflictingSourceMetadataTypeA.MetadataObj->SetBoolField( TEXT("*ConflictingType"), true );
				ConflictingSourceMetadataTypeB.MetadataObj->SetStringField( TEXT("*ConflictingType"), TEXT("true") );

				TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataTypeA, ContextA );

				// Though the Source items are considered to be a match(they are equal) in this case, the manifest reports this as a conflict
				//  and should not add an entry.  The reason is that AddSource does an 'exact' match check on the metadata object
				bool bResult = TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataTypeB, ContextB );

				CHECK_FALSE_MESSAGE(TEXT("AddSource result == false"), bResult);
				CHECK_MESSAGE(TEXT("ManifestCount == 1"), CountManifestEntries(TestManifest) == 1 );

				// We should be able to find the entry using either Source FLocItem
				TSharedPtr< FManifestEntry > FoundEntry1 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataTypeA );
				CHECK_MESSAGE(TEXT("FoundEntry1 is valid"), FoundEntry1.IsValid());

				TSharedPtr< FManifestEntry > FoundEntry2 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataTypeB );
				CHECK_MESSAGE(TEXT("FoundEntry2 is valid"), FoundEntry2.IsValid());
			}

			// Source mismatched by * prefixed metadata value
			{
				FInternationalizationManifest TestManifest;

				FLocItem ConflictingSourceMetadataValueA = Source;
				FLocItem ConflictingSourceMetadataValueB = Source;

				// Set metadata with the same name and type but different value
				ConflictingSourceMetadataValueA.MetadataObj->SetStringField( TEXT("*ConflictingValue"), TEXT("A") );
				ConflictingSourceMetadataValueB.MetadataObj->SetStringField( TEXT("*ConflictingValue"), TEXT("B") );

				TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataValueA, ContextA );

				// Thought the Source items are considered to be a match(they are equal) in this case, the manifest reports this as a conflict
				//  and should not add an entry.  The reason is that AddSource does an 'exact' match check on the metadata object
				bool bResult = TestManifest.AddSource( TestNamespace, ConflictingSourceMetadataValueB, ContextB );

				CHECK_FALSE_MESSAGE(TEXT("AddSource result == false"), bResult);
				CHECK_MESSAGE(TEXT("ManifestCount == 1"), CountManifestEntries(TestManifest) == 1 );

				// We should be able to find the entry using either Source FLocItem
				TSharedPtr< FManifestEntry > FoundEntry1 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataValueA );
				CHECK_MESSAGE(TEXT("FoundEntry1 is valid"), FoundEntry1.IsValid());

				TSharedPtr< FManifestEntry > FoundEntry2 = TestManifest.FindEntryBySource( TestNamespace, ConflictingSourceMetadataValueB );
				CHECK_MESSAGE(TEXT("FoundEntry2 is valid"), FoundEntry2.IsValid());

			}

		}

		// Adding an entry that only differs in the optional flag 
		{
			FInternationalizationManifest TestManifest;

			// Reports success and our entry count does not change.  bIsOptional is not a key and is not used during lookup.  When
			//  we AddSource, we find an existing matching entry so AddSource returns true but no new entry is added.  The original
			//  entry's bIsOptional value is not updated.
			FManifestContext ContextConflictingOptionalFlag = ContextA;
			ContextConflictingOptionalFlag.bIsOptional = !ContextA.bIsOptional;

			TestManifest.AddSource( TestNamespace, Source, ContextA );
			bool bResult = TestManifest.AddSource( TestNamespace, Source, ContextConflictingOptionalFlag);

			CHECK_MESSAGE(TEXT("AddSource result == true"), bResult);
			CHECK_MESSAGE(TEXT("ManifestCount == 1"), CountManifestEntries(TestManifest) == 1 );

			// We should be able to look up the existing entry using the ContextConflictingOptionalFlag context but the entries bIsOptional flag will match ContextA
			TSharedPtr< FManifestEntry > FoundEntry = TestManifest.FindEntryByContext( TestNamespace, ContextConflictingOptionalFlag );
			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FManifestEntry could not find entry using FindEntryByContext."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->bIsOptional == ContextA->bIsOptional"), FoundEntry->Contexts[0].bIsOptional == ContextA.bIsOptional);
			}
		}

		// Add an entry with null key metadata to see if we can retrieve it with non-null but empty key metadata
		{
			FInternationalizationManifest TestManifest;

			FManifestContext ContextC;
			ContextC.Key			= TEXT("KeyC");
			ContextC.SourceLocation = TEXT("SourceLocationC");
			ContextC.InfoMetadataObj = MakeShareable( new FLocMetadataObject(*InfoMetadataB) );
			ContextC.KeyMetadataObj = NULL;

			Source.MetadataObj = NULL;
			TestManifest.AddSource( TestNamespace, Source, ContextC );

			//  Now give our context and source valid but empty metadata
			ContextC.KeyMetadataObj = MakeShareable( new FLocMetadataObject() );
			Source.MetadataObj = MakeShareable( new FLocMetadataObject() );

			// Ensure we find the entry we added by source
			TSharedPtr< FManifestEntry > FoundEntry;
			FoundEntry = TestManifest.FindEntryBySource( TestNamespace, Source );
			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FManifestEntry could not find entry using FindEntryBySource."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry->Source == Source);
			}
			FoundEntry = TestManifest.FindEntryByContext(TestNamespace, ContextC);
			// Ensure we find the entry we added by context
			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FManifestEntry could not find entry using FindEntryByContext."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry->Source == Source);
			}
		}

		// Add an entry with non-null but empty key metadata and see if we can retrieve null metadata
		{
			FInternationalizationManifest TestManifest;

			FManifestContext ContextC;
			ContextC.Key			= TEXT("KeyC");
			ContextC.SourceLocation = TEXT("SourceLocationC");
			ContextC.InfoMetadataObj = MakeShareable( new FLocMetadataObject(*InfoMetadataB) );
			ContextC.KeyMetadataObj = MakeShareable( new FLocMetadataObject());

			Source.MetadataObj = MakeShareable( new FLocMetadataObject());
			TestManifest.AddSource( TestNamespace, Source, ContextC );

			//  Now give our context and source Null metadata
			ContextC.KeyMetadataObj = NULL;
			Source.MetadataObj = NULL;

			// Ensure we find the entry we added by source
			TSharedPtr< FManifestEntry > FoundEntry;
			FoundEntry = TestManifest.FindEntryBySource( TestNamespace, Source );
			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FManifestEntry could not find entry using FindEntryBySource."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry->Source == Source);
			}
			// Ensure we find the entry we added by context
			FoundEntry = TestManifest.FindEntryByContext( TestNamespace, ContextC );
			if (!FoundEntry.IsValid())
			{
				FAIL_CHECK(TEXT("FManifestEntry could not find entry using FindEntryByContext."));
			}
			else
			{
				CHECK_MESSAGE(TEXT("FoundEntry->Source == Source"), FoundEntry->Source == Source);
			}
		}
	}
}

#endif //WITH_TESTS