// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/InternationalizationMetadata.h"

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FMetadataTest, "System::Core::Misc::Internationalization Metadata", "[EditorContext][ClientContext][SmokeFilter]")
{
	// BooleanValue metadata
	bool BoolFalse = false;
	bool BoolTrue = true;
	TSharedPtr<FLocMetadataValue> MetadataValueBoolFalse = MakeShareable( new FLocMetadataValueBoolean( BoolFalse ) );
	TSharedPtr<FLocMetadataValue> MetadataValueBoolTrue = MakeShareable( new FLocMetadataValueBoolean( BoolTrue ) );

	// StringValue metadata
	FString StringA = "A";
	FString StringB = "B";
	TSharedPtr<FLocMetadataValue> MetadataValueStringA = MakeShareable( new FLocMetadataValueString( StringA ) );
	TSharedPtr<FLocMetadataValue> MetadataValueStringB = MakeShareable( new FLocMetadataValueString( StringB ) );

	// ArrayValue metadata
	TArray< TSharedPtr< FLocMetadataValue > > ArrayA;
	TArray< TSharedPtr< FLocMetadataValue > > ArrayB;

	ArrayA.Add( MetadataValueBoolFalse );
	ArrayA.Add( MetadataValueStringA );

	ArrayB.Add( MetadataValueBoolTrue );
	ArrayB.Add( MetadataValueStringB );

	TSharedPtr<FLocMetadataValue> MetadataValueArrayA = MakeShareable( new FLocMetadataValueArray( ArrayA ) );
	TSharedPtr<FLocMetadataValue> MetadataValueArrayB = MakeShareable( new FLocMetadataValueArray( ArrayB ) );

	// Object metadata
	TSharedPtr< FLocMetadataObject > MetadataObjectA = MakeShareable( new FLocMetadataObject );
	TSharedPtr< FLocMetadataObject > MetadataObjectB = MakeShareable( new FLocMetadataObject );

	// Setup object A
	MetadataObjectA->SetField( TEXT("MetadataBoolFalse"), MetadataValueBoolFalse );
	MetadataObjectA->SetField( TEXT("MetadataStringA"), MetadataValueStringA );
	MetadataObjectA->SetField( TEXT("MetadataArrayA"), MetadataValueArrayA );
	MetadataObjectA->SetField( TEXT("*MetadataCompareModifier"), MetadataValueStringA);// Note: The * name prefix modifies the way entries in the object are compared.

	// Setup object B
	MetadataObjectB->SetField( TEXT("MetadataBoolFalse"), MetadataValueBoolTrue );
	MetadataObjectB->SetField( TEXT("MetadataStringB"), MetadataValueStringB );
	MetadataObjectB->SetField( TEXT("MetadataArrayB"), MetadataValueArrayB );
	MetadataObjectA->SetBoolField( TEXT("*MetadataCompareModifier"), true);// Note: Different type/value. The * name prefix modifies the way entries in the object are compared.

	// ObjectValue metadata
	TSharedPtr< FLocMetadataValue > MetadataValueObjectA = MakeShareable( new FLocMetadataValueObject( MetadataObjectA ) );
	TSharedPtr< FLocMetadataValue > MetadataValueObjectB = MakeShareable( new FLocMetadataValueObject( MetadataObjectB ) );

	// Testing the bool meta data value type
	{
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueBoolFalse == MetadataValueBoolTrue"), *MetadataValueBoolFalse == *MetadataValueBoolTrue );
		CHECK_MESSAGE(TEXT("MetadataValueBoolFalse < MetadataValueBoolTrue"), *MetadataValueBoolFalse < *MetadataValueBoolTrue );
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueBoolTrue < MetadataValueBoolFalse"), *MetadataValueBoolTrue < *MetadataValueBoolFalse);

		CHECK_MESSAGE(TEXT("MetadataValueBoolFalse < MetadataValueString"), *MetadataValueBoolFalse < *MetadataValueStringA );
		CHECK_MESSAGE(TEXT("MetadataValueBoolTrue < MetadataValueString"), *MetadataValueBoolTrue < *MetadataValueStringA);

		CHECK_MESSAGE(TEXT("MetadataValueBoolFalse < MetadataValueArray"), *MetadataValueBoolFalse < *MetadataValueArrayA );
		CHECK_MESSAGE(TEXT("MetadataValueBoolTrue < MetadataValueArray"), *MetadataValueBoolTrue < *MetadataValueArrayA);

		CHECK_MESSAGE(TEXT("MetadataValueBoolFalse < MetadataValueObject"), *MetadataValueBoolFalse < *MetadataValueObjectA );
		CHECK_MESSAGE(TEXT("MetadataValueBoolTrue < MetadataValueObject"), *MetadataValueBoolTrue < *MetadataValueObjectA);

		TSharedPtr<FLocMetadataValue> MetadataValueBoolFalseClone = MetadataValueBoolFalse->Clone();
		TSharedPtr<FLocMetadataValue> MetadataValueBoolTrueClone = MetadataValueBoolTrue->Clone();

		if (MetadataValueBoolFalse == MetadataValueBoolFalseClone)
		{
			FAIL_CHECK(TEXT("MetadataValueBool and its Clone are not unique objects."));
		}
		CHECK_MESSAGE(TEXT("MetadataValueBoolFalseClone == MetadataValueBoolFalse"), *MetadataValueBoolFalseClone == *MetadataValueBoolFalse );
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueBoolFalseClone < MetadataValueBoolFalse"), *MetadataValueBoolFalseClone < *MetadataValueBoolFalse );

		CHECK_MESSAGE(TEXT("MetadataValueBoolTrueClone == MetadataValueBoolTrue"), *MetadataValueBoolTrueClone == *MetadataValueBoolTrue );
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueBoolTrueClone < MetadataValueBoolTrue"), *MetadataValueBoolTrueClone < *MetadataValueBoolTrue );

		// Test the bool metadata when it is part of an object
		TSharedPtr< FLocMetadataObject > MetadataObjectFalse = MakeShareable( new FLocMetadataObject );
		MetadataObjectFalse->SetField(TEXT("MetadataValueBool"), MetadataValueBoolFalse);

		TSharedPtr< FLocMetadataObject > MetadataObjectTrue = MakeShareable( new FLocMetadataObject );
		MetadataObjectTrue->SetField(TEXT("MetadataValueBool"), MetadataValueBoolTrue);

		CHECK_FALSE_MESSAGE(TEXT("GetBoolField(MetadataValueBool)"), MetadataObjectFalse->GetBoolField( TEXT("MetadataValueBool") ) );
		CHECK_MESSAGE(TEXT("GetBoolField(MetadataValueBool)"), MetadataObjectTrue->GetBoolField( TEXT("MetadataValueBool") ) );

		CHECK_FALSE_MESSAGE(TEXT("MetadataObjectFalse == MetadataObjectTrue"), *MetadataObjectFalse == *MetadataObjectTrue );
		CHECK_MESSAGE(TEXT("MetadataObjectFalse < MetadataObjectTrue"), *MetadataObjectFalse < *MetadataObjectTrue );
	}

	// Testing string metadata value type
	{
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueStringA == MetadataValueStringB"), *MetadataValueStringA == *MetadataValueStringB );
		CHECK_MESSAGE(TEXT("MetadataValueStringA < MetadataValueStringB"), *MetadataValueStringA < *MetadataValueStringB );
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueStringB < MetadataValueStringA"), *MetadataValueStringB < *MetadataValueStringA);

		CHECK_MESSAGE(TEXT("MetadataValueString < MetadataValueArray"), *MetadataValueStringA < *MetadataValueArrayA );

		CHECK_MESSAGE(TEXT("MetadataValueStringA < MetadataValueObject"), *MetadataValueStringA < *MetadataValueObjectA );


		TSharedPtr<FLocMetadataValue> MetadataValueStringAClone = MetadataValueStringA->Clone();

		if (MetadataValueStringA == MetadataValueStringAClone)
		{
			FAIL_CHECK(TEXT("MetadataValueString and its Clone are not unique objects."));
		}
		CHECK_MESSAGE(TEXT("MetadataValueStringAClone == MetadataValueStringA"), *MetadataValueStringAClone == *MetadataValueStringA );
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueStringAClone < MetadataValueStringA"), *MetadataValueStringAClone < *MetadataValueStringA );
		CHECK_MESSAGE(TEXT("MetadataValueStringAClone < MetadataValueStringB"), *MetadataValueStringAClone < *MetadataValueStringB );

		// Test the string metadata when it is part of an object
		TSharedPtr< FLocMetadataObject > TestMetadataObjectA = MakeShareable( new FLocMetadataObject );
		TestMetadataObjectA->SetField(TEXT("MetadataValueString"), MetadataValueStringA);

		TSharedPtr< FLocMetadataObject > TestMetadataObjectB = MakeShareable( new FLocMetadataObject );
		TestMetadataObjectB->SetField(TEXT("MetadataValueString"), MetadataValueStringB);

		CHECK_MESSAGE(TEXT("GetStringField(MetadataValueString) == StringA"), TestMetadataObjectA->GetStringField( TEXT("MetadataValueString") ) == StringA );
	
		CHECK_FALSE_MESSAGE(TEXT("TestMetadataObjectA == TestMetadataObjectB"), *TestMetadataObjectA == *TestMetadataObjectB );
		CHECK_MESSAGE(TEXT("TestMetadataObjectA < TestMetadataObjectB"), *TestMetadataObjectA < *TestMetadataObjectB );
	}

	// Testing array metadata value type
	{
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueArrayA == MetadataValueArrayB"), *MetadataValueArrayA == *MetadataValueArrayB );
		CHECK_MESSAGE(TEXT("MetadataValueArrayA < MetadataValueArrayB"), *MetadataValueArrayA < *MetadataValueArrayB );
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueArrayB < MetadataValueArrayA"), *MetadataValueArrayB < *MetadataValueArrayA);

		CHECK_MESSAGE(TEXT("MetadataValueArrayA < MetadataValueObject"), *MetadataValueArrayA < *MetadataValueObjectA );

		TSharedPtr<FLocMetadataValue> MetadataValueArrayAClone = MetadataValueArrayA->Clone();

		if (MetadataValueArrayA == MetadataValueArrayAClone)
		{
			FAIL_CHECK(TEXT("MetadataValueString and its Clone are not unique objects."));
		}
		CHECK_MESSAGE(TEXT("MetadataValueArrayAClone == MetadataValueArrayA"), *MetadataValueArrayAClone == *MetadataValueArrayA );
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueArrayAClone < MetadataValueArrayA"), *MetadataValueArrayAClone < *MetadataValueArrayA );
		CHECK_MESSAGE(TEXT("MetadataValueArrayAClone < MetadataValueArrayB"), *MetadataValueArrayAClone < *MetadataValueArrayB );

		// Test Less than and equality checks.  Metadata arrays are equivalent if they contain equivalent contents in any order.  
		//  To calculate if a metadata array is less than another, we sort both arrays and check each entry index against its counterpart.
		//  If we encounter an entry that is less than another we stop looking.
		TArray< TSharedPtr< FLocMetadataValue > > ArrayC;
		ArrayC.Add( MetadataValueBoolFalse );
		ArrayC.Add( MetadataValueBoolFalse->Clone() );
		TSharedPtr<FLocMetadataValue> MetadataValueArrayC = MakeShareable( new FLocMetadataValueArray( ArrayC ) );
	
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueArrayA == MetadataValueArrayC"), *MetadataValueArrayA == *MetadataValueArrayC );
		CHECK_MESSAGE(TEXT("MetadataValueArrayC < MetadataValueArrayA"), *MetadataValueArrayC < *MetadataValueArrayA );
		CHECK_MESSAGE(TEXT("MetadataValueArrayC < MetadataValueArrayB"), *MetadataValueArrayC < *MetadataValueArrayB );

		TArray< TSharedPtr< FLocMetadataValue > > ArrayD;
		ArrayD.Add( MetadataValueBoolFalse );
		ArrayD.Add( MetadataValueBoolFalse->Clone() );
		ArrayD.Add( MetadataValueBoolFalse->Clone() );
		TSharedPtr<FLocMetadataValue> MetadataValueArrayD = MakeShareable( new FLocMetadataValueArray( ArrayD ) );

		CHECK_FALSE_MESSAGE(TEXT("MetadataValueArrayA == MetadataValueArrayD"), *MetadataValueArrayA == *MetadataValueArrayD );
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueArrayC == MetadataValueArrayD"), *MetadataValueArrayC == *MetadataValueArrayD );
		CHECK_MESSAGE(TEXT("MetadataValueArrayC < MetadataValueArrayD"), *MetadataValueArrayC < *MetadataValueArrayD );
		CHECK_MESSAGE(TEXT("MetadataValueArrayD < MetadataValueArrayA"), *MetadataValueArrayD < *MetadataValueArrayA );

		// Test the array metadata when it is part of an object
		TSharedPtr< FLocMetadataObject > TestMetadataObjectA = MakeShareable( new FLocMetadataObject );
		TestMetadataObjectA->SetField(TEXT("MetadataValueArray"), MetadataValueArrayA);

		TSharedPtr< FLocMetadataObject > TestMetadataObjectB = MakeShareable( new FLocMetadataObject );
		TestMetadataObjectB->SetField(TEXT("MetadataValueArray"), MetadataValueArrayB);

		CHECK_MESSAGE(TEXT("GetArrayField(MetadataValueArray) == ArrayA"), TestMetadataObjectA->GetArrayField( TEXT("MetadataValueArray") ) == ArrayA );

		CHECK_FALSE_MESSAGE(TEXT("TestMetadataObjectA == TestMetadataObjectB"), *TestMetadataObjectA == *TestMetadataObjectB );
		CHECK_MESSAGE(TEXT("TestMetadataObjectA < TestMetadataObjectB"), *TestMetadataObjectA < *TestMetadataObjectB );
	}

	// Testing object metadata value type
	{
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueObjectA == MetadataValueObjectB"), *MetadataValueObjectA == *MetadataValueObjectB );
		CHECK_MESSAGE(TEXT("MetadataValueObjectA < MetadataValueObjectB"), *MetadataValueObjectA < *MetadataValueObjectB );
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueObjectB < MetadataValueObjectA"), *MetadataValueObjectB < *MetadataValueObjectA);

		TSharedPtr<FLocMetadataValue> MetadataValueObjectAClone = MetadataValueObjectA->Clone();

		if (MetadataValueObjectA == MetadataValueObjectAClone)
		{
			FAIL_CHECK(TEXT("MetadataValueString and its Clone are not unique objects."));
		}

		CHECK_MESSAGE(TEXT("MetadataValueObjectAClone == MetadataValueObjectA"), *MetadataValueObjectAClone == *MetadataValueObjectA );
		CHECK_FALSE_MESSAGE(TEXT("MetadataValueObjectAClone < MetadataValueObjectA"), *MetadataValueObjectAClone < *MetadataValueObjectA );
		CHECK_MESSAGE(TEXT("MetadataValueObjectAClone < MetadataValueObjectB"), *MetadataValueObjectAClone < *MetadataValueObjectB );


		// Test the object metadata when it is part of another object
		TSharedPtr< FLocMetadataObject > TestMetadataObjectA = MakeShareable( new FLocMetadataObject );
		TestMetadataObjectA->SetField(TEXT("MetadataValueObject"), MetadataValueObjectA);

		TSharedPtr< FLocMetadataObject > TestMetadataObjectB = MakeShareable( new FLocMetadataObject );
		TestMetadataObjectB->SetField(TEXT("MetadataValueObject"), MetadataValueObjectB);

		CHECK_MESSAGE(TEXT("GetObjectField(MetadataValueObject) == MetadataObjectA"), *TestMetadataObjectA->GetObjectField( TEXT("MetadataValueObject") ) == *MetadataObjectA );

		CHECK_FALSE_MESSAGE(TEXT("TestMetadataObjectA == TestMetadataObjectB"), *TestMetadataObjectA == *TestMetadataObjectB );
		CHECK_MESSAGE(TEXT("TestMetadataObjectA < TestMetadataObjectB"), *TestMetadataObjectA < *TestMetadataObjectB );
	}

	// Testing Loc Metadata Object
	{
		CHECK_FALSE_MESSAGE(TEXT("MetadataObjectA == MetadataObjectB"), *MetadataObjectA == *MetadataObjectB );
		CHECK_MESSAGE(TEXT("MetadataObjectA < MetadataObjectB"), *MetadataObjectA < *MetadataObjectB );
		CHECK_FALSE_MESSAGE(TEXT("MetadataObjectB < MetadataObjectA"), *MetadataObjectB < *MetadataObjectA);

		// Test copy ctor
		{
			FLocMetadataObject MetadataObjectAClone = *MetadataObjectA;
			CHECK_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectA"), MetadataObjectAClone == *MetadataObjectA );
		}
		
		// Test assignment
		{
			FLocMetadataObject MetadataObjectAClone = *MetadataObjectB;
			// PVS-Studio complains about double initialization, but that's something we're testing
			// so we disable the warning
			MetadataObjectAClone = *MetadataObjectA; //-V519
			CHECK_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectA"), MetadataObjectAClone == *MetadataObjectA );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectB"), MetadataObjectAClone == *MetadataObjectB );
		}
		
		// Test comparison operator  
		{
			FLocMetadataObject MetadataObjectAClone = *MetadataObjectA;
			
			
			// Adding standard entry
			MetadataObjectAClone.SetStringField( TEXT("NewEntry"), TEXT("NewEntryValue") );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectA"), MetadataObjectAClone == *MetadataObjectA );
			
			// Adding non-standard entry.  Note metadata with * prefix in the name will ignore value and type when performing comparisons
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("*NewEntry"), TEXT("*NewEntryValue") );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectA"), MetadataObjectAClone == *MetadataObjectA );

			// Value mismatch on entry with * prefix with same type
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("*NoCompare"), TEXT("NoCompare") );
			FLocMetadataObject MetadataObjectAClone2 = *MetadataObjectA;
			MetadataObjectAClone2.SetStringField( TEXT("*NoCompare"), TEXT("NoCompare2") );
			CHECK_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectAClone2"), MetadataObjectAClone == MetadataObjectAClone2 );

			// Value and type mismatch on entry with * prefix
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("*NoCompare"), TEXT("NoCompare") );
			MetadataObjectAClone2 = *MetadataObjectA;
			MetadataObjectAClone2.SetBoolField( TEXT("*NoCompare"), true );
			CHECK_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectAClone2"), MetadataObjectAClone == MetadataObjectAClone2 );
		
			// Value mismatch on standard entry
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("DoCompare"), TEXT("DoCompare") );
			MetadataObjectAClone2 = *MetadataObjectA;
			MetadataObjectAClone2.SetStringField( TEXT("DoCompare"), TEXT("DoCompare2") );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectAClone2"), MetadataObjectAClone == MetadataObjectAClone2 );

			// Value and type mismatch on standard entry
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("DoCompare"), TEXT("DoCompare") );
			MetadataObjectAClone2 = *MetadataObjectA;
			MetadataObjectAClone2.SetBoolField( TEXT("DoCompare"), true );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectAClone2"), MetadataObjectAClone == MetadataObjectAClone2 );
		}		
		
		// Test IsExactMatch function.  Note: Differs from the == operator which takes into account the COMPARISON_MODIFIER_PREFIX
		{
			// This function behaves similar to the comparison operator except that it ensures all metadata entries match exactly.  In other
			//  words, it will also perform exact match checks on * prefixed metadata items.
			FLocMetadataObject MetadataObjectAClone = *MetadataObjectA;

			CHECK_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectA"), MetadataObjectAClone.IsExactMatch( *MetadataObjectA) );

			// Adding standard entry
			MetadataObjectAClone.SetStringField( TEXT("NewEntry"), TEXT("NewEntryValue") );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectA"), MetadataObjectAClone.IsExactMatch( *MetadataObjectA) );

			// Adding non-standard entry.  Note metadata with * prefix in the name will ignore value and type when performing comparisons
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("*NewEntry"), TEXT("*NewEntryValue") );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectA"),MetadataObjectAClone.IsExactMatch( *MetadataObjectA)  );

			// Value mismatch on entry with * prefix with same type
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("*NoCompare"), TEXT("NoCompare") );
			FLocMetadataObject MetadataObjectAClone2 = *MetadataObjectA;
			MetadataObjectAClone2.SetStringField( TEXT("*NoCompare"), TEXT("NoCompare2") );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectAClone2"), MetadataObjectAClone.IsExactMatch( MetadataObjectAClone2) );

			// Value and type mismatch on entry with * prefix
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("*NoCompare"), TEXT("NoCompare") );
			MetadataObjectAClone2 = *MetadataObjectA;
			MetadataObjectAClone2.SetBoolField( TEXT("*NoCompare"), true );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectAClone2"),  MetadataObjectAClone.IsExactMatch( MetadataObjectAClone2) );

			// Value mismatch on standard entry
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("DoCompare"), TEXT("DoCompare") );
			MetadataObjectAClone2 = *MetadataObjectA;
			MetadataObjectAClone2.SetStringField( TEXT("DoCompare"), TEXT("DoCompare2") );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectAClone2"),  MetadataObjectAClone.IsExactMatch( MetadataObjectAClone2) );

			// Value and type mismatch on standard entry
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("DoCompare"), TEXT("DoCompare") );
			MetadataObjectAClone2 = *MetadataObjectA;
			MetadataObjectAClone2.SetBoolField( TEXT("DoCompare"), true );
			CHECK_FALSE_MESSAGE(TEXT("MetadataObjectAClone == MetadataObjectAClone2"),  MetadataObjectAClone.IsExactMatch( MetadataObjectAClone2) );
		}

		// Test less than operator
		{
			
			// Adding standard entry that would appear before other entries
			FLocMetadataObject MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("ANewEntry"), TEXT("NewEntryValue") );
			CHECK_MESSAGE(TEXT("MetadataObjectAClone < MetadataObjectA"), MetadataObjectAClone < *MetadataObjectA );
			
			// Adding standard entry that would appear after other entries
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("ZNewEntry"), TEXT("NewEntryValue") );
			CHECK_MESSAGE(TEXT("MetadataObjectA < MetadataObjectAClone"), *MetadataObjectA < MetadataObjectAClone );

			// Adding non-standard entry that would appear before other entries.  Note metadata with * prefix has no special treatment in less than operator
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("*NewEntry"), TEXT("NewEntryValue") );
			CHECK_MESSAGE(TEXT("MetadataObjectAClone < MetadataObjectA"), MetadataObjectAClone < *MetadataObjectA );

			// Value mismatch on entry with * prefix with same type. Note metadata with * prefix has no special treatment in less than operator
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetStringField( TEXT("*NoCompare"), TEXT("NoCompare") );
			FLocMetadataObject MetadataObjectAClone2 = *MetadataObjectA;
			MetadataObjectAClone2.SetStringField( TEXT("*NoCompare"), TEXT("NoCompare2") );
			CHECK_MESSAGE(TEXT("MetadataObjectAClone < MetadataObjectAClone2"), MetadataObjectAClone < MetadataObjectAClone2 );

			// Value and type mismatch on entry with * prefix.  Note metadata with * prefix has no special treatment in less than operator
			MetadataObjectAClone = *MetadataObjectA;
			MetadataObjectAClone.SetBoolField( TEXT("*NoCompare"), true );
			MetadataObjectAClone2 = *MetadataObjectA;
			MetadataObjectAClone2.SetStringField( TEXT("*NoCompare"), TEXT("NoCompare") );
			CHECK_MESSAGE(TEXT("MetadataObjectAClone < MetadataObjectAClone2"), MetadataObjectAClone < MetadataObjectAClone2 );
		}
	}
}

#endif //WITH_TESTS