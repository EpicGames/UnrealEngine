// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Internationalization/Text.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

class FNumberFormattingRulesTestClass {
public:
	void Test(FNumberFormattingRulesTestClass* const ExplicitThis, const TCHAR* const Desc, const FText& A, const FText& B)
	{
		if (!A.EqualTo(B))
		{
			FAIL_CHECK(FString::Printf(TEXT("%s - A=%s B=%s"), Desc, *A.ToString(), *B.ToString()));
		}
	}

	void NumberFormattingRulesTest() {

		/**
		*	List of functions and types to test.
		*
		*	Function	Type
		*	---------	------
		*	AsNumber 	double
		*	AsNumber 	float
		*	AsNumber 	uint8
		*	AsNumber 	uint16
		*	AsNumber 	uint32
		*	AsNumber 	uint64
		*	AsNumber 	int8
		*	AsNumber 	int16
		*	AsNumber 	int32
		*	AsNumber 	int64
		*	AsCurrency 	double
		*	AsCurrency 	float
		*	AsCurrency 	uint8
		*	AsCurrency 	uint16
		*	AsCurrency 	uint32
		*	AsCurrency 	uint64
		*	AsCurrency 	int8
		*	AsCurrency 	int16
		*	AsCurrency 	int32
		*	AsCurrency 	int64
		*	AsPercent 	double
		*	AsPercent	float
		*/

#if UE_ENABLE_ICU
		double DoubleValue = 12345678.901;
		float FloatValue = 1234.567f;
		double DoubleNegativeValue = -12345678.901;
		float FloatNegativeValue = -1234.567f;
		uint8 Uint8Value = 0xFF;
		uint16 Uint16Value = 0xFFFF;
		uint32 Uint32Value = 0xFFFFFFFF;
		uint64 Uint64Value = 0x1999999999999999;
		int8 Int8Value = 123;
		int16 Int16Value = 12345;
		int32 Int32Value = 12345;
		int64 Int64Value = 12345;
		int8 Int8NegativeValue = -123;
		int16 Int16NegativeValue = -12345;
		int32 Int32NegativeValue = -12345;
		int64 Int64NegativeValue = -12345;
		int64 Int64MinValue = MIN_int64;

		FInternationalization& I18N = FInternationalization::Get();

		FInternationalization::FCultureStateSnapshot OriginalCultureState;
		I18N.BackupCultureState(OriginalCultureState);

		if (I18N.SetCurrentCulture("en-US"))
		{
			Test(this, TEXT("Convert a Double to a number formatted correct for en-US"), FText::AsNumber(DoubleValue), FText::FromString(TEXT("12,345,678.901")));
			Test(this, TEXT("Convert a Float to a number formatted correct for en-US"), FText::AsNumber(FloatValue), FText::FromString(TEXT("1,234.567")));
			Test(this, TEXT("Convert a Negative Double to a number formatted correct for en-US"), FText::AsNumber(DoubleNegativeValue), FText::FromString(TEXT("-12,345,678.901")));
			Test(this, TEXT("Convert a Negative Float to a number formatted correct for en-US"), FText::AsNumber(FloatNegativeValue), FText::FromString(TEXT("-1,234.567")));
			Test(this, TEXT("Convert a uint8 to a number formatted correct for en-US"), FText::AsNumber(Uint8Value), FText::FromString(TEXT("255")));
			Test(this, TEXT("Convert a uint16 to a number formatted correct for en-US"), FText::AsNumber(Uint16Value), FText::FromString(TEXT("65,535")));
			Test(this, TEXT("Convert a uint32 to a number formatted correct for en-US"), FText::AsNumber(Uint32Value), FText::FromString(TEXT("4,294,967,295")));
			Test(this, TEXT("Convert a uint64 to a number formatted correct for en-US"), FText::AsNumber(Uint64Value), FText::FromString(TEXT("1,844,674,407,370,955,161")));
			Test(this, TEXT("Convert a int8 to a number formatted correct for en-US"), FText::AsNumber(Int8Value), FText::FromString(TEXT("123")));
			Test(this, TEXT("Convert a int16 to a number formatted correct for en-US"), FText::AsNumber(Int16Value), FText::FromString(TEXT("12,345")));
			Test(this, TEXT("Convert a int32 to a number formatted correct for en-US"), FText::AsNumber(Int32Value), FText::FromString(TEXT("12,345")));
			Test(this, TEXT("Convert a int64 to a number formatted correct for en-US"), FText::AsNumber(Int64Value), FText::FromString(TEXT("12,345")));
			Test(this, TEXT("Convert a Negative int8 to a number formatted correct for en-US"), FText::AsNumber(Int8NegativeValue), FText::FromString(TEXT("-123")));
			Test(this, TEXT("Convert a Negative int16 to a number formatted correct for en-US"), FText::AsNumber(Int16NegativeValue), FText::FromString(TEXT("-12,345")));
			Test(this, TEXT("Convert a Negative int32 to a number formatted correct for en-US"), FText::AsNumber(Int32NegativeValue), FText::FromString(TEXT("-12,345")));
			Test(this, TEXT("Convert a Negative int64 to a number formatted correct for en-US"), FText::AsNumber(Int64NegativeValue), FText::FromString(TEXT("-12,345")));
			Test(this, TEXT("Convert a Negative int64 to a number formatted correct for en-US"), FText::AsNumber(Int64MinValue), FText::FromString(TEXT("-9,223,372,036,854,775,808")));

			// Test Number Formatting Options
			{
				FNumberFormattingOptions NumberFormattingOptions;
				NumberFormattingOptions.MinimumIntegralDigits = 20;
				NumberFormattingOptions.MaximumIntegralDigits = 20;
				NumberFormattingOptions.MinimumFractionalDigits = 3;
				NumberFormattingOptions.MaximumFractionalDigits = 3;
				NumberFormattingOptions.UseGrouping = false;

				Test(this, TEXT("Convert a Double to a number formatted correct for en-US"), FText::AsNumber(DoubleValue, &(NumberFormattingOptions)), FText::FromString(TEXT("00000000000012345678.901")));
				Test(this, TEXT("Convert a Float to a number formatted correct for en-US"), FText::AsNumber(FloatValue, &(NumberFormattingOptions)), FText::FromString(TEXT("00000000000000001234.567")));
				Test(this, TEXT("Convert a Negative Double to a number formatted correct for en-US"), FText::AsNumber(DoubleNegativeValue, &(NumberFormattingOptions)), FText::FromString(TEXT("-00000000000012345678.901")));
				Test(this, TEXT("Convert a Negative Float to a number formatted correct for en-US"), FText::AsNumber(FloatNegativeValue, &(NumberFormattingOptions)), FText::FromString(TEXT("-00000000000000001234.567")));
				Test(this, TEXT("Convert a uint8 to a number formatted correct for en-US"), FText::AsNumber(Uint8Value, &(NumberFormattingOptions)), FText::FromString(TEXT("00000000000000000255.000")));
				Test(this, TEXT("Convert a uint16 to a number formatted correct for en-US"), FText::AsNumber(Uint16Value, &(NumberFormattingOptions)), FText::FromString(TEXT("00000000000000065535.000")));
				Test(this, TEXT("Convert a uint32 to a number formatted correct for en-US"), FText::AsNumber(Uint32Value, &(NumberFormattingOptions)), FText::FromString(TEXT("00000000004294967295.000")));
				Test(this, TEXT("Convert a uint64 to a number formatted correct for en-US"), FText::AsNumber(Uint64Value, &(NumberFormattingOptions)), FText::FromString(TEXT("01844674407370955161.000")));
				Test(this, TEXT("Convert a int8 to a number formatted correct for en-US"), FText::AsNumber(Int8Value, &(NumberFormattingOptions)), FText::FromString(TEXT("00000000000000000123.000")));
				Test(this, TEXT("Convert a int16 to a number formatted correct for en-US"), FText::AsNumber(Int16Value, &(NumberFormattingOptions)), FText::FromString(TEXT("00000000000000012345.000")));
				Test(this, TEXT("Convert a int32 to a number formatted correct for en-US"), FText::AsNumber(Int32Value, &(NumberFormattingOptions)), FText::FromString(TEXT("00000000000000012345.000")));
				Test(this, TEXT("Convert a int64 to a number formatted correct for en-US"), FText::AsNumber(Int64Value, &(NumberFormattingOptions)), FText::FromString(TEXT("00000000000000012345.000")));
				Test(this, TEXT("Convert a Negative int8 to a number formatted correct for en-US"), FText::AsNumber(Int8NegativeValue, &(NumberFormattingOptions)), FText::FromString(TEXT("-00000000000000000123.000")));
				Test(this, TEXT("Convert a Negative int16 to a number formatted correct for en-US"), FText::AsNumber(Int16NegativeValue, &(NumberFormattingOptions)), FText::FromString(TEXT("-00000000000000012345.000")));
				Test(this, TEXT("Convert a Negative int32 to a number formatted correct for en-US"), FText::AsNumber(Int32NegativeValue, &(NumberFormattingOptions)), FText::FromString(TEXT("-00000000000000012345.000")));
				Test(this, TEXT("Convert a Negative int64 to a number formatted correct for en-US"), FText::AsNumber(Int64NegativeValue, &(NumberFormattingOptions)), FText::FromString(TEXT("-00000000000000012345.000")));
				Test(this, TEXT("Convert a Negative int64 to a number formatted correct for en-US"), FText::AsNumber(Int64MinValue, &(NumberFormattingOptions)), FText::FromString(TEXT("-09223372036854775808.000")));
			}

			{
				FNumberFormattingOptions NumberFormattingOptions;
				NumberFormattingOptions.MinimumIntegralDigits = 5;
				NumberFormattingOptions.MaximumIntegralDigits = 5;
				NumberFormattingOptions.UseGrouping = true;

				Test(this, TEXT("Convert a 5 digit int to 5 grouped integral digits formatted correct for en-US"), FText::AsNumber(12345, &NumberFormattingOptions), FText::FromString(TEXT("12,345")));
			}

			{
				FNumberFormattingOptions NumberFormattingOptions;
				NumberFormattingOptions.MinimumIntegralDigits = 6;
				NumberFormattingOptions.MaximumIntegralDigits = 6;
				NumberFormattingOptions.UseGrouping = true;

				Test(this, TEXT("Convert a 5 digit int to 6 grouped integral digits formatted correct for en-US"), FText::AsNumber(12345, &NumberFormattingOptions), FText::FromString(TEXT("012,345")));
			}

			{
				FNumberFormattingOptions NumberFormattingOptions;
				NumberFormattingOptions.MinimumIntegralDigits = 1;
				NumberFormattingOptions.MaximumIntegralDigits = 1;
				NumberFormattingOptions.MinimumFractionalDigits = 0;
				NumberFormattingOptions.MaximumFractionalDigits = 0;
				NumberFormattingOptions.UseGrouping = false;

				NumberFormattingOptions.RoundingMode = ERoundingMode::HalfToEven;

				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(-1.50, &(NumberFormattingOptions)), FText::FromString(TEXT("-2")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(-1.00, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(-0.75, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(-0.50, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(-0.25, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(-0.00, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(0.00, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(0.25, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(0.50, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(0.75, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(1.00, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToEven"), FText::AsNumber(1.50, &(NumberFormattingOptions)), FText::FromString(TEXT("2")));

				NumberFormattingOptions.RoundingMode = ERoundingMode::HalfFromZero;

				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfFromZero"), FText::AsNumber(-1.00, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfFromZero"), FText::AsNumber(-0.75, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfFromZero"), FText::AsNumber(-0.50, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfFromZero"), FText::AsNumber(-0.25, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfFromZero"), FText::AsNumber(-0.00, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfFromZero"), FText::AsNumber(0.00, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfFromZero"), FText::AsNumber(0.25, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfFromZero"), FText::AsNumber(0.50, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfFromZero"), FText::AsNumber(0.75, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfFromZero"), FText::AsNumber(1.00, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));

				NumberFormattingOptions.RoundingMode = ERoundingMode::HalfToZero;

				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToZero"), FText::AsNumber(-1.00, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToZero"), FText::AsNumber(-0.75, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToZero"), FText::AsNumber(-0.50, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToZero"), FText::AsNumber(-0.25, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToZero"), FText::AsNumber(-0.00, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToZero"), FText::AsNumber(0.00, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToZero"), FText::AsNumber(0.25, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToZero"), FText::AsNumber(0.50, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToZero"), FText::AsNumber(0.75, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using HalfToZero"), FText::AsNumber(1.00, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));

				NumberFormattingOptions.RoundingMode = ERoundingMode::FromZero;

				Test(this, TEXT("Round a Double to a number formatted correct for en-US using FromZero"), FText::AsNumber(-1.0, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using FromZero"), FText::AsNumber(-0.5, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using FromZero"), FText::AsNumber(-0.0, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using FromZero"), FText::AsNumber(0.0, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using FromZero"), FText::AsNumber(0.5, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using FromZero"), FText::AsNumber(1.0, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));

				NumberFormattingOptions.RoundingMode = ERoundingMode::ToZero;

				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToZero"), FText::AsNumber(-1.0, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToZero"), FText::AsNumber(-0.5, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToZero"), FText::AsNumber(-0.0, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToZero"), FText::AsNumber(0.0, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToZero"), FText::AsNumber(0.5, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToZero"), FText::AsNumber(1.0, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));

				NumberFormattingOptions.RoundingMode = ERoundingMode::ToNegativeInfinity;

				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToNegativeInfinity"), FText::AsNumber(-1.0, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToNegativeInfinity"), FText::AsNumber(-0.5, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToNegativeInfinity"), FText::AsNumber(-0.0, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToNegativeInfinity"), FText::AsNumber(0.0, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToNegativeInfinity"), FText::AsNumber(0.5, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToNegativeInfinity"), FText::AsNumber(1.0, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));

				NumberFormattingOptions.RoundingMode = ERoundingMode::ToPositiveInfinity;

				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToPositiveInfinity"), FText::AsNumber(-1.0, &(NumberFormattingOptions)), FText::FromString(TEXT("-1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToPositiveInfinity"), FText::AsNumber(-0.5, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToPositiveInfinity"), FText::AsNumber(-0.0, &(NumberFormattingOptions)), FText::FromString(TEXT("-0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToPositiveInfinity"), FText::AsNumber(0.0, &(NumberFormattingOptions)), FText::FromString(TEXT("0")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToPositiveInfinity"), FText::AsNumber(0.5, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));
				Test(this, TEXT("Round a Double to a number formatted correct for en-US using ToPositiveInfinity"), FText::AsNumber(1.0, &(NumberFormattingOptions)), FText::FromString(TEXT("1")));
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Test(this, TEXT("Convert a Double to a currency formatted correct for en-US"), FText::AsCurrency(DoubleValue), FText::FromString(TEXT("$12,345,678.90")));
			Test(this, TEXT("Convert a Float to a currency formatted correct for en-US"), FText::AsCurrency(FloatValue), FText::FromString(TEXT("$1,234.57")));
			Test(this, TEXT("Convert a Negative Double to a currency formatted correct for en-US"), FText::AsCurrency(DoubleNegativeValue), FText::FromString(TEXT("-$12,345,678.90")));
			Test(this, TEXT("Convert a Negative Float to a currency formatted correct for en-US"), FText::AsCurrency(FloatNegativeValue), FText::FromString(TEXT("-$1,234.57")));
			Test(this, TEXT("Convert a uint8 to a currency formatted correct for en-US"), FText::AsCurrency(Uint8Value), FText::FromString(TEXT("$255.00")));
			Test(this, TEXT("Convert a uint16 to a currency formatted correct for en-US"), FText::AsCurrency(Uint16Value), FText::FromString(TEXT("$65,535.00")));
			Test(this, TEXT("Convert a uint32 to a currency formatted correct for en-US"), FText::AsCurrency(Uint32Value), FText::FromString(TEXT("$4,294,967,295.00")));
			Test(this, TEXT("Convert a uint64 to a currency formatted correct for en-US"), FText::AsCurrency(Uint64Value), FText::FromString(TEXT("$1,844,674,407,370,955,161.00")));
			Test(this, TEXT("Convert a int8 to a currency formatted correct for en-US"), FText::AsCurrency(Int8Value), FText::FromString(TEXT("$123.00")));
			Test(this, TEXT("Convert a int16 to a currency formatted correct for en-US"), FText::AsCurrency(Int16Value), FText::FromString(TEXT("$12,345.00")));
			Test(this, TEXT("Convert a int32 to a currency formatted correct for en-US"), FText::AsCurrency(Int32Value), FText::FromString(TEXT("$12,345.00")));
			Test(this, TEXT("Convert a int64 to a currency formatted correct for en-US"), FText::AsCurrency(Int64Value), FText::FromString(TEXT("$12,345.00")));
			Test(this, TEXT("Convert a Negative int8 to a currency formatted correct for en-US"), FText::AsCurrency(Int8NegativeValue), FText::FromString(TEXT("-$123.00")));
			Test(this, TEXT("Convert a Negative int16 to a currency formatted correct for en-US"), FText::AsCurrency(Int16NegativeValue), FText::FromString(TEXT("-$12,345.00")));
			Test(this, TEXT("Convert a Negative int32 to a currency formatted correct for en-US"), FText::AsCurrency(Int32NegativeValue), FText::FromString(TEXT("-$12,345.00")));
			Test(this, TEXT("Convert a Negative int64 to a currency formatted correct for en-US"), FText::AsCurrency(Int64NegativeValue), FText::FromString(TEXT("-$12,345.00")));
			Test(this, TEXT("Convert a Negative int64 to a currency formatted correct for en-US"), FText::AsCurrency(Int64MinValue), FText::FromString(TEXT("-$9,223,372,036,854,775,808.00")));
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			Test(this, TEXT("Convert a Double to a percent formatted correct for en-US"), FText::AsPercent(DoubleValue), FText::FromString(TEXT("1,234,567,890%")));
			Test(this, TEXT("Convert a Float to a percent formatted correct for en-US"), FText::AsPercent(FloatValue), FText::FromString(TEXT("123,457%")));
			Test(this, TEXT("Convert a Negative Double to a percent formatted correct for en-US"), FText::AsPercent(DoubleNegativeValue), FText::FromString(TEXT("-1,234,567,890%")));
			Test(this, TEXT("Convert a Negative Float to a percent formatted correct for en-US"), FText::AsPercent(FloatNegativeValue), FText::FromString(TEXT("-123,457%")));
		}
		else
		{
			WARN(FString::Printf(TEXT("Internationalization data for %s missing - test is partially disabled."), TEXT("en-US")));
		}

		if (I18N.SetCurrentCulture("hi-IN"))
		{
			Test(this, TEXT("Convert a Double to a number formatted correct for hi-IN"), FText::AsNumber(DoubleValue), FText::FromString(TEXT("1,23,45,678.901")));
			Test(this, TEXT("Convert a Float to a number formatted correct for hi-IN"), FText::AsNumber(FloatValue), FText::FromString(TEXT("1,234.567")));
			Test(this, TEXT("Convert a Negative Double to a number formatted correct for hi-IN"), FText::AsNumber(DoubleNegativeValue), FText::FromString(TEXT("-1,23,45,678.901")));
			Test(this, TEXT("Convert a Negative Float to a number formatted correct for hi-IN"), FText::AsNumber(FloatNegativeValue), FText::FromString(TEXT("-1,234.567")));
			Test(this, TEXT("Convert a uint8 to a number formatted correct for hi-IN"), FText::AsNumber(Uint8Value), FText::FromString(TEXT("255")));
			Test(this, TEXT("Convert a uint16 to a number formatted correct for hi-IN"), FText::AsNumber(Uint16Value), FText::FromString(TEXT("65,535")));
			Test(this, TEXT("Convert a uint32 to a number formatted correct for hi-IN"), FText::AsNumber(Uint32Value), FText::FromString(TEXT("4,29,49,67,295")));
			Test(this, TEXT("Convert a uint64 to a number formatted correct for hi-IN"), FText::AsNumber(Uint64Value), FText::FromString(TEXT("18,44,67,44,07,37,09,55,161")));
			Test(this, TEXT("Convert a int8 to a number formatted correct for hi-IN"), FText::AsNumber(Int8Value), FText::FromString(TEXT("123")));
			Test(this, TEXT("Convert a int16 to a number formatted correct for hi-IN"), FText::AsNumber(Int16Value), FText::FromString(TEXT("12,345")));
			Test(this, TEXT("Convert a int32 to a number formatted correct for hi-IN"), FText::AsNumber(Int32Value), FText::FromString(TEXT("12,345")));
			Test(this, TEXT("Convert a int64 to a number formatted correct for hi-IN"), FText::AsNumber(Int64Value), FText::FromString(TEXT("12,345")));
			Test(this, TEXT("Convert a Negative int8 to a number formatted correct for hi-IN"), FText::AsNumber(Int8NegativeValue), FText::FromString(TEXT("-123")));
			Test(this, TEXT("Convert a Negative int16 to a number formatted correct for hi-IN"), FText::AsNumber(Int16NegativeValue), FText::FromString(TEXT("-12,345")));
			Test(this, TEXT("Convert a Negative int32 to a number formatted correct for hi-IN"), FText::AsNumber(Int32NegativeValue), FText::FromString(TEXT("-12,345")));
			Test(this, TEXT("Convert a Negative int64 to a number formatted correct for hi-IN"), FText::AsNumber(Int64NegativeValue), FText::FromString(TEXT("-12,345")));
			Test(this, TEXT("Convert a Negative int64 to a number formatted correct for hi-IN"), FText::AsNumber(Int64MinValue), FText::FromString(TEXT("-92,23,37,20,36,85,47,75,808")));

			{
				const FCulturePtr& InvariantCulture = I18N.GetInvariantCulture();
				Test(this, TEXT("Convert a Double to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(DoubleValue, NULL, InvariantCulture), FText::FromString(TEXT("12345678.901")));
				Test(this, TEXT("Convert a Float to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(FloatValue, NULL, InvariantCulture), FText::FromString(TEXT("1234.567017")));
				Test(this, TEXT("Convert a Negative Double to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(DoubleNegativeValue, NULL, InvariantCulture), FText::FromString(TEXT("-12345678.901")));
				Test(this, TEXT("Convert a Negative Float to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(FloatNegativeValue, NULL, InvariantCulture), FText::FromString(TEXT("-1234.567017")));
				Test(this, TEXT("Convert a uint8 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Uint8Value, NULL, InvariantCulture), FText::FromString(TEXT("255")));
				Test(this, TEXT("Convert a uint16 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Uint16Value, NULL, InvariantCulture), FText::FromString(TEXT("65535")));
				Test(this, TEXT("Convert a uint32 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Uint32Value, NULL, InvariantCulture), FText::FromString(TEXT("4294967295")));
				Test(this, TEXT("Convert a uint64 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Uint64Value, NULL, InvariantCulture), FText::FromString(TEXT("1844674407370955161")));
				Test(this, TEXT("Convert a int8 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Int8Value, NULL, InvariantCulture), FText::FromString(TEXT("123")));
				Test(this, TEXT("Convert a int16 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Int16Value, NULL, InvariantCulture), FText::FromString(TEXT("12345")));
				Test(this, TEXT("Convert a int32 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Int32Value, NULL, InvariantCulture), FText::FromString(TEXT("12345")));
				Test(this, TEXT("Convert a int64 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Int64Value, NULL, InvariantCulture), FText::FromString(TEXT("12345")));
				Test(this, TEXT("Convert a Negative int8 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Int8NegativeValue, NULL, InvariantCulture), FText::FromString(TEXT("-123")));
				Test(this, TEXT("Convert a Negative int16 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Int16NegativeValue, NULL, InvariantCulture), FText::FromString(TEXT("-12345")));
				Test(this, TEXT("Convert a Negative int32 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Int32NegativeValue, NULL, InvariantCulture), FText::FromString(TEXT("-12345")));
				Test(this, TEXT("Convert a Negative int64 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Int64NegativeValue, NULL, InvariantCulture), FText::FromString(TEXT("-12345")));
				Test(this, TEXT("Convert a Negative int64 to a number formatted correct for hi-IN but as invariant"), FText::AsNumber(Int64MinValue, NULL, InvariantCulture), FText::FromString(TEXT("-9223372036854775808")));
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Test(this, TEXT("Convert a Double to a currency formatted correct for hi-IN"), FText::AsCurrency(DoubleValue), FText::FromString(TEXT("\x20B9") TEXT("1,23,45,678.90")));
			Test(this, TEXT("Convert a Float to a currency formatted correct for hi-IN"), FText::AsCurrency(FloatValue), FText::FromString(TEXT("\x20B9") TEXT("1,234.57")));
			Test(this, TEXT("Convert a Negative Double to a currency formatted correct for hi-IN"), FText::AsCurrency(DoubleNegativeValue), FText::FromString(TEXT("-") TEXT("\x20B9") TEXT("1,23,45,678.90")));
			Test(this, TEXT("Convert a Negative Float to a currency formatted correct for hi-IN"), FText::AsCurrency(FloatNegativeValue), FText::FromString(TEXT("-") TEXT("\x20B9") TEXT("1,234.57")));
			Test(this, TEXT("Convert a uint8 to a currency formatted correct for hi-IN"), FText::AsCurrency(Uint8Value), FText::FromString(TEXT("\x20B9") TEXT("255.00")));
			Test(this, TEXT("Convert a uint16 to a currency formatted correct for hi-IN"), FText::AsCurrency(Uint16Value), FText::FromString(TEXT("\x20B9") TEXT("65,535.00")));
			Test(this, TEXT("Convert a uint32 to a currency formatted correct for hi-IN"), FText::AsCurrency(Uint32Value), FText::FromString(TEXT("\x20B9") TEXT("4,29,49,67,295.00")));
			Test(this, TEXT("Convert a uint64 to a currency formatted correct for hi-IN"), FText::AsCurrency(Uint64Value), FText::FromString(TEXT("\x20B9") TEXT("18,44,67,44,07,37,09,55,161.00")));
			Test(this, TEXT("Convert a int8 to a currency formatted correct for hi-IN"), FText::AsCurrency(Int8Value), FText::FromString(TEXT("\x20B9") TEXT("123.00")));
			Test(this, TEXT("Convert a int16 to a currency formatted correct for hi-IN"), FText::AsCurrency(Int16Value), FText::FromString(TEXT("\x20B9") TEXT("12,345.00")));
			Test(this, TEXT("Convert a int32 to a currency formatted correct for hi-IN"), FText::AsCurrency(Int32Value), FText::FromString(TEXT("\x20B9") TEXT("12,345.00")));
			Test(this, TEXT("Convert a int64 to a currency formatted correct for hi-IN"), FText::AsCurrency(Int64Value), FText::FromString(TEXT("\x20B9") TEXT("12,345.00")));
			Test(this, TEXT("Convert a Negative int8 to a currency formatted correct for hi-IN"), FText::AsCurrency(Int8NegativeValue), FText::FromString(TEXT("-") TEXT("\x20B9") TEXT("123.00")));
			Test(this, TEXT("Convert a Negative int16 to a currency formatted correct for hi-IN"), FText::AsCurrency(Int16NegativeValue), FText::FromString(TEXT("-") TEXT("\x20B9") TEXT("12,345.00")));
			Test(this, TEXT("Convert a Negative int32 to a currency formatted correct for hi-IN"), FText::AsCurrency(Int32NegativeValue), FText::FromString(TEXT("-") TEXT("\x20B9") TEXT("12,345.00")));
			Test(this, TEXT("Convert a Negative int64 to a currency formatted correct for hi-IN"), FText::AsCurrency(Int64NegativeValue), FText::FromString(TEXT("-") TEXT("\x20B9") TEXT("12,345.00")));
			Test(this, TEXT("Convert a Negative int64 to a currency formatted correct for hi-IN"), FText::AsCurrency(Int64MinValue), FText::FromString(TEXT("-") TEXT("\x20B9") TEXT("92,23,37,20,36,85,47,75,808.00")));
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			Test(this, TEXT("Convert a Double to a percent formatted correct for hi-IN"), FText::AsPercent(DoubleValue), FText::FromString(TEXT("1,23,45,67,890%")));
			Test(this, TEXT("Convert a Float to a percent formatted correct for hi-IN"), FText::AsPercent(FloatValue), FText::FromString(TEXT("1,23,457%")));
			Test(this, TEXT("Convert a Negative Double to a percent formatted correct for hi-IN"), FText::AsPercent(DoubleNegativeValue), FText::FromString(TEXT("-1,23,45,67,890%")));
			Test(this, TEXT("Convert a Negative Float to a percent formatted correct for hi-IN"), FText::AsPercent(FloatNegativeValue), FText::FromString(TEXT("-1,23,457%")));
		}
		else
		{
			WARN(FString::Printf(TEXT("Internationalization data for %s missing - test is partially disabled."), TEXT("hi-IN")));
		}

		I18N.RestoreCultureState(OriginalCultureState);

		FText Number = FText::AsNumber(Int64NegativeValue);
		FText Percent = FText::AsPercent(DoubleValue);
		FText Currency = FText::AsCurrencyBase(Int64Value, TEXT("USD"));

		if (GIsEditor && (Number.IsTransient() || Percent.IsTransient() || Currency.IsTransient()))
		{
			FAIL_CHECK(TEXT("Number formatting functions should not produce transient text in the editor"));
		}
		if (!GIsEditor && (!Number.IsTransient() || !Percent.IsTransient() || !Currency.IsTransient()))
		{
			FAIL_CHECK(TEXT("Number formatting functions should always produce transient text outside of the editor"));
		}
		if (Number.IsCultureInvariant() || Percent.IsCultureInvariant() || Currency.IsCultureInvariant())
		{
			FAIL_CHECK(TEXT("Number formatting functions should never produce a Culture Invariant Text"));
		}
#else
		WARN("ICU is disabled thus locale-aware number formatting is disabled.");
#endif
	}
};

TEST_CASE_NAMED(FNumberFormattingRulesTest, "System::Core::Misc::Number Formatting Rules", "[.][EditorContext][ClientContext][EngineFilter]")
{
	FNumberFormattingRulesTestClass Instanse;
	Instanse.NumberFormattingRulesTest();
}

#endif //WITH_TESTS