// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "Misc/DateTime.h"
#include "CoreGlobals.h"
#include "AutoRTFM.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextChar.h"
#include "Internationalization/TextTransformer.h"
#include "Internationalization/Culture.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Internationalization/StringTableRegistry.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Cultures/LeetCulture.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Internationalization/ICUUtilities.h"

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

#define LOCTEXT_NAMESPACE "Core.Tests.TextFormatTest"

class FTextTestClass {
public:
		FText FormatWithoutArguments(const FText& Pattern)
		{
			FFormatOrderedArguments Arguments;
			return FText::Format(Pattern, Arguments);
		}

		void ArrayToString(const TArray<FString>& Array, FString& String)
		{
			const int32 Count = Array.Num();
			for (int32 i = 0; i < Count; ++i)
			{
				if (i > 0)
				{
					String += TEXT(", ");
				}
				String += Array[i];
			}
		}

		void TestPatternParameterEnumeration(FTextTestClass& Test, const FText& Pattern, TArray<FString>& ActualParameters, const TArray<FString>& ExpectedParameters)
		{
			ActualParameters.Empty(ExpectedParameters.Num());
			FText::GetFormatPatternParameters(Pattern, ActualParameters);
			if (ActualParameters != ExpectedParameters)
			{
				FString ActualParametersString;
				ArrayToString(ActualParameters, ActualParametersString);
				FString ExpectedParametersString;
				ArrayToString(ExpectedParameters, ExpectedParametersString);
				FAIL_CHECK(FString::Printf(TEXT("\"%s\" contains parameters (%s) but expected parameters (%s)."), *(Pattern.ToString()), *(ActualParametersString), *(ExpectedParametersString)));
			}
		}

		void TestIdentical(FTextTestClass& Test, const FText& One, const FText& Two, const ETextIdenticalModeFlags CompareFlags, const bool bExpectedResult, const int32 TestLine)
		{
			const bool bActualResult = One.IdenticalTo(Two, CompareFlags);
			if (bActualResult != bExpectedResult)
			{
				FAIL_CHECK(FString::Printf(TEXT("FText(\"%s\").IdenticalTo(FText(\"%s\")) on line %d produced %s when it was expected to produce %s."), *One.ToString(), *Two.ToString(), TestLine, *LexToString(bActualResult), *LexToString(bExpectedResult)));
			}
		}
		void TextTest() {
			FInternationalization& I18N = FInternationalization::Get();

			FInternationalization::FCultureStateSnapshot OriginalCultureState;
			I18N.BackupCultureState(OriginalCultureState);

			FText ArgText0 = INVTEXT("Arg0");
			FText ArgText1 = INVTEXT("Arg1");
			FText ArgText2 = INVTEXT("Arg2");
			FText ArgText3 = INVTEXT("Arg3");

#define TEST( A, B, CompareFlags, Expected ) TestIdentical(*this, A, B, CompareFlags, Expected, __LINE__)
			{
				const int32 TestNumber1 = 10;
				const int32 TestNumber2 = 20;
				const FDateTime TestDateTime = FDateTime(1991, 6, 21, 9, 30);
				const FText TestIdenticalStr1 = LOCTEXT("TestIdenticalStr1", "Str1");
				const FText TestIdenticalStr2 = LOCTEXT("TestIdenticalStr2", "Str2");

				TEST(TestIdenticalStr1, TestIdenticalStr1, ETextIdenticalModeFlags::None, true);
				TEST(TestIdenticalStr1, TestIdenticalStr2, ETextIdenticalModeFlags::None, false);
				TEST(TestIdenticalStr1, TestIdenticalStr1, ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants, true);
				TEST(TestIdenticalStr1, TestIdenticalStr2, ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants, false);

				TEST(FText::AsCultureInvariant(TEXT("Wooble")), FText::AsCultureInvariant(TEXT("Wooble")), ETextIdenticalModeFlags::None, false);
				TEST(FText::FromString(TEXT("Wooble")), FText::FromString(TEXT("Wooble")), ETextIdenticalModeFlags::None, false);
				TEST(FText::AsCultureInvariant(TEXT("Wooble")), FText::AsCultureInvariant(TEXT("Wooble")), ETextIdenticalModeFlags::LexicalCompareInvariants, true);
				TEST(FText::FromString(TEXT("Wooble")), FText::FromString(TEXT("Wooble")), ETextIdenticalModeFlags::LexicalCompareInvariants, true);
				TEST(FText::AsCultureInvariant(TEXT("Wooble")), FText::AsCultureInvariant(TEXT("Wooble2")), ETextIdenticalModeFlags::LexicalCompareInvariants, false);
				TEST(FText::FromString(TEXT("Wooble")), FText::FromString(TEXT("Wooble2")), ETextIdenticalModeFlags::LexicalCompareInvariants, false);

				TEST(FText::Format(LOCTEXT("TestIdenticalPattern", "This takes an arg {0}"), ArgText0), FText::Format(LOCTEXT("TestIdenticalPattern", "This takes an arg {0}"), ArgText0), ETextIdenticalModeFlags::None, false);
				TEST(FText::Format(LOCTEXT("TestIdenticalPattern", "This takes an arg {0}"), ArgText0), FText::Format(LOCTEXT("TestIdenticalPattern", "This takes an arg {0}"), ArgText0), ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants, true);
				TEST(FText::Format(LOCTEXT("TestIdenticalPattern", "This takes an arg {0}"), ArgText0), FText::Format(LOCTEXT("TestIdenticalPattern", "This takes an arg {0}"), ArgText1), ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants, false);
				TEST(FText::Format(LOCTEXT("TestIdenticalPattern", "This takes an arg {0}"), ArgText0), FText::Format(LOCTEXT("TestIdenticalPattern2", "This takes an arg {0}!"), ArgText0), ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants, false);

				TEST(FText::AsDate(TestDateTime), FText::AsDate(TestDateTime), ETextIdenticalModeFlags::None, false);
				TEST(FText::AsDate(TestDateTime), FText::AsDate(TestDateTime), ETextIdenticalModeFlags::DeepCompare, true);
				TEST(FText::AsTime(TestDateTime), FText::AsTime(TestDateTime), ETextIdenticalModeFlags::None, false);
				TEST(FText::AsTime(TestDateTime), FText::AsTime(TestDateTime), ETextIdenticalModeFlags::DeepCompare, true);
				TEST(FText::AsDateTime(TestDateTime), FText::AsDateTime(TestDateTime), ETextIdenticalModeFlags::None, false);
				TEST(FText::AsDateTime(TestDateTime), FText::AsDateTime(TestDateTime), ETextIdenticalModeFlags::DeepCompare, true);

				TEST(FText::AsNumber(TestNumber1), FText::AsNumber(TestNumber1), ETextIdenticalModeFlags::None, false);
				TEST(FText::AsNumber(TestNumber1), FText::AsNumber(TestNumber1), ETextIdenticalModeFlags::DeepCompare, true);
				TEST(FText::AsNumber(TestNumber1), FText::AsNumber(TestNumber2), ETextIdenticalModeFlags::None, false);
				TEST(FText::AsNumber(TestNumber1), FText::AsNumber(TestNumber2), ETextIdenticalModeFlags::DeepCompare, false);

				TEST(TestIdenticalStr1.ToUpper(), TestIdenticalStr1.ToUpper(), ETextIdenticalModeFlags::None, false);
				TEST(TestIdenticalStr1.ToUpper(), TestIdenticalStr1.ToUpper(), ETextIdenticalModeFlags::DeepCompare, true);
				TEST(TestIdenticalStr1.ToUpper(), TestIdenticalStr1.ToLower(), ETextIdenticalModeFlags::None, false);
				TEST(TestIdenticalStr1.ToUpper(), TestIdenticalStr1.ToLower(), ETextIdenticalModeFlags::DeepCompare, false);
				TEST(TestIdenticalStr1.ToUpper(), TestIdenticalStr2.ToUpper(), ETextIdenticalModeFlags::None, false);
				TEST(TestIdenticalStr1.ToUpper(), TestIdenticalStr2.ToUpper(), ETextIdenticalModeFlags::DeepCompare, false);
			}

			#undef TEST
#define TEST( Desc, A, B ) if( !A.EqualTo(B) ) FAIL_CHECK(FString::Printf(TEXT("%s - A=%s B=%s"),*Desc,*A.ToString(),*B.ToString()))

			FText TestText;

			TestText = INVTEXT("Format with single apostrophes quotes: '{0}'");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0), INVTEXT("Format with single apostrophes quotes: 'Arg0'"));
			TestText = INVTEXT("Format with double apostrophes quotes: ''{0}''");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0), INVTEXT("Format with double apostrophes quotes: ''Arg0''"));
			TestText = INVTEXT("Print with single graves: `{0}`");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0), INVTEXT("Print with single graves: {0}`"));
			TestText = INVTEXT("Format with double graves: ``{0}``");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0), INVTEXT("Format with double graves: `Arg0`"));

			TestText = INVTEXT("Testing `escapes` here.");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("Testing `escapes` here."));
			TestText = INVTEXT("Testing ``escapes` here.");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("Testing `escapes` here."));
			TestText = INVTEXT("Testing ``escapes`` here.");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("Testing `escapes` here."));

			TestText = INVTEXT("Testing `}escapes{ here.");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("Testing }escapes{ here."));
			TestText = INVTEXT("Testing `}escapes{ here.`");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("Testing }escapes{ here.`"));
			TestText = INVTEXT("Testing `}escapes{` here.");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("Testing }escapes{` here."));
			TestText = INVTEXT("Testing }escapes`{ here.");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("Testing }escapes{ here."));
			TestText = INVTEXT("`Testing }escapes`{ here.");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("`Testing }escapes{ here."));

			TestText = INVTEXT("Testing `{escapes} here.");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("Testing {escapes} here."));
			TestText = INVTEXT("Testing `{escapes} here.`");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("Testing {escapes} here.`"));
			TestText = INVTEXT("Testing `{escapes}` here.");
			TEST(TestText.ToString(), FormatWithoutArguments(TestText), INVTEXT("Testing {escapes}` here."));

			TestText = INVTEXT("Starting text: {0} {1}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Starting text: Arg0 Arg1"));
			TestText = INVTEXT("{0} {1} - Ending Text.");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Arg0 Arg1 - Ending Text."));
			TestText = INVTEXT("Starting text: {0} {1} - Ending Text.");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Starting text: Arg0 Arg1 - Ending Text."));
			TestText = INVTEXT("{0} {1}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Arg0 Arg1"));
			TestText = INVTEXT("{1} {0}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Arg1 Arg0"));
			TestText = INVTEXT("{0}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0), INVTEXT("Arg0"));
			TestText = INVTEXT("{0} - {1} - {2} - {3}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1, ArgText2, ArgText3), INVTEXT("Arg0 - Arg1 - Arg2 - Arg3"));
			TestText = INVTEXT("{0} - {0} - {0} - {1}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Arg0 - Arg0 - Arg0 - Arg1"));

			TestText = INVTEXT("Starting text: {1}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Starting text: Arg1"));
			TestText = INVTEXT("{0} - Ending Text.");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Arg0 - Ending Text."));
			TestText = INVTEXT("Starting text: {0} - Ending Text.");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Starting text: Arg0 - Ending Text."));

			TestText = INVTEXT("{0} {2}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1, ArgText2), INVTEXT("Arg0 Arg2"));
			TestText = INVTEXT("{1}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1, ArgText2), INVTEXT("Arg1"));

			TestText = INVTEXT("Starting text: {0} {1}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Starting text: Arg0 Arg1"));
			TestText = INVTEXT("{0} {1} - Ending Text.");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Arg0 Arg1 - Ending Text."));
			TestText = INVTEXT("Starting text: {0} {1} - Ending Text.");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Starting text: Arg0 Arg1 - Ending Text."));
			TestText = INVTEXT("{0} {1}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Arg0 Arg1"));
			TestText = INVTEXT("{1} {0}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Arg1 Arg0"));
			TestText = INVTEXT("{0}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0), INVTEXT("Arg0"));
			TestText = INVTEXT("{0} - {1} - {2} - {3}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1, ArgText2, ArgText3), INVTEXT("Arg0 - Arg1 - Arg2 - Arg3"));
			TestText = INVTEXT("{0} - {0} - {0} - {1}");
			TEST(TestText.ToString(), FText::Format(TestText, ArgText0, ArgText1), INVTEXT("Arg0 - Arg0 - Arg0 - Arg1"));

			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Age"), INVTEXT("23"));
				Arguments.Add(TEXT("Height"), INVTEXT("68"));
				Arguments.Add(TEXT("Gender"), INVTEXT("male"));
				Arguments.Add(TEXT("Name"), INVTEXT("Saul"));

				// Not using all the arguments is okay.
				TestText = INVTEXT("My name is {Name}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("My name is Saul."));
				TestText = INVTEXT("My age is {Age}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("My age is 23."));
				TestText = INVTEXT("My gender is {Gender}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("My gender is male."));
				TestText = INVTEXT("My height is {Height}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("My height is 68."));

				// Using arguments out of order is okay.
				TestText = INVTEXT("My name is {Name}. My age is {Age}. My gender is {Gender}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("My name is Saul. My age is 23. My gender is male."));
				TestText = INVTEXT("My age is {Age}. My gender is {Gender}. My name is {Name}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("My age is 23. My gender is male. My name is Saul."));
				TestText = INVTEXT("My gender is {Gender}. My name is {Name}. My age is {Age}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("My gender is male. My name is Saul. My age is 23."));
				TestText = INVTEXT("My gender is {Gender}. My age is {Age}. My name is {Name}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("My gender is male. My age is 23. My name is Saul."));
				TestText = INVTEXT("My age is {Age}. My name is {Name}. My gender is {Gender}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("My age is 23. My name is Saul. My gender is male."));
				TestText = INVTEXT("My name is {Name}. My gender is {Gender}. My age is {Age}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("My name is Saul. My gender is male. My age is 23."));

				// Reusing arguments is okay.
				TestText = INVTEXT("If my age is {Age}, I have been alive for {Age} year(s).");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("If my age is 23, I have been alive for 23 year(s)."));

				// Not providing an argument leaves the parameter as text.
				TestText = INVTEXT("What... is the air-speed velocity of an unladen swallow? {AirSpeedOfAnUnladenSwallow}.");
				TEST(TestText.ToString(), FText::Format(TestText, Arguments), INVTEXT("What... is the air-speed velocity of an unladen swallow? {AirSpeedOfAnUnladenSwallow}."));
			}

			{
				FFormatNamedArguments ArgumentList;
				ArgumentList.Emplace(TEXT("Age"), INVTEXT("23"));
				ArgumentList.Emplace(TEXT("Height"), INVTEXT("68"));
				ArgumentList.Emplace(TEXT("Gender"), INVTEXT("male"));
				ArgumentList.Emplace(TEXT("Name"), INVTEXT("Saul"));

				// Not using all the arguments is okay.
				TestText = INVTEXT("My name is {Name}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("My name is Saul."));
				TestText = INVTEXT("My age is {Age}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("My age is 23."));
				TestText = INVTEXT("My gender is {Gender}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("My gender is male."));
				TestText = INVTEXT("My height is {Height}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("My height is 68."));

				// Using arguments out of order is okay.
				TestText = INVTEXT("My name is {Name}. My age is {Age}. My gender is {Gender}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("My name is Saul. My age is 23. My gender is male."));
				TestText = INVTEXT("My age is {Age}. My gender is {Gender}. My name is {Name}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("My age is 23. My gender is male. My name is Saul."));
				TestText = INVTEXT("My gender is {Gender}. My name is {Name}. My age is {Age}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("My gender is male. My name is Saul. My age is 23."));
				TestText = INVTEXT("My gender is {Gender}. My age is {Age}. My name is {Name}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("My gender is male. My age is 23. My name is Saul."));
				TestText = INVTEXT("My age is {Age}. My name is {Name}. My gender is {Gender}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("My age is 23. My name is Saul. My gender is male."));
				TestText = INVTEXT("My name is {Name}. My gender is {Gender}. My age is {Age}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("My name is Saul. My gender is male. My age is 23."));

				// Reusing arguments is okay.
				TestText = INVTEXT("If my age is {Age}, I have been alive for {Age} year(s).");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("If my age is 23, I have been alive for 23 year(s)."));

				// Not providing an argument leaves the parameter as text.
				TestText = INVTEXT("What... is the air-speed velocity of an unladen swallow? {AirSpeedOfAnUnladenSwallow}.");
				TEST(TestText.ToString(), FText::Format(TestText, ArgumentList), INVTEXT("What... is the air-speed velocity of an unladen swallow? {AirSpeedOfAnUnladenSwallow}."));
			}

#undef TEST
#define TEST( Pattern, Actual, Expected ) TestPatternParameterEnumeration(*this, Pattern, Actual, Expected)

			TArray<FString> ActualArguments;
			TArray<FString> ExpectedArguments;

			TestText = INVTEXT("My name is {Name}.");
			ExpectedArguments.Empty(1);
			ExpectedArguments.Add(TEXT("Name"));
			TEST(TestText, ActualArguments, ExpectedArguments);

			TestText = INVTEXT("My age is {Age}.");
			ExpectedArguments.Empty(1);
			ExpectedArguments.Add(TEXT("Age"));
			TEST(TestText, ActualArguments, ExpectedArguments);

			TestText = INVTEXT("If my age is {Age}, I have been alive for {Age} year(s).");
			ExpectedArguments.Empty(1);
			ExpectedArguments.Add(TEXT("Age"));
			TEST(TestText, ActualArguments, ExpectedArguments);

			TestText = INVTEXT("{0} - {1} - {2} - {3}");
			ExpectedArguments.Empty(4);
			ExpectedArguments.Add(TEXT("0"));
			ExpectedArguments.Add(TEXT("1"));
			ExpectedArguments.Add(TEXT("2"));
			ExpectedArguments.Add(TEXT("3"));
			TEST(TestText, ActualArguments, ExpectedArguments);

			TestText = INVTEXT("My name is {Name}. My age is {Age}. My gender is {Gender}.");
			ExpectedArguments.Empty(3);
			ExpectedArguments.Add(TEXT("Name"));
			ExpectedArguments.Add(TEXT("Age"));
			ExpectedArguments.Add(TEXT("Gender"));
			TEST(TestText, ActualArguments, ExpectedArguments);

#undef TEST

#if UE_ENABLE_ICU
			if (I18N.SetCurrentCulture("en-US"))
			{
#define TEST(NumBytes, UnitStandard, ExpectedString) \
			if (!FText::FromString(TEXT(ExpectedString)).EqualTo(FText::AsMemory(NumBytes, &NumberFormattingOptions, nullptr, UnitStandard))) \
			{ \
				FAIL_CHECK(FString::Printf(TEXT("FText::AsMemory expected %s bytes in %s to be %s - got %s"), TEXT(#NumBytes), TEXT(#UnitStandard), TEXT(ExpectedString), *FText::AsMemory(NumBytes, &NumberFormattingOptions, nullptr, UnitStandard).ToString())); \
			} \

			{
				FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
					.SetRoundingMode(ERoundingMode::HalfFromZero)
					.SetMinimumFractionalDigits(0)
					.SetMaximumFractionalDigits(3);

				TEST(0, EMemoryUnitStandard::SI, "0 B");
				TEST(1, EMemoryUnitStandard::SI, "1 B");
				TEST(1000, EMemoryUnitStandard::SI, "1 kB");
				TEST(1000000, EMemoryUnitStandard::SI, "1 MB");
				TEST(1000000000, EMemoryUnitStandard::SI, "1 GB");
				TEST(1000000000000, EMemoryUnitStandard::SI, "1 TB");
				TEST(1000000000000000, EMemoryUnitStandard::SI, "1 PB");
				TEST(1000000000000000000, EMemoryUnitStandard::SI, "1 EB");
				TEST(999, EMemoryUnitStandard::SI, "999 B");
				TEST(999999, EMemoryUnitStandard::SI, "999.999 kB");
				TEST(999999999, EMemoryUnitStandard::SI, "999.999 MB");
				TEST(999999999999, EMemoryUnitStandard::SI, "999.999 GB");
				TEST(999999999999999, EMemoryUnitStandard::SI, "999.999 TB");
				TEST(999999999999999999, EMemoryUnitStandard::SI, "999.999 PB");
				TEST(18446744073709551615ULL, EMemoryUnitStandard::SI, "18.446 EB");

				TEST(0, EMemoryUnitStandard::IEC, "0 B");
				TEST(1, EMemoryUnitStandard::IEC, "1 B");
				TEST(1024, EMemoryUnitStandard::IEC, "1 KiB");
				TEST(1048576, EMemoryUnitStandard::IEC, "1 MiB");
				TEST(1073741824, EMemoryUnitStandard::IEC, "1 GiB");
				TEST(1099511627776, EMemoryUnitStandard::IEC, "1 TiB");
				TEST(1125899906842624, EMemoryUnitStandard::IEC, "1 PiB");
				TEST(1152921504606846976, EMemoryUnitStandard::IEC, "1 EiB");
				TEST(1023, EMemoryUnitStandard::IEC, "0.999 KiB");
				TEST(1048575, EMemoryUnitStandard::IEC, "0.999 MiB");
				TEST(1073741823, EMemoryUnitStandard::IEC, "0.999 GiB");
				TEST(1099511627775, EMemoryUnitStandard::IEC, "0.999 TiB");
				TEST(1125899906842623, EMemoryUnitStandard::IEC, "0.999 PiB");
				TEST(1152921504606846975, EMemoryUnitStandard::IEC, "0.999 EiB");
				TEST(18446744073709551615ULL, EMemoryUnitStandard::IEC, "15.999 EiB");
			}
#undef TEST

#define TEST_UNIT(NumBytes, Unit, ExpectedString) \
		if (!FText::FromString(TEXT(ExpectedString)).EqualTo(FText::AsMemory(NumBytes, Unit, &NumberFormattingOptions, nullptr))) \
		{ \
			FAIL_CHECK(FString::Printf(TEXT("FText::AsMemory(EMemoryUnit) expected %s bytes in %s to be %s - got %s"), TEXT(#NumBytes), TEXT(#Unit), TEXT(ExpectedString), *FText::AsMemory(NumBytes, Unit, &NumberFormattingOptions, nullptr).ToString())); \
		} \

			{
				FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
					.SetRoundingMode(ERoundingMode::HalfFromZero)
					.SetMinimumFractionalDigits(0)
					.SetMaximumFractionalDigits(3);

				// --- Auto_IEC (These will map to FText::AsMemory() not accepting EMemoryUnit) ---
				// Bytes range: 0, 1, and just below the 1 KiB boundary
				TEST_UNIT(0, EMemoryUnit::Auto_IEC, "0 B");
				TEST_UNIT(1, EMemoryUnit::Auto_IEC, "1 B");
				TEST_UNIT(1023, EMemoryUnit::Auto_IEC, "0.999 KiB");
				
				// Exact powers of 1024 (KiB through EiB)
				TEST_UNIT(1024, EMemoryUnit::Auto_IEC, "1 KiB");
				TEST_UNIT(1048576, EMemoryUnit::Auto_IEC, "1 MiB");
				TEST_UNIT(1073741824, EMemoryUnit::Auto_IEC, "1 GiB");
				TEST_UNIT(1099511627776, EMemoryUnit::Auto_IEC, "1 TiB");
				TEST_UNIT(1125899906842624, EMemoryUnit::Auto_IEC, "1 PiB");
				TEST_UNIT(1152921504606846976, EMemoryUnit::Auto_IEC, "1 EiB");
				
				// Just below the next boundary
				TEST_UNIT(1048575, EMemoryUnit::Auto_IEC, "0.999 MiB");
				TEST_UNIT(1073741823, EMemoryUnit::Auto_IEC, "0.999 GiB");
				TEST_UNIT(1099511627775, EMemoryUnit::Auto_IEC, "0.999 TiB");
				TEST_UNIT(1125899906842623, EMemoryUnit::Auto_IEC, "0.999 PiB");
				TEST_UNIT(1152921504606846975, EMemoryUnit::Auto_IEC, "0.999 EiB");

				// MAX uint64
				TEST_UNIT(18446744073709551615ULL, EMemoryUnit::Auto_IEC, "15.999 EiB");

				// --- Auto_SI (These will map to FText::AsMemory() not accepting EMemoryUnit) ---
				// Bytes range: 0, 1, and just below the 1 kB boundary
				TEST_UNIT(0, EMemoryUnit::Auto_SI, "0 B");
				TEST_UNIT(1, EMemoryUnit::Auto_SI, "1 B");
				TEST_UNIT(999, EMemoryUnit::Auto_SI, "999 B");

				// Exact powers of 1000 (kB through EB)
				TEST_UNIT(1000, EMemoryUnit::Auto_SI, "1 kB");
				TEST_UNIT(1000000, EMemoryUnit::Auto_SI, "1 MB");
				TEST_UNIT(1000000000, EMemoryUnit::Auto_SI, "1 GB");
				TEST_UNIT(1000000000000, EMemoryUnit::Auto_SI, "1 TB");
				TEST_UNIT(1000000000000000, EMemoryUnit::Auto_SI, "1 PB");
				TEST_UNIT(1000000000000000000, EMemoryUnit::Auto_SI, "1 EB");

				// Just below the next boundary
				TEST_UNIT(999999, EMemoryUnit::Auto_SI, "999.999 kB");
				TEST_UNIT(999999999, EMemoryUnit::Auto_SI, "999.999 MB");
				TEST_UNIT(999999999999, EMemoryUnit::Auto_SI, "999.999 GB");
				TEST_UNIT(999999999999999, EMemoryUnit::Auto_SI, "999.999 TB");
				TEST_UNIT(999999999999999999, EMemoryUnit::Auto_SI, "999.999 PB");

				// MAX uint64
				TEST_UNIT(18446744073709551615ULL, EMemoryUnit::Auto_SI, "18.446 EB");

				// --- Unit_Bytes ---
				// Always formats as bytes regardless of magnitude
				TEST_UNIT(0, EMemoryUnit::Unit_Bytes, "0 B");
				TEST_UNIT(1, EMemoryUnit::Unit_Bytes, "1 B");
				TEST_UNIT(999, EMemoryUnit::Unit_Bytes, "999 B");
				TEST_UNIT(1024, EMemoryUnit::Unit_Bytes, "1,024 B");

				// --- Explicit IEC units ---
				// Exact powers
				TEST_UNIT(1024, EMemoryUnit::Unit_KiB, "1 KiB");
				TEST_UNIT(1048576, EMemoryUnit::Unit_MiB, "1 MiB");
				TEST_UNIT(1073741824, EMemoryUnit::Unit_GiB, "1 GiB");
				TEST_UNIT(1099511627776, EMemoryUnit::Unit_TiB, "1 TiB");
				TEST_UNIT(1125899906842624, EMemoryUnit::Unit_PiB, "1 PiB");
				TEST_UNIT(1152921504606846976, EMemoryUnit::Unit_EiB, "1 EiB");
				// Fractional values
				TEST_UNIT(512, EMemoryUnit::Unit_KiB, "0.5 KiB");
				TEST_UNIT(1536, EMemoryUnit::Unit_KiB, "1.5 KiB");
				// MAX uint64 under explicit unit (float division: 18446744073709551615 / 1024^6 = ~16)
				TEST_UNIT(18446744073709551615ULL, EMemoryUnit::Unit_EiB, "16 EiB");

				// --- Explicit SI units ---
				// Exact powers
				TEST_UNIT(1000, EMemoryUnit::Unit_KB, "1 kB");
				TEST_UNIT(1000000, EMemoryUnit::Unit_MB, "1 MB");
				TEST_UNIT(1000000000, EMemoryUnit::Unit_GB, "1 GB");
				TEST_UNIT(1000000000000, EMemoryUnit::Unit_TB, "1 TB");
				TEST_UNIT(1000000000000000, EMemoryUnit::Unit_PB, "1 PB");
				TEST_UNIT(1000000000000000000, EMemoryUnit::Unit_EB, "1 EB");
				// Fractional values
				TEST_UNIT(500, EMemoryUnit::Unit_KB, "0.5 kB");
				TEST_UNIT(1500, EMemoryUnit::Unit_KB, "1.5 kB");
				// MAX uint64 under explicit unit (float division: 18446744073709551615 / 1000^6 = ~18.447)
				TEST_UNIT(18446744073709551615ULL, EMemoryUnit::Unit_EB, "18.447 EB");
			}
#undef TEST_UNIT

#define TEST( A, B, ComparisonLevel ) if( !(FText::FromString(A)).EqualTo(FText::FromString(B), (ComparisonLevel)) ) FAIL_CHECK(FString::Printf(TEXT("Testing comparison of equivalent characters with comparison level (%s). - A=%s B=%s"),TEXT(#ComparisonLevel),(A),(B)))

			// Basic sanity checks
			TEST(TEXT("a"), TEXT("A"), ETextComparisonLevel::Primary); // Basic sanity check
			TEST(TEXT("a"), TEXT("a"), ETextComparisonLevel::Tertiary); // Basic sanity check
			TEST(TEXT("A"), TEXT("A"), ETextComparisonLevel::Tertiary); // Basic sanity check

			// Test equivalence
			TEST(TEXT("ss"), TEXT("\x00DF"), ETextComparisonLevel::Primary); // Lowercase Sharp s
			TEST(TEXT("SS"), TEXT("\x1E9E"), ETextComparisonLevel::Primary); // Uppercase Sharp S
			TEST(TEXT("ae"), TEXT("\x00E6"), ETextComparisonLevel::Primary); // Lowercase ae
			TEST(TEXT("AE"), TEXT("\x00C6"), ETextComparisonLevel::Primary); // Uppercase AE

			// Test accentuation
			TEST(TEXT("u"), TEXT("\x00FC"), ETextComparisonLevel::Primary); // Lowercase u with dieresis
			TEST(TEXT("U"), TEXT("\x00DC"), ETextComparisonLevel::Primary); // Uppercase U with dieresis

#undef TEST
			}
			else
			{
				WARN(FString::Printf(TEXT("Internationalization data for %s missing - test is partially disabled."), TEXT("en-US")));
			}
#else
			WARN("ICU is disabled thus locale-aware string comparison is disabled.");
#endif

#if UE_ENABLE_ICU
			// Sort Testing
			// French
			if (I18N.SetCurrentCulture("fr"))
			{
				TArray<FText> CorrectlySortedValues;
				CorrectlySortedValues.Add(INVTEXT("cote"));
				CorrectlySortedValues.Add(INVTEXT("cot\u00e9"));
				CorrectlySortedValues.Add(INVTEXT("c\u00f4te"));
				CorrectlySortedValues.Add(INVTEXT("c\u00f4t\u00e9"));

				{
					// Make unsorted.
					TArray<FText> Values;
					Values.Reserve(CorrectlySortedValues.Num());

					Values.Add(CorrectlySortedValues[1]);
					Values.Add(CorrectlySortedValues[3]);
					Values.Add(CorrectlySortedValues[2]);
					Values.Add(CorrectlySortedValues[0]);

					// Execute sort.
					Values.Sort(FText::FSortPredicate());

					// Test if sorted.
					bool Identical = true;
					for (int32 j = 0; j < Values.Num(); ++j)
					{
						Identical = Values[j].EqualTo(CorrectlySortedValues[j]);
						if (!Identical)
						{
							break;
						}
					}

					if (!Identical)
					{
						//currently failing FAIL_CHECK(FString::Printf(TEXT("Sort order is wrong for culture (%s)."), *FInternationalization::Get().GetCurrentCulture()->GetEnglishName()));
					}
				}
			}
			else
			{
				WARN(FString::Printf(TEXT("Internationalization data for %s missing - test is partially disabled."), TEXT("fr")));
			}

			// French Canadian
			if (I18N.SetCurrentCulture("fr-CA"))
			{
				TArray<FText> CorrectlySortedValues;
				CorrectlySortedValues.Add(INVTEXT("cote"));
				CorrectlySortedValues.Add(INVTEXT("côte"));
				CorrectlySortedValues.Add(INVTEXT("coté"));
				CorrectlySortedValues.Add(INVTEXT("côté"));

				{
					// Make unsorted.
					TArray<FText> Values;
					Values.Reserve(CorrectlySortedValues.Num());

					Values.Add(CorrectlySortedValues[1]);
					Values.Add(CorrectlySortedValues[3]);
					Values.Add(CorrectlySortedValues[2]);
					Values.Add(CorrectlySortedValues[0]);

					// Execute sort.
					Values.Sort(FText::FSortPredicate());

					// Test if sorted.
					bool Identical = true;
					for (int32 j = 0; j < Values.Num(); ++j)
					{
						Identical = Values[j].EqualTo(CorrectlySortedValues[j]);
						if (!Identical) break;
					}

					if (!Identical)
					{
						//currently failing FAIL_CHECK(FString::Printf(TEXT("Sort order is wrong for culture (%s)."), *FInternationalization::Get().GetCurrentCulture()->GetEnglishName()));
					}
				}
			}
			else
			{
				WARN(FString::Printf(TEXT("Internationalization data for %s missing - test is partially disabled."), TEXT("fr-CA")));
			}
#else
			WARN("ICU is disabled thus locale-aware string collation is disabled.");
#endif

#if UE_ENABLE_ICU
			{
				I18N.RestoreCultureState(OriginalCultureState);

				TArray<uint8> FormattedHistoryAsEnglish;
				TArray<uint8> FormattedHistoryAsFrenchCanadian;
				TArray<uint8> InvariantFTextData;

				FString InvariantString = "This is a culture invariant string.";
				FString FormattedTestLayer2_OriginalLanguageSourceString;
				FText FormattedTestLayer2;

				// Scoping to allow all locals to leave scope after we serialize at the end
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("String1"), LOCTEXT("RebuildFTextTest1_Lorem", "Lorem"));
					Args.Add(TEXT("String2"), LOCTEXT("RebuildFTextTest1_Ipsum", "Ipsum"));
					FText FormattedTest1 = FText::Format(LOCTEXT("RebuildNamedText1", "{String1} \"Lorem Ipsum\" {String2}"), Args);

					FFormatOrderedArguments ArgsOrdered;
					ArgsOrdered.Add(LOCTEXT("RebuildFTextTest1_Lorem", "Lorem"));
					ArgsOrdered.Add(LOCTEXT("RebuildFTextTest1_Ipsum", "Ipsum"));
					FText FormattedTestOrdered1 = FText::Format(LOCTEXT("RebuildOrderedText1", "{0} \"Lorem Ipsum\" {1}"), ArgsOrdered);

					// Will change to 5.542 due to default settings for numbers
					FText AsNumberTest1 = FText::AsNumber(5.5421);

					FText AsPercentTest1 = FText::AsPercent(0.925);
					FText AsCurrencyTest1 = FText::AsCurrencyBase(10025, TEXT("USD"));

					FDateTime DateTimeInfo(2080, 8, 20, 9, 33, 22);
					FText AsDateTimeTest1 = FText::AsDateTime(DateTimeInfo, EDateTimeStyle::Default, EDateTimeStyle::Default, TEXT("UTC"));

					// FormattedTestLayer2 must be updated when adding or removing from this block. Further, below, 
					// verifying the LEET translated string must be changed to reflect what the new string looks like.
					FFormatNamedArguments ArgsLayer2;
					ArgsLayer2.Add("NamedLayer1", FormattedTest1);
					ArgsLayer2.Add("OrderedLayer1", FormattedTestOrdered1);
					ArgsLayer2.Add("FTextNumber", AsNumberTest1);
					ArgsLayer2.Add("Number", 5010.89221);
					ArgsLayer2.Add("DateTime", AsDateTimeTest1);
					ArgsLayer2.Add("Percent", AsPercentTest1);
					ArgsLayer2.Add("Currency", AsCurrencyTest1);
					FormattedTestLayer2 = FText::Format(LOCTEXT("RebuildTextLayer2", "{NamedLayer1} | {OrderedLayer1} | {FTextNumber} | {Number} | {DateTime} | {Percent} | {Currency}"), ArgsLayer2);

					{
						// Serialize the full, bulky FText that is a composite of most of the other FTextHistories.
						FMemoryWriter Ar(FormattedHistoryAsEnglish);
						Ar << FormattedTestLayer2;
						Ar.Close();
					}

					// The original string in the native language.
					FormattedTestLayer2_OriginalLanguageSourceString = FormattedTestLayer2.BuildSourceString();

#if ENABLE_LOC_TESTING
					if (FTextLocalizationManager::IsDisplayStringSupportEnabled()) // Leetification requires display strings
					{
						// Swap to "LEET" culture to check if rebuilding works (verify the whole)
						I18N.SetCurrentCulture(FLeetCulture::StaticGetName());

						// When changes are made to FormattedTestLayer2, please pull out the newly translated LEET string and update the below if-statement to keep the test passing!
						FString LEETTranslatedString = FormattedTestLayer2.ToString();

						FString DesiredOutput = FString(TEXT("\x2021") TEXT("\xAB") TEXT("\x2021") TEXT("\xAB") TEXT("\x2021") TEXT("L0r3m") TEXT("\x2021") TEXT("\xBB") TEXT(" \"L0r3m 1p$um\" ") TEXT("\xAB") TEXT("\x2021") TEXT("1p$um") TEXT("\x2021") TEXT("\xBB") TEXT("\x2021") TEXT("\xBB") TEXT(" | ") TEXT("\xAB") TEXT("\x2021") TEXT("\xAB") TEXT("\x2021") TEXT("L0r3m") TEXT("\x2021") TEXT("\xBB") TEXT(" \"L0r3m 1p$um\" ") TEXT("\xAB") TEXT("\x2021") TEXT("1p$um") TEXT("\x2021") TEXT("\xBB") TEXT("\x2021") TEXT("\xBB") TEXT(" | ") TEXT("\xAB") TEXT("5.5421") TEXT("\xBB") TEXT(" | ") TEXT("\xAB") TEXT("5010.89221") TEXT("\xBB") TEXT(" | ") TEXT("\xAB") TEXT("Aug 20, 2080, 9:33:22 AM") TEXT("\xBB") TEXT(" | ") TEXT("\xAB") TEXT("92%") TEXT("\xBB") TEXT(" | ") TEXT("\xAB") TEXT("$") TEXT("\xA0") TEXT("100.25") TEXT("\xBB") TEXT("\x2021"));
						// Convert the baked string into an FText, which will be leetified, then compare it to the rebuilt FText
						if (LEETTranslatedString != DesiredOutput)
						{
							FAIL_CHECK(TEXT("FormattedTestLayer2 did not rebuild to correctly in LEET!"));
							FAIL_CHECK(TEXT("Formatted Output=") + LEETTranslatedString);
							FAIL_CHECK(TEXT("Desired Output=") + DesiredOutput);
						}					
					}
#endif

					// Swap to French-Canadian to check if rebuilding works (verify each numerical component)
					{
						I18N.SetCurrentCulture("fr-CA");

						// Need the FText to be rebuilt in fr-CA.
						(void)FormattedTestLayer2.ToString();


						if (AsNumberTest1.CompareTo(FText::AsNumber(5.5421)) != 0)
						{
							FAIL_CHECK(TEXT("AsNumberTest1 did not rebuild correctly in French-Canadian"));
							FAIL_CHECK(TEXT("Number Output=") + AsNumberTest1.ToString());
						}

						if (AsPercentTest1.CompareTo(FText::AsPercent(0.925)) != 0)
						{
							FAIL_CHECK(TEXT("AsPercentTest1 did not rebuild correctly in French-Canadian"));
							FAIL_CHECK(TEXT("Percent Output=") + AsPercentTest1.ToString());
						}

						if (AsCurrencyTest1.CompareTo(FText::AsCurrencyBase(10025, TEXT("USD"))) != 0)
						{
							FAIL_CHECK(TEXT("AsCurrencyTest1 did not rebuild correctly in French-Canadian"));
							FAIL_CHECK(TEXT("Currency Output=") + AsCurrencyTest1.ToString());
						}

						if (AsDateTimeTest1.CompareTo(FText::AsDateTime(DateTimeInfo, EDateTimeStyle::Default, EDateTimeStyle::Default, TEXT("UTC"))) != 0)
						{
							FAIL_CHECK(TEXT("AsDateTimeTest1 did not rebuild correctly in French-Canadian"));
							FAIL_CHECK(TEXT("DateTime Output=") + AsDateTimeTest1.ToString());
						}

						{
							// Serialize the full, bulky FText that is a composite of most of the other FTextHistories.
							// We don't care how this may be translated, we will be serializing this in as LEET.
							FMemoryWriter Ar(FormattedHistoryAsFrenchCanadian);
							Ar << FormattedTestLayer2;
							Ar.Close();
						}

						{
							FText InvariantFText = FText::FromString(InvariantString);

							// Serialize an invariant FText
							FMemoryWriter Ar(InvariantFTextData);
							Ar << InvariantFText;
							Ar.Close();
						}
					}
				}

#if ENABLE_LOC_TESTING
				{
					I18N.SetCurrentCulture(FLeetCulture::StaticGetName());

					FText FormattedEnglishTextHistoryAsLeet;
					FText FormattedFrenchCanadianTextHistoryAsLeet;

					{
						FMemoryReader Ar(FormattedHistoryAsEnglish);
						Ar << FormattedEnglishTextHistoryAsLeet;
						Ar.Close();
					}
					{
						FMemoryReader Ar(FormattedHistoryAsFrenchCanadian);
						Ar << FormattedFrenchCanadianTextHistoryAsLeet;
						Ar.Close();
					}

					// Confirm the two FText's serialize in and get translated into the current (LEET) translation. One originated in English, the other in French-Canadian locales.
					if (FormattedEnglishTextHistoryAsLeet.CompareTo(FormattedFrenchCanadianTextHistoryAsLeet) != 0)
					{
						FAIL_CHECK(TEXT("Serialization of text histories from source English and source French-Canadian to LEET did not produce the same results!"));
						FAIL_CHECK(TEXT("English Output=") + FormattedEnglishTextHistoryAsLeet.ToString());
						FAIL_CHECK(TEXT("French-Canadian Output=") + FormattedFrenchCanadianTextHistoryAsLeet.ToString());
					}

					// Confirm the two FText's source strings for the serialized FTexts are the same.
					if (FormattedEnglishTextHistoryAsLeet.BuildSourceString() != FormattedFrenchCanadianTextHistoryAsLeet.BuildSourceString())
					{
						FAIL_CHECK(TEXT("Serialization of text histories from source English and source French-Canadian to LEET did not produce the same source results!"));
						FAIL_CHECK(TEXT("English Output=") + FormattedEnglishTextHistoryAsLeet.BuildSourceString());
						FAIL_CHECK(TEXT("French-Canadian Output=") + FormattedFrenchCanadianTextHistoryAsLeet.BuildSourceString());
					}

					// Rebuild in LEET so that when we build the source string the DisplayString is still in LEET. 
					(void)FormattedTestLayer2.ToString();

					{
						I18N.RestoreCultureState(OriginalCultureState);

						FText InvariantFText;

						FMemoryReader Ar(InvariantFTextData);
						Ar << InvariantFText;
						Ar.Close();

						if (InvariantFText.ToString() != InvariantString)
						{
							FAIL_CHECK(TEXT("Invariant FText did not match the original FString after serialization!"));
							FAIL_CHECK(TEXT("Invariant Output=") + InvariantFText.ToString());
						}


						FString FormattedTestLayer2_SourceString = FormattedTestLayer2.BuildSourceString();

						// Compare the source string of the LEETified version of FormattedTestLayer2 to ensure it is correct.
						if (FormattedTestLayer2_OriginalLanguageSourceString != FormattedTestLayer2_SourceString)
						{
							FAIL_CHECK(TEXT("FormattedTestLayer2's source string was incorrect!"));
							FAIL_CHECK(TEXT("Output=") + FormattedTestLayer2_SourceString);
							FAIL_CHECK(TEXT("Desired Output=") + FormattedTestLayer2_OriginalLanguageSourceString);
						}
					}
				}
#endif
			}
#else
			WARN("ICU is disabled thus locale-aware formatting needed in rebuilding source text from history is disabled.");
#endif

			//**********************************
			// FromString Test
			//**********************************
			TestText = FText::FromString(TEXT("Test String"));

			if (GIsEditor && TestText.IsCultureInvariant())
			{
				FAIL_CHECK(TEXT("FromString should not produce a Culture Invariant Text when called inside the editor"));
			}

			if (!GIsEditor && !TestText.IsCultureInvariant())
			{
				FAIL_CHECK(TEXT("FromString should produce a Culture Invariant Text when called outside the editor"));
			}

			if (TestText.IsTransient())
			{
				FAIL_CHECK(TEXT("FromString should never produce a Transient Text"));
			}

			I18N.RestoreCultureState(OriginalCultureState);
		}
};

TEST_CASE_NAMED(FTextTest, "System::Core::Misc::Text", "[.][EditorContext][ClientContext][EngineFilter]")
{
	FTextTestClass Instance;
	Instance.TextTest();
}

TEST_CASE_NAMED(FTextRoundingTest, "System::Core::Misc::TextRounding", "[.][EditorContext][ClientContext][EngineFilter]")
{
	static const TCHAR* RoundingModeNames[] = {
		TEXT("HalfToEven"),
		TEXT("HalfFromZero"),
		TEXT("HalfToZero"),
		TEXT("FromZero"),
		TEXT("ToZero"),
		TEXT("ToNegativeInfinity"),
		TEXT("ToPositiveInfinity"),
	};

	static_assert(ERoundingMode::ToPositiveInfinity == UE_ARRAY_COUNT(RoundingModeNames) - 1, "RoundingModeNames array needs updating");

	static const double InputValues[] = {
		1000.1224,
		1000.1225,
		1000.1226,
		1000.1234,
		1000.1235,
		1000.1236,
		
		1000.1244,
		1000.1245,
		1000.1246,
		1000.1254,
		1000.1255,
		1000.1256,

		-1000.1224,
		-1000.1225,
		-1000.1226,
		-1000.1234,
		-1000.1235,
		-1000.1236,
		
		-1000.1244,
		-1000.1245,
		-1000.1246,
		-1000.1254,
		-1000.1255,
		-1000.1256,
	};

	static const TCHAR* OutputValues[][UE_ARRAY_COUNT(RoundingModeNames)] = 
	{
		// HalfToEven        | HalfFromZero      | HalfToZero        | FromZero          | ToZero            | ToNegativeInfinity | ToPositiveInfinity
		{  TEXT("1000.122"),   TEXT("1000.122"),   TEXT("1000.122"),   TEXT("1000.123"),   TEXT("1000.122"),   TEXT("1000.122"),    TEXT("1000.123") },
		{  TEXT("1000.122"),   TEXT("1000.123"),   TEXT("1000.122"),   TEXT("1000.123"),   TEXT("1000.122"),   TEXT("1000.122"),    TEXT("1000.123") },
		{  TEXT("1000.123"),   TEXT("1000.123"),   TEXT("1000.123"),   TEXT("1000.123"),   TEXT("1000.122"),   TEXT("1000.122"),    TEXT("1000.123") },
		{  TEXT("1000.123"),   TEXT("1000.123"),   TEXT("1000.123"),   TEXT("1000.124"),   TEXT("1000.123"),   TEXT("1000.123"),    TEXT("1000.124") },
		{  TEXT("1000.124"),   TEXT("1000.124"),   TEXT("1000.123"),   TEXT("1000.124"),   TEXT("1000.123"),   TEXT("1000.123"),    TEXT("1000.124") },
		{  TEXT("1000.124"),   TEXT("1000.124"),   TEXT("1000.124"),   TEXT("1000.124"),   TEXT("1000.123"),   TEXT("1000.123"),    TEXT("1000.124") },

		{  TEXT("1000.124"),   TEXT("1000.124"),   TEXT("1000.124"),   TEXT("1000.125"),   TEXT("1000.124"),   TEXT("1000.124"),    TEXT("1000.125") },
		{  TEXT("1000.124"),   TEXT("1000.125"),   TEXT("1000.124"),   TEXT("1000.125"),   TEXT("1000.124"),   TEXT("1000.124"),    TEXT("1000.125") },
		{  TEXT("1000.125"),   TEXT("1000.125"),   TEXT("1000.125"),   TEXT("1000.125"),   TEXT("1000.124"),   TEXT("1000.124"),    TEXT("1000.125") },
		{  TEXT("1000.125"),   TEXT("1000.125"),   TEXT("1000.125"),   TEXT("1000.126"),   TEXT("1000.125"),   TEXT("1000.125"),    TEXT("1000.126") },
		{  TEXT("1000.126"),   TEXT("1000.126"),   TEXT("1000.125"),   TEXT("1000.126"),   TEXT("1000.125"),   TEXT("1000.125"),    TEXT("1000.126") },
		{  TEXT("1000.126"),   TEXT("1000.126"),   TEXT("1000.126"),   TEXT("1000.126"),   TEXT("1000.125"),   TEXT("1000.125"),    TEXT("1000.126") },

		{ TEXT("-1000.122"),  TEXT("-1000.122"),  TEXT("-1000.122"),  TEXT("-1000.123"),  TEXT("-1000.122"),  TEXT("-1000.123"),   TEXT("-1000.122") },
		{ TEXT("-1000.122"),  TEXT("-1000.123"),  TEXT("-1000.122"),  TEXT("-1000.123"),  TEXT("-1000.122"),  TEXT("-1000.123"),   TEXT("-1000.122") },
		{ TEXT("-1000.123"),  TEXT("-1000.123"),  TEXT("-1000.123"),  TEXT("-1000.123"),  TEXT("-1000.122"),  TEXT("-1000.123"),   TEXT("-1000.122") },
		{ TEXT("-1000.123"),  TEXT("-1000.123"),  TEXT("-1000.123"),  TEXT("-1000.124"),  TEXT("-1000.123"),  TEXT("-1000.124"),   TEXT("-1000.123") },
		{ TEXT("-1000.124"),  TEXT("-1000.124"),  TEXT("-1000.123"),  TEXT("-1000.124"),  TEXT("-1000.123"),  TEXT("-1000.124"),   TEXT("-1000.123") },
		{ TEXT("-1000.124"),  TEXT("-1000.124"),  TEXT("-1000.124"),  TEXT("-1000.124"),  TEXT("-1000.123"),  TEXT("-1000.124"),   TEXT("-1000.123") },

		{ TEXT("-1000.124"),  TEXT("-1000.124"),  TEXT("-1000.124"),  TEXT("-1000.125"),  TEXT("-1000.124"),  TEXT("-1000.125"),   TEXT("-1000.124") },
		{ TEXT("-1000.124"),  TEXT("-1000.125"),  TEXT("-1000.124"),  TEXT("-1000.125"),  TEXT("-1000.124"),  TEXT("-1000.125"),   TEXT("-1000.124") },
		{ TEXT("-1000.125"),  TEXT("-1000.125"),  TEXT("-1000.125"),  TEXT("-1000.125"),  TEXT("-1000.124"),  TEXT("-1000.125"),   TEXT("-1000.124") },
		{ TEXT("-1000.125"),  TEXT("-1000.125"),  TEXT("-1000.125"),  TEXT("-1000.126"),  TEXT("-1000.125"),  TEXT("-1000.126"),   TEXT("-1000.125") },
		{ TEXT("-1000.126"),  TEXT("-1000.126"),  TEXT("-1000.125"),  TEXT("-1000.126"),  TEXT("-1000.125"),  TEXT("-1000.126"),   TEXT("-1000.125") },
		{ TEXT("-1000.126"),  TEXT("-1000.126"),  TEXT("-1000.126"),  TEXT("-1000.126"),  TEXT("-1000.125"),  TEXT("-1000.126"),   TEXT("-1000.125") },
	};

	static_assert(UE_ARRAY_COUNT(InputValues) == UE_ARRAY_COUNT(OutputValues), "The size of InputValues does not match OutputValues");

	FInternationalization& I18N = FInternationalization::Get();

	FInternationalization::FCultureStateSnapshot OriginalCultureState;
	I18N.BackupCultureState(OriginalCultureState);
	
	// This test needs to be run using an English culture
	I18N.SetCurrentCulture(TEXT("en"));

	// Test to make sure that the decimal formatter is rounding fractional numbers correctly (to 3 decimal places)
	FNumberFormattingOptions FormattingOptions = FNumberFormattingOptions()
		.SetUseGrouping(false)
		.SetMaximumFractionalDigits(3);

	auto DoSingleTest = [&](const double InNumber, const FString& InExpectedString, const FString& InDescription)
	{
		const FText ResultText = FText::AsNumber(InNumber, &FormattingOptions);
		if (ResultText.ToString() != InExpectedString)
		{
			FAIL_CHECK(FString::Printf(TEXT("Text rounding failure: source '%f' - expected '%s' - result '%s'. %s."), InNumber, *InExpectedString, *ResultText.ToString(), *InDescription));
		}
	};

	auto DoAllTests = [&](const ERoundingMode InRoundingMode)
	{
		FormattingOptions.SetRoundingMode(InRoundingMode);

		for (int32 TestValueIndex = 0; TestValueIndex < UE_ARRAY_COUNT(InputValues); ++TestValueIndex)
		{
			DoSingleTest(InputValues[TestValueIndex], OutputValues[TestValueIndex][InRoundingMode], RoundingModeNames[InRoundingMode]);
		}
	};

	DoAllTests(ERoundingMode::HalfToEven);
	DoAllTests(ERoundingMode::HalfFromZero);
	DoAllTests(ERoundingMode::HalfToZero);
	DoAllTests(ERoundingMode::FromZero);
	DoAllTests(ERoundingMode::ToZero);
	DoAllTests(ERoundingMode::ToNegativeInfinity);
	DoAllTests(ERoundingMode::ToPositiveInfinity);

	// HalfToEven - Rounds to the nearest place, equidistant ties go to the value which is closest to an even value: 1.5 becomes 2, 0.5 becomes 0
	{
		FormattingOptions.SetRoundingMode(ERoundingMode::HalfToEven);

		DoSingleTest(1000.12459, TEXT("1000.125"), TEXT("HalfToEven"));
		DoSingleTest(1000.124549, TEXT("1000.125"), TEXT("HalfToEven"));
		DoSingleTest(1000.124551, TEXT("1000.125"), TEXT("HalfToEven"));
		DoSingleTest(1000.12451, TEXT("1000.125"), TEXT("HalfToEven"));
		DoSingleTest(1000.1245000001, TEXT("1000.125"), TEXT("HalfToEven"));
		DoSingleTest(1000.12450000000001, TEXT("1000.124"), TEXT("HalfToEven"));

		DoSingleTest(512.9999, TEXT("513"), TEXT("HalfToEven"));
		DoSingleTest(-512.9999, TEXT("-513"), TEXT("HalfToEven"));
	}

	// Restore original culture
	I18N.RestoreCultureState(OriginalCultureState);
}

TEST_CASE_NAMED(FTextPaddingTest, "System::Core::Misc::TextPadding", "[.][EditorContext][ClientContext][EngineFilter]")
{
	FInternationalization& I18N = FInternationalization::Get();

	FInternationalization::FCultureStateSnapshot OriginalCultureState;
	I18N.BackupCultureState(OriginalCultureState);

	// This test needs to be run using an English culture
	I18N.SetCurrentCulture(TEXT("en"));

	// Test to make sure that the decimal formatter is padding integral numbers correctly
	FNumberFormattingOptions FormattingOptions;

	auto DoSingleIntTest = [&](const int32 InNumber, const FString& InExpectedString, const FString& InDescription)
	{
		const FText ResultText = FText::AsNumber(InNumber, &FormattingOptions);
		if (ResultText.ToString() != InExpectedString)
		{
			FAIL_CHECK(FString::Printf(TEXT("Text padding failure: source '%d' - expected '%s' - result '%s'. %s."), InNumber, *InExpectedString, *ResultText.ToString(), *InDescription));
		}
	};

	auto DoSingleDoubleTest = [&](const double InNumber, const FString& InExpectedString, const FString& InDescription)
	{
		const FText ResultText = FText::AsNumber(InNumber, &FormattingOptions);
		if (ResultText.ToString() != InExpectedString)
		{
			FAIL_CHECK(FString::Printf(TEXT("Text padding failure: source '%f' - expected '%s' - result '%s'. %s."), InNumber, *InExpectedString, *ResultText.ToString(), *InDescription));
		}
	};

	// Test with a max limit of 3
	{
		FormattingOptions = FNumberFormattingOptions()
			.SetUseGrouping(false)
			.SetMaximumIntegralDigits(3);

		DoSingleIntTest(123456,  TEXT("456"),  TEXT("Truncating '123456' to a max of 3 integral digits"));
		DoSingleIntTest(-123456, TEXT("-456"), TEXT("Truncating '-123456' to a max of 3 integral digits"));
	}

	// Test with a min limit of 6
	{
		FormattingOptions = FNumberFormattingOptions()
			.SetUseGrouping(false)
			.SetMinimumIntegralDigits(6);

		DoSingleIntTest(123,  TEXT("000123"),  TEXT("Padding '123' to a min of 6 integral digits"));
		DoSingleIntTest(-123, TEXT("-000123"), TEXT("Padding '-123' to a min of 6 integral digits"));
	}

	// Test with forced fractional digits
	{
		FormattingOptions = FNumberFormattingOptions()
			.SetUseGrouping(false)
			.SetMinimumFractionalDigits(3);

		DoSingleIntTest(123,  TEXT("123.000"),  TEXT("Padding '123' to a min of 3 fractional digits"));
		DoSingleIntTest(-123, TEXT("-123.000"), TEXT("Padding '-123' to a min of 3 fractional digits"));
	}

	// Testing with leading zeros on a real number
	{
		FormattingOptions = FNumberFormattingOptions()
			.SetUseGrouping(false)
			.SetMaximumFractionalDigits(4);

		DoSingleDoubleTest(0.00123,  TEXT("0.0012"),  TEXT("Padding '0.00123' to a max of 4 fractional digits"));
		DoSingleDoubleTest(-0.00123, TEXT("-0.0012"), TEXT("Padding '-0.00123' to a max of 4 fractional digits"));
	}

	// Testing with leading zeros on a real number
	{
		FormattingOptions = FNumberFormattingOptions()
			.SetUseGrouping(false)
			.SetMaximumFractionalDigits(8);

		DoSingleDoubleTest(0.00123,  TEXT("0.00123"),  TEXT("Padding '0.00123' to a max of 8 fractional digits"));
		DoSingleDoubleTest(-0.00123, TEXT("-0.00123"), TEXT("Padding '-0.00123' to a max of 8 fractional digits"));
	}

	// Test with forced fractional digits on a real number
	{
		FormattingOptions = FNumberFormattingOptions()
			.SetUseGrouping(false)
			.SetMinimumFractionalDigits(8)
			.SetMaximumFractionalDigits(8);

		DoSingleDoubleTest(0.00123,  TEXT("0.00123000"),  TEXT("Padding '0.00123' to a min of 8 fractional digits"));
		DoSingleDoubleTest(-0.00123, TEXT("-0.00123000"), TEXT("Padding '-0.00123' to a min of 8 fractional digits"));
	}

	// Restore original culture
	I18N.RestoreCultureState(OriginalCultureState);
}


class FTextNumericParsingTestClass {
public:
	struct FTextNumericParsingTestUtil
	{
		template <typename T>
		static void DoTest(const TCHAR* InStr, const int32 InStrLen, const FDecimalNumberFormattingRules& InFormattingRules, const FNumberParsingOptions& InParsingOptions, const T InExpectedValue, const bool bExpectedToParse, const TCHAR* InDescription)
		{
			T Value;
			const bool bDidParse = FastDecimalFormat::StringToNumber(InStr, InStrLen, InFormattingRules, InParsingOptions, Value);

			if (bDidParse != bExpectedToParse)
			{
				FAIL_CHECK(FString::Printf(TEXT("Text parsing failure: source '%s' - expected to parse '%s' - result '%s'. %s."), InStr, bExpectedToParse ? TEXT("true") : TEXT("false"), bDidParse ? TEXT("true") : TEXT("false"), InDescription));
				return;
			}

			if (bDidParse && Value != InExpectedValue)
			{
				FAIL_CHECK(FString::Printf(TEXT("Text parsing failure: source '%s' - expected value '%f' - result '%f'. %s."), InStr, (double)InExpectedValue, (double)Value, InDescription));
				return;
			}
		}

		template <typename T>
		static void DoGroupingTest(const TCHAR* InStr, const int32 InStrLen, const FDecimalNumberFormattingRules& InFormattingRules, const T InExpectedValue, const bool bExpectedToParse, const TCHAR* InDescription)
		{
			DoTest(InStr, InStrLen, InFormattingRules, FNumberParsingOptions::DefaultWithGrouping(), InExpectedValue, bExpectedToParse, InDescription);
		}

		template <typename T>
		static void DoGroupingTest(const TCHAR* InStr, const FDecimalNumberFormattingRules& InFormattingRules, const T InExpectedValue, const bool bExpectedToParse, const TCHAR* InDescription)
		{
			DoGroupingTest(InStr, FCString::Strlen(InStr), InFormattingRules, InExpectedValue, bExpectedToParse, InDescription);
		}

		template <typename T>
		static void DoLimitsTest(const TCHAR* InStr, const FDecimalNumberFormattingRules& InFormattingRules, const T InExpectedValue, const bool bExpectedToParse, const TCHAR* InDescription)
		{
			DoTest(InStr, FCString::Strlen(InStr), InFormattingRules, FNumberParsingOptions().SetUseGrouping(true).SetInsideLimits(true), InExpectedValue, bExpectedToParse, InDescription);
		}

		template <typename T>
		static void DoClampTest(const TCHAR* InStr, const FDecimalNumberFormattingRules& InFormattingRules, const T InExpectedValue, const bool bExpectedToParse, const TCHAR* InDescription)
		{
			DoTest(InStr, FCString::Strlen(InStr), InFormattingRules, FNumberParsingOptions().SetUseGrouping(true).SetUseClamping(true), InExpectedValue, bExpectedToParse, InDescription);
		}

		template <typename T>
		static void DoAllTests(const TCHAR* InStr, const FDecimalNumberFormattingRules& InFormattingRules, const T InExpectedValue, const T InExpectedClampedValue, const bool bExpectedToParse, const bool bExpectedToParseStrict, const TCHAR* InDescription)
		{
			DoGroupingTest(InStr, InFormattingRules, InExpectedValue, bExpectedToParse, InDescription);
			DoClampTest(InStr, InFormattingRules, InExpectedClampedValue, bExpectedToParse, InDescription);
			DoLimitsTest(InStr, InFormattingRules, InExpectedValue, bExpectedToParseStrict, InDescription);
		}
	};

	void TextNumericParsingTest() {
		FInternationalization& I18N = FInternationalization::Get();

		auto DoTests = [this](const FString& InCulture)
		{
			FCulturePtr Culture = FInternationalization::Get().GetCulture(InCulture);
			if (Culture.IsValid())
			{
				const FDecimalNumberFormattingRules& FormattingRules = Culture->GetDecimalNumberFormattingRules();

				auto BuildDescription = [&InCulture](const TCHAR* InTestStr, const TCHAR* InTypeStr) -> FString
				{
					return FString::Printf(TEXT("[%s] Parsing '%s' as '%s'"), *InCulture, InTestStr, InTypeStr);
				};

				const FString UnsignedString = FString::Printf(TEXT("135%c456"), FormattingRules.DecimalSeparatorCharacter);
				const FString PositiveString = FString::Printf(TEXT("%s135%c456"), *FormattingRules.PlusString, FormattingRules.DecimalSeparatorCharacter);
				const FString NegativeString = FString::Printf(TEXT("%s135%c456"), *FormattingRules.MinusString, FormattingRules.DecimalSeparatorCharacter);
				const FString PositiveASCIIString = FString::Printf(TEXT("+135%c456"), FormattingRules.DecimalSeparatorCharacter);
				const FString NegativeASCIIString = FString::Printf(TEXT("-135%c456"), FormattingRules.DecimalSeparatorCharacter);
				const FString GroupSeparatedString = FString::Printf(TEXT("1%c234%c5"), FormattingRules.GroupingSeparatorCharacter, FormattingRules.DecimalSeparatorCharacter);

				int32 Number135 = 135;

				FTextNumericParsingTestUtil::DoAllTests<int8>(*UnsignedString, FormattingRules, static_cast<int8>(Number135), TNumericLimits<int8>::Max(), true, false, *BuildDescription(*UnsignedString, TEXT("int8")));
				FTextNumericParsingTestUtil::DoAllTests<uint8>(*UnsignedString, FormattingRules, static_cast<uint8>(Number135), 135, true, true, *BuildDescription(*UnsignedString, TEXT("uint8")));
				FTextNumericParsingTestUtil::DoGroupingTest<int16>(*UnsignedString, FormattingRules, static_cast<int16>(Number135), true, *BuildDescription(*UnsignedString, TEXT("int16")));
				FTextNumericParsingTestUtil::DoGroupingTest<uint16>(*UnsignedString, FormattingRules, static_cast<uint16>(Number135), true, *BuildDescription(*UnsignedString, TEXT("uint16")));
				FTextNumericParsingTestUtil::DoGroupingTest<int32>(*UnsignedString, FormattingRules, Number135, true, *BuildDescription(*UnsignedString, TEXT("int32")));
				FTextNumericParsingTestUtil::DoGroupingTest<uint32>(*UnsignedString, FormattingRules, static_cast<uint32>(Number135), true, *BuildDescription(*UnsignedString, TEXT("uint32")));
				FTextNumericParsingTestUtil::DoGroupingTest<int64>(*UnsignedString, FormattingRules, 135, true, *BuildDescription(*UnsignedString, TEXT("int64")));
				FTextNumericParsingTestUtil::DoGroupingTest<uint64>(*UnsignedString, FormattingRules, static_cast<uint64>(Number135), true, *BuildDescription(*UnsignedString, TEXT("uint64")));
				FTextNumericParsingTestUtil::DoGroupingTest<float>(*UnsignedString, FormattingRules, 135.456f, true, *BuildDescription(*UnsignedString, TEXT("float")));
				FTextNumericParsingTestUtil::DoGroupingTest<double>(*UnsignedString, FormattingRules, 135.456, true, *BuildDescription(*UnsignedString, TEXT("double")));

				FTextNumericParsingTestUtil::DoGroupingTest<int32>(*PositiveString, FormattingRules, 135, true, *BuildDescription(*PositiveString, TEXT("int32")));
				FTextNumericParsingTestUtil::DoGroupingTest<uint32>(*PositiveString, FormattingRules, 135, true, *BuildDescription(*PositiveString, TEXT("uint32")));
				FTextNumericParsingTestUtil::DoGroupingTest<float>(*PositiveString, FormattingRules, 135.456f, true, *BuildDescription(*PositiveString, TEXT("float")));
				FTextNumericParsingTestUtil::DoGroupingTest<double>(*PositiveString, FormattingRules, 135.456, true, *BuildDescription(*PositiveString, TEXT("double")));

				FTextNumericParsingTestUtil::DoAllTests<int8>(*NegativeString, FormattingRules, static_cast<int8>(-Number135), TNumericLimits<int8>::Lowest(), true, false, *BuildDescription(*NegativeString, TEXT("int8")));
				FTextNumericParsingTestUtil::DoAllTests<int32>(*NegativeString, FormattingRules, -135, -135, true, true, *BuildDescription(*NegativeString, TEXT("int32")));
				FTextNumericParsingTestUtil::DoAllTests<uint32>(*NegativeString, FormattingRules, static_cast<uint32>(-Number135), 0, true, false, *BuildDescription(*NegativeString, TEXT("uint32")));
				FTextNumericParsingTestUtil::DoGroupingTest<float>(*NegativeString, FormattingRules, -135.456f, true, *BuildDescription(*NegativeString, TEXT("float")));
				FTextNumericParsingTestUtil::DoGroupingTest<double>(*NegativeString, FormattingRules, -135.456, true, *BuildDescription(*NegativeString, TEXT("double")));

				FTextNumericParsingTestUtil::DoGroupingTest<int32>(*PositiveASCIIString, FormattingRules, 135, true, *BuildDescription(*PositiveASCIIString, TEXT("int32")));
				FTextNumericParsingTestUtil::DoGroupingTest<int32>(*NegativeASCIIString, FormattingRules, -135, true, *BuildDescription(*NegativeASCIIString, TEXT("int32")));
				FTextNumericParsingTestUtil::DoGroupingTest<float>(*PositiveASCIIString, FormattingRules, 135.456f, true, *BuildDescription(*PositiveASCIIString, TEXT("float")));
				FTextNumericParsingTestUtil::DoGroupingTest<float>(*NegativeASCIIString, FormattingRules, -135.456f, true, *BuildDescription(*PositiveASCIIString, TEXT("float")));

				int32 Number1234 = 1234;
				FTextNumericParsingTestUtil::DoAllTests<uint8>(*GroupSeparatedString, FormattingRules, static_cast<uint8>(Number1234), TNumericLimits<uint8>::Max(), true, false, *BuildDescription(*GroupSeparatedString, TEXT("int32")));
				FTextNumericParsingTestUtil::DoGroupingTest<int32>(*GroupSeparatedString, FormattingRules, Number1234, true, *BuildDescription(*GroupSeparatedString, TEXT("int32")));
				FTextNumericParsingTestUtil::DoGroupingTest<uint32>(*GroupSeparatedString, FormattingRules, static_cast<uint32>(Number1234), true, *BuildDescription(*GroupSeparatedString, TEXT("uint32")));
				FTextNumericParsingTestUtil::DoGroupingTest<float>(*GroupSeparatedString, FormattingRules, 1234.5f, true, *BuildDescription(*GroupSeparatedString, TEXT("float")));

				uint64 BigNumber = 9223372036854775809ull; // (double)9223372036854775809 == 9223372036854775808.0, (last digic is not the same) && 9223372036854775809 > int64::max()
				const FString BigUnsignedString = FString::Printf(TEXT("9223372036854775809"));
				const FString BigPositiveString = FString::Printf(TEXT("%s9223372036854775809"), *FormattingRules.PlusString);
				const FString BigNegativeString = FString::Printf(TEXT("%s9223372036854775809"), *FormattingRules.MinusString);
				int64 BigGroupedNumber = 9223372036854775800ll;
				const FString BigGroupSeparatedString = FString::Printf(TEXT("9%c223%c372%c036%c854%c775%c800")
					, FormattingRules.GroupingSeparatorCharacter, FormattingRules.GroupingSeparatorCharacter, FormattingRules.GroupingSeparatorCharacter
					, FormattingRules.GroupingSeparatorCharacter, FormattingRules.GroupingSeparatorCharacter, FormattingRules.GroupingSeparatorCharacter);

				FTextNumericParsingTestUtil::DoAllTests<int32>(*BigUnsignedString, FormattingRules, static_cast<int32>(BigNumber), TNumericLimits<int32>::Max(), true, false, *BuildDescription(*BigUnsignedString, TEXT("int32")));
				FTextNumericParsingTestUtil::DoAllTests<uint32>(*BigUnsignedString, FormattingRules, static_cast<uint32>(BigNumber), TNumericLimits<uint32>::Max(), true, false, *BuildDescription(*BigUnsignedString, TEXT("uint32")));
				FTextNumericParsingTestUtil::DoAllTests<int64>(*BigUnsignedString, FormattingRules, static_cast<int64>(BigNumber), TNumericLimits<int64>::Max(), true, false, *BuildDescription(*BigUnsignedString, TEXT("int64")));
				FTextNumericParsingTestUtil::DoAllTests<uint64>(*BigUnsignedString, FormattingRules, BigNumber, BigNumber, true, true, *BuildDescription(*BigUnsignedString, TEXT("uint64")));
				FTextNumericParsingTestUtil::DoAllTests<float>(*BigUnsignedString, FormattingRules, static_cast<float>(BigNumber), static_cast<float>(BigNumber), true, true, *BuildDescription(*BigUnsignedString, TEXT("float")));
				FTextNumericParsingTestUtil::DoAllTests<double>(*BigUnsignedString, FormattingRules, static_cast<double>(BigNumber), static_cast<double>(BigNumber), true, true, *BuildDescription(*BigUnsignedString, TEXT("double")));

				FTextNumericParsingTestUtil::DoAllTests<int64>(*BigPositiveString, FormattingRules, static_cast<int64>(BigNumber), TNumericLimits<int64>::Max(), true, false, *BuildDescription(*BigPositiveString, TEXT("int64")));
				FTextNumericParsingTestUtil::DoAllTests<uint64>(*BigPositiveString, FormattingRules, BigNumber, BigNumber, true, true, *BuildDescription(*BigPositiveString, TEXT("uint64")));
				FTextNumericParsingTestUtil::DoAllTests<float>(*BigPositiveString, FormattingRules, static_cast<float>(BigNumber), static_cast<float>(BigNumber), true, true, *BuildDescription(*BigPositiveString, TEXT("float")));
				FTextNumericParsingTestUtil::DoAllTests<double>(*BigPositiveString, FormattingRules, static_cast<double>(BigNumber), static_cast<double>(BigNumber), true, true, *BuildDescription(*BigPositiveString, TEXT("double")));

				FTextNumericParsingTestUtil::DoAllTests<int64>(*BigNegativeString, FormattingRules, -static_cast<int64>(BigNumber), TNumericLimits<int64>::Lowest(), true, false, *BuildDescription(*NegativeString, TEXT("int64")));
				FTextNumericParsingTestUtil::DoAllTests<uint64>(*BigNegativeString, FormattingRules, -static_cast<int64>(BigNumber), TNumericLimits<uint64>::Lowest(), true, false, *BuildDescription(*NegativeString, TEXT("uint64")));
				FTextNumericParsingTestUtil::DoAllTests<float>(*BigNegativeString, FormattingRules, -static_cast<float>(BigNumber), -static_cast<float>(BigNumber), true, true, *BuildDescription(*NegativeString, TEXT("float")));
				FTextNumericParsingTestUtil::DoAllTests<double>(*BigNegativeString, FormattingRules, -static_cast<double>(BigNumber), -static_cast<double>(BigNumber), true, true, *BuildDescription(*NegativeString, TEXT("double")));

				FTextNumericParsingTestUtil::DoAllTests<int32>(*BigGroupSeparatedString, FormattingRules, static_cast<int32>(BigGroupedNumber), TNumericLimits<int32>::Max(), true, false, *BuildDescription(*BigGroupSeparatedString, TEXT("int32")));
				FTextNumericParsingTestUtil::DoAllTests<uint32>(*BigGroupSeparatedString, FormattingRules, static_cast<int32>(BigGroupedNumber), TNumericLimits<uint32>::Max(), true, false, *BuildDescription(*BigGroupSeparatedString, TEXT("uint32")));
				FTextNumericParsingTestUtil::DoAllTests<int64>(*BigGroupSeparatedString, FormattingRules, BigGroupedNumber, BigGroupedNumber, true, true, *BuildDescription(*BigGroupSeparatedString, TEXT("int64")));
				FTextNumericParsingTestUtil::DoAllTests<uint64>(*BigGroupSeparatedString, FormattingRules, static_cast<uint64>(BigGroupedNumber), BigGroupedNumber, true, true, *BuildDescription(*BigGroupSeparatedString, TEXT("uint64")));
				FTextNumericParsingTestUtil::DoAllTests<float>(*BigGroupSeparatedString, FormattingRules, static_cast<float>(BigGroupedNumber), static_cast<float>(BigGroupedNumber), true, true, *BuildDescription(*BigGroupSeparatedString, TEXT("float")));
				FTextNumericParsingTestUtil::DoAllTests<double>(*BigGroupSeparatedString, FormattingRules, static_cast<double>(BigGroupedNumber), static_cast<double>(BigGroupedNumber), true, true, *BuildDescription(*BigGroupSeparatedString, TEXT("double")));
			}
		};

		DoTests(TEXT("en"));
		DoTests(TEXT("fr"));
		DoTests(TEXT("ar"));

		{
			const FDecimalNumberFormattingRules& AgnosticFormattingRules = FastDecimalFormat::GetCultureAgnosticFormattingRules();

			FTextNumericParsingTestUtil::DoGroupingTest<int32>(TEXT("10a"), AgnosticFormattingRules, 0, false, TEXT("Parsing '10a' as 'int32'"));
			FTextNumericParsingTestUtil::DoGroupingTest<uint32>(TEXT("10a"), AgnosticFormattingRules, 0, false, TEXT("Parsing '10a' as 'uint32'"));

			FTextNumericParsingTestUtil::DoGroupingTest<int32>(TEXT("10a"), 2, AgnosticFormattingRules, 10, true, TEXT("Parsing '10a' (len 2) as 'int32'"));
			FTextNumericParsingTestUtil::DoGroupingTest<uint32>(TEXT("10a"), 2, AgnosticFormattingRules, 10, true, TEXT("Parsing '10a' (len 2) as 'uint32'"));
		}

		{
			const FDecimalNumberFormattingRules& AgnosticFormattingRules = FastDecimalFormat::GetCultureAgnosticFormattingRules();

			// test limits
			FTextNumericParsingTestUtil::DoAllTests<int8>(TEXT("-128"), AgnosticFormattingRules, TNumericLimits<int8>::Lowest(), TNumericLimits<int8>::Lowest(), true, true, TEXT("Parsing int8 lowest"));
			FTextNumericParsingTestUtil::DoAllTests<int8>(TEXT("127"), AgnosticFormattingRules, TNumericLimits<int8>::Max(), TNumericLimits<int8>::Max(), true, true, TEXT("Parsing int8 max"));
			FTextNumericParsingTestUtil::DoAllTests<uint8>(TEXT("0"), AgnosticFormattingRules, TNumericLimits<uint8>::Lowest(), TNumericLimits<uint8>::Lowest(), true, true, TEXT("Parsing uint8 lowest"));
			FTextNumericParsingTestUtil::DoAllTests<uint8>(TEXT("255"), AgnosticFormattingRules, TNumericLimits<uint8>::Max(), TNumericLimits<uint8>::Max(), true, true, TEXT("Parsing uint8 max"));
			FTextNumericParsingTestUtil::DoAllTests<int64>(TEXT("-9223372036854775808"), AgnosticFormattingRules, TNumericLimits<int64>::Lowest(), TNumericLimits<int64>::Lowest(), true, true, TEXT("Parsing int64 lowest"));
			FTextNumericParsingTestUtil::DoAllTests<int64>(TEXT("9223372036854775807"), AgnosticFormattingRules, TNumericLimits<int64>::Max(), TNumericLimits<int64>::Max(), true, true, TEXT("Parsing int64 max"));
			FTextNumericParsingTestUtil::DoAllTests<uint64>(TEXT("0"), AgnosticFormattingRules, TNumericLimits<uint64>::Lowest(), TNumericLimits<uint64>::Lowest(), true, true, TEXT("Parsing uint64 lowest"));
			FTextNumericParsingTestUtil::DoAllTests<uint64>(TEXT("18446744073709551615"), AgnosticFormattingRules, TNumericLimits<uint64>::Max(), TNumericLimits<uint64>::Max(), true, true, TEXT("Parsing uint64 max"));

			// test limits +- 1
			int32 Number129 = 129;
			int32 Number128 = 128;
			int32 Number1 = 1;
			int32 Number256 = 256;
			FTextNumericParsingTestUtil::DoAllTests<int8>(TEXT("-129"), AgnosticFormattingRules, static_cast<int8>(-Number129), TNumericLimits<int8>::Lowest(), true, false, TEXT("Parsing int8 +/-1 lowest"));
			FTextNumericParsingTestUtil::DoAllTests<int8>(TEXT("128"), AgnosticFormattingRules, static_cast<int8>(Number128), TNumericLimits<int8>::Max(), true, false, TEXT("Parsing int8 +/-1 max"));
			FTextNumericParsingTestUtil::DoAllTests<uint8>(TEXT("-1"), AgnosticFormattingRules, static_cast<uint8>(-Number1), TNumericLimits<uint8>::Lowest(), true, false, TEXT("Parsing uint8 +/-1 lowest"));
			FTextNumericParsingTestUtil::DoAllTests<uint8>(TEXT("256"), AgnosticFormattingRules, static_cast<uint8>(Number256), TNumericLimits<uint8>::Max(), true, false, TEXT("Parsing uint8 +/-1 max"));

			int64 Number9223372036854775809 = TNumericLimits<int64>::Lowest();
			--Number9223372036854775809;
			int64 Number9223372036854775808 = TNumericLimits<int64>::Max();
			++Number9223372036854775808;
			uint64 Number18446744073709551616 = TNumericLimits<uint64>::Max();
			++Number18446744073709551616;
			FTextNumericParsingTestUtil::DoAllTests<int64>(TEXT("-9223372036854775809"), AgnosticFormattingRules, Number9223372036854775809, TNumericLimits<int64>::Lowest(), true, false, TEXT("Parsing int64 +/-1 lowest"));
			FTextNumericParsingTestUtil::DoAllTests<int64>(TEXT("9223372036854775808"), AgnosticFormattingRules, Number9223372036854775808, TNumericLimits<int64>::Max(), true, false, TEXT("Parsing +/-1 int64 max"));
			FTextNumericParsingTestUtil::DoAllTests<uint64>(TEXT("-1"), AgnosticFormattingRules, static_cast<uint64>(-Number1), TNumericLimits<uint64>::Lowest(), true, false, TEXT("Parsing uint64 +/-1 lowest"));
			FTextNumericParsingTestUtil::DoAllTests<uint64>(TEXT("18446744073709551616"), AgnosticFormattingRules, Number18446744073709551616, TNumericLimits<uint64>::Max(), true, false, TEXT("Parsing +/-1 uint64 max"));

			FTextNumericParsingTestUtil::DoGroupingTest<int64>(TEXT("-18446744073709551616"), AgnosticFormattingRules, 0, true, TEXT("Parsing negative overflow int64 max"));
			FTextNumericParsingTestUtil::DoLimitsTest<int64>(TEXT("-18446744073709551616"), AgnosticFormattingRules, 0, false, TEXT("Parsing negative overflow int64 max"));
			FTextNumericParsingTestUtil::DoClampTest<int64>(TEXT("-18446744073709551616"), AgnosticFormattingRules, TNumericLimits<int64>::Lowest(), true, TEXT("Parsing negative overflow int64 max"));
		}
	}
};

TEST_CASE_NAMED(FTextNumericParsingTest, "System::Core::Misc::TextNumericParsing", "[.][EditorContext][ClientContext][EngineFilter][ServerContext][CommandletContext][SupportsAutoRTFM]")
{
	FTextNumericParsingTestClass Instance;
	Instance.TextNumericParsingTest();
}

TEST_CASE_NAMED(FTextStringificationTest, "System::Core::Misc::TextStringification", "[.][EditorContext][ClientContext][EngineFilter][ServerContext][CommandletContext][SupportsAutoRTFM]")
{
	FInternationalization& I18N = FInternationalization::Get();

	FInternationalization::FCultureStateSnapshot OriginalCultureState;
	I18N.BackupCultureState(OriginalCultureState);

	// This test needs to be run using the English (US) culture to ensure the time formatting has a valid timezone to work with
	I18N.SetCurrentCulture(TEXT("en-US"));

	auto DoSingleTest = [](const FText& InExpectedText, const FString& InExpectedString, const FString& InCppString, const bool bImportCppString)
	{
		// Validate that the text produces the string we expect
		FString ActualString;
		FTextStringHelper::WriteToBuffer(ActualString, InExpectedText);
		if (!ActualString.Equals(InExpectedString, ESearchCase::CaseSensitive))
		{
			FAIL_CHECK(FString::Printf(TEXT("Text export failure (from text): Text '%s' was expected to export as '%s', but produced '%s'."), *InExpectedText.ToString(), *InExpectedString, *ActualString));
		}
		// Validate that the string produces the text we expect
		FText ActualText;
		if (!FTextStringHelper::ReadFromBuffer(*InExpectedString, ActualText))
		{
			FAIL_CHECK(FString::Printf(TEXT("Text import failure (from string): String '%s' failed to import."), *InExpectedString));
		}

		if (!InExpectedText.ToString().Equals(ActualText.ToString(), ESearchCase::CaseSensitive))
		{
			FAIL_CHECK(FString::Printf(TEXT("Text import failure (from string): String '%s' was expected to import as '%s', but produced '%s'."), *InExpectedString, *InExpectedText.ToString(), *ActualText.ToString()));
		}
		// Validate that the C++ string produces the text we expect
		if (bImportCppString)
		{
			FText ActualCppText;
			if (!FTextStringHelper::ReadFromBuffer(*InCppString, ActualCppText))
			{
				FAIL_CHECK(FString::Printf(TEXT("Text import failure (from C++): String '%s' failed to import."), *InCppString));
			}
			if (!InExpectedText.ToString().Equals(ActualCppText.ToString(), ESearchCase::CaseSensitive))
			{
				FAIL_CHECK(FString::Printf(TEXT("Text import failure (from C++): String '%s' was expected to import as '%s', but produced '%s'."), *InCppString, *InExpectedText.ToString(), *ActualCppText.ToString()));
			}
		}
	};

#define TEST(Text, Str)                       DoSingleTest(Text, TEXT(Str), TEXT(#Text), true)
#define TEST_EX(Text, Str, ImportCpp)         DoSingleTest(Text, TEXT(Str), TEXT(#Text), ImportCpp)

#if UE_ENABLE_ICU
	#define TEST_ICU(Text, Str)               DoSingleTest(Text, TEXT(Str), TEXT(#Text), true)
	#define TEST_EX_ICU(Text, Str, ImportCpp) DoSingleTest(Text, TEXT(Str), TEXT(#Text), ImportCpp)
#else
	#define TEST_ICU(Text, Str)              
	#define TEST_EX_ICU(Text, Str, ImportCpp)
#endif

	// Add the test string table, but only if it isn't already!
	if (!FStringTableRegistry::Get().FindStringTable("Core.Tests.TextFormatTest"))
	{
		LOCTABLE_NEW("Core.Tests.TextFormatTest", "Core.Tests.TextFormatTest");
		LOCTABLE_SETSTRING("Core.Tests.TextFormatTest", "TextStringificationTest_Lorem", "Lorem");
		LOCTABLE_SETNOTESTRING("Core.Tests.TextFormatTest", "TextStringificationTest_Ipsum", "Ipsum", "A note about Ipsum");
	}

	TEST(NSLOCTEXT("Core.Tests.TextFormatTest", "TextStringificationTest_Lorem", "Lorem"), "NSLOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Lorem\", \"Lorem\")");
	TEST(LOCTEXT("TextStringificationTest_Lorem", "Lorem"), "NSLOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Lorem\", \"Lorem\")");
	TEST(LOCTABLE("Core.Tests.TextFormatTest", "TextStringificationTest_Lorem"), "LOCTABLE(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Lorem\")");
	
#if WITH_EDITORONLY_DATA
	TEST(NSNOTELOCTEXT("Core.Tests.TextFormatTest", "TextStringificationTest_Ipsum", "Ipsum", "A note about Ipsum"), "NSNOTELOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Ipsum\", \"Ipsum\", \"A note about Ipsum\")");
	TEST(NOTELOCTEXT("TextStringificationTest_Ipsum", "Ipsum", "A note about Ipsum"), "NSNOTELOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Ipsum\", \"Ipsum\", \"A note about Ipsum\")");
#else
	// Notes are stripped outside of editor
	TEST(NSNOTELOCTEXT("Core.Tests.TextFormatTest", "TextStringificationTest_Ipsum", "Ipsum", "A note about Ipsum"), "NSLOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Ipsum\", \"Ipsum\")");
	TEST(NOTELOCTEXT("TextStringificationTest_Ipsum", "Ipsum", "A note about Ipsum"), "NSLOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Ipsum\", \"Ipsum\")");
#endif

#if WITH_EDITORONLY_DATA
	// Test that DevNotes can be retrieved
	{
		const FText IpsumText = NOTELOCTEXT("TextStringificationTest_Ipsum", "Ipsum", "A note about Ipsum");
		const FString IpsumTextDevNotes = IpsumText.GetDevNotes();
		if (!IpsumTextDevNotes.Equals(TEXT("A note about Ipsum"), ESearchCase::CaseSensitive))
		{
			FAIL_CHECK(FString::Printf(TEXT("Text DevNotes import failure (from C++): Expected DevNotes 'A note about Ipsum' but got '%s'."), *IpsumTextDevNotes));
		}

		const FText NSIpsumText = NSNOTELOCTEXT("Core.Tests.TextFormatTest", "TextStringificationTest_Ipsum", "Ipsum", "A note about Ipsum");
		const FString NSIpsumTextDevNotes = NSIpsumText.GetDevNotes();
		if (!NSIpsumTextDevNotes.Equals(TEXT("A note about Ipsum"), ESearchCase::CaseSensitive))
		{
			FAIL_CHECK(FString::Printf(TEXT("Text DevNotes import failure (from C++): Expected DevNotes 'A note about Ipsum' but got '%s'."), *NSIpsumTextDevNotes));
		}
	}
#endif

	TEST(INVTEXT("DummyText"), "INVTEXT(\"DummyText\")");
	if (GIsEditor)
	{
		TEST_EX(FText::FromString(TEXT("DummyString")), "DummyString", false);
	}
	else
	{
		TEST_EX(FText::FromString(TEXT("DummyString")), "INVTEXT(\"DummyString\")", false);
	}

	TEST(LOCGEN_NUMBER(10, ""), "LOCGEN_NUMBER(10, \"\")");
	TEST(LOCGEN_NUMBER_GROUPED(12.5f, ""), "LOCGEN_NUMBER_GROUPED(12.500000f, \"\")");
	TEST(LOCGEN_NUMBER_UNGROUPED(12.5f, ""), "LOCGEN_NUMBER_UNGROUPED(12.500000f, \"\")");
	TEST(LOCGEN_NUMBER_CUSTOM(+10, SetAlwaysSign(true).SetRoundingMode(ERoundingMode::ToZero).SetMinimumFractionalDigits(2), ""), "LOCGEN_NUMBER_CUSTOM(10, SetAlwaysSign(true).SetRoundingMode(ERoundingMode::ToZero).SetMinimumFractionalDigits(2), \"\")");
	TEST_ICU(LOCGEN_NUMBER(-10, "en"), "LOCGEN_NUMBER(-10, \"en\")");

	TEST(LOCGEN_PERCENT(0.1f, ""), "LOCGEN_PERCENT(0.100000f, \"\")");
	TEST(LOCGEN_PERCENT_GROUPED(0.1f, ""), "LOCGEN_PERCENT_GROUPED(0.100000f, \"\")");
	TEST(LOCGEN_PERCENT_UNGROUPED(0.1f, ""), "LOCGEN_PERCENT_UNGROUPED(0.100000f, \"\")");
	TEST(LOCGEN_PERCENT_CUSTOM(0.1f, SetAlwaysSign(true).SetRoundingMode(ERoundingMode::ToZero).SetMinimumFractionalDigits(2), ""), "LOCGEN_PERCENT_CUSTOM(0.100000f, SetAlwaysSign(true).SetRoundingMode(ERoundingMode::ToZero).SetMinimumFractionalDigits(2), \"\")");
	TEST_ICU(LOCGEN_PERCENT(0.1, "en"), "LOCGEN_PERCENT(0.100000, \"en\")");

	TEST(LOCGEN_CURRENCY(125, "USD", ""), "LOCGEN_CURRENCY(125, \"USD\", \"\")");
	TEST_EX_ICU(FText::AsCurrency(1.25f, TEXT("USD"), nullptr, FInternationalization::Get().GetCulture(TEXT("en"))), "LOCGEN_CURRENCY(125, \"USD\", \"en\")", false);

	TEST_ICU(LOCGEN_DATE_UTC(1526342400, EDateTimeStyle::Short, "", "en-GB"), "LOCGEN_DATE_UTC(1526342400, EDateTimeStyle::Short, \"\", \"en-GB\")");
	TEST(LOCGEN_DATE_LOCAL(1526342400, EDateTimeStyle::Medium, ""), "LOCGEN_DATE_LOCAL(1526342400, EDateTimeStyle::Medium, \"\")");

	TEST_ICU(LOCGEN_TIME_UTC(1526342400, EDateTimeStyle::Long, "", "en-GB"), "LOCGEN_TIME_UTC(1526342400, EDateTimeStyle::Long, \"\", \"en-GB\")");
	TEST(LOCGEN_TIME_LOCAL(1526342400, EDateTimeStyle::Full, ""), "LOCGEN_TIME_LOCAL(1526342400, EDateTimeStyle::Full, \"\")");

	TEST_ICU(LOCGEN_DATETIME_UTC(1526342400, EDateTimeStyle::Short, EDateTimeStyle::Medium, "", "en-GB"), "LOCGEN_DATETIME_UTC(1526342400, EDateTimeStyle::Short, EDateTimeStyle::Medium, \"\", \"en-GB\")");
	TEST(LOCGEN_DATETIME_LOCAL(1526342400, EDateTimeStyle::Long, EDateTimeStyle::Full, ""), "LOCGEN_DATETIME_LOCAL(1526342400, EDateTimeStyle::Long, EDateTimeStyle::Full, \"\")");

	TEST_ICU(LOCGEN_DATETIME_CUSTOM_UTC(1526342400, "%A, %B %e, %Y", "", "en-GB"), "LOCGEN_DATETIME_CUSTOM_UTC(1526342400, \"%A, %B %e, %Y\", \"\", \"en-GB\")");
	TEST(LOCGEN_DATETIME_CUSTOM_LOCAL(1526342400, "%A, %B %e, %Y", ""), "LOCGEN_DATETIME_CUSTOM_LOCAL(1526342400, \"%A, %B %e, %Y\", \"\")");

	TEST(LOCGEN_TOUPPER(LOCTEXT("TextStringificationTest_Lorem", "Lorem")), "LOCGEN_TOUPPER(NSLOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Lorem\", \"Lorem\"))");
	TEST(LOCGEN_TOLOWER(LOCTEXT("TextStringificationTest_Lorem", "Lorem")), "LOCGEN_TOLOWER(NSLOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Lorem\", \"Lorem\"))");

	TEST(LOCGEN_FORMAT_ORDERED(LOCTEXT("TextStringificationTest_FmtO", "{0} weighs {1}kg"), LOCTEXT("TextStringificationTest_Bear", "Bear"), 227), "LOCGEN_FORMAT_ORDERED(NSLOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_FmtO\", \"{0} weighs {1}kg\"), NSLOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Bear\", \"Bear\"), 227)");
	TEST(LOCGEN_FORMAT_NAMED(LOCTEXT("TextStringificationTest_FmtN", "{Animal} weighs {Weight}kg"), TEXT("Animal"), LOCTEXT("TextStringificationTest_Bear", "Bear"), TEXT("Weight"), 227), "LOCGEN_FORMAT_NAMED(NSLOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_FmtN\", \"{Animal} weighs {Weight}kg\"), \"Animal\", NSLOCTEXT(\"Core.Tests.TextFormatTest\", \"TextStringificationTest_Bear\", \"Bear\"), \"Weight\", 227)");

#undef TEST
#undef TEST_EX
#undef TEST_ICU
#undef TEST_EX_ICU

	// Restore original culture
	I18N.RestoreCultureState(OriginalCultureState);
}

TEST_CASE_NAMED(FTextFormatArgModifierTest, "System::Core::Misc::TextFormatArgModifiers", "[.][EditorContext][ClientContext][EngineFilter][ServerContext][CommandletContext][SupportsAutoRTFM]")
{
	auto EnsureValidResult = [&](const FString& InResult, const FString& InExpected, const FString& InName, const FString& InDescription)
	{
		if (!InResult.Equals(InExpected, ESearchCase::CaseSensitive))
		{
			FAIL_CHECK(FString::Printf(TEXT("%s failure: result '%s' (expected '%s'). %s."), *InName, *InResult, *InExpected, *InDescription));
		}
	};

	FInternationalization& I18N = FInternationalization::Get();

	FInternationalization::FCultureStateSnapshot OriginalCultureState;
	I18N.BackupCultureState(OriginalCultureState);

	// This test needs to be run using an English culture
	I18N.SetCurrentCulture(TEXT("en"));

	{
		const FTextFormat CardinalFormatText = INVTEXT("There {NumCats}|plural(one=is,other=are) {NumCats} {NumCats}|plural(one=cat,other=cats)");
		EnsureValidResult(FText::FormatNamed(CardinalFormatText, TEXT("NumCats"), 0).ToString(), TEXT("There are 0 cats"), TEXT("CardinalResult0"), CardinalFormatText.GetSourceText().ToString());
		EnsureValidResult(FText::FormatNamed(CardinalFormatText, TEXT("NumCats"), 1).ToString(), TEXT("There is 1 cat"), TEXT("CardinalResult1"), CardinalFormatText.GetSourceText().ToString());
		EnsureValidResult(FText::FormatNamed(CardinalFormatText, TEXT("NumCats"), 2).ToString(), TEXT("There are 2 cats"), TEXT("CardinalResult2"), CardinalFormatText.GetSourceText().ToString());
		EnsureValidResult(FText::FormatNamed(CardinalFormatText, TEXT("NumCats"), 3).ToString(), TEXT("There are 3 cats"), TEXT("CardinalResult3"), CardinalFormatText.GetSourceText().ToString());
		EnsureValidResult(FText::FormatNamed(CardinalFormatText, TEXT("NumCats"), 4).ToString(), TEXT("There are 4 cats"), TEXT("CardinalResult4"), CardinalFormatText.GetSourceText().ToString());
	}

	{
		const FTextFormat OrdinalFormatText = INVTEXT("You came {Place}{Place}|ordinal(one=st,two=nd,few=rd,other=th)!");
		EnsureValidResult(FText::FormatNamed(OrdinalFormatText, TEXT("Place"), 0).ToString(), TEXT("You came 0th!"), TEXT("OrdinalResult0"), OrdinalFormatText.GetSourceText().ToString());
		EnsureValidResult(FText::FormatNamed(OrdinalFormatText, TEXT("Place"), 1).ToString(), TEXT("You came 1st!"), TEXT("OrdinalResult1"), OrdinalFormatText.GetSourceText().ToString());
		EnsureValidResult(FText::FormatNamed(OrdinalFormatText, TEXT("Place"), 2).ToString(), TEXT("You came 2nd!"), TEXT("OrdinalResult2"), OrdinalFormatText.GetSourceText().ToString());
		EnsureValidResult(FText::FormatNamed(OrdinalFormatText, TEXT("Place"), 3).ToString(), TEXT("You came 3rd!"), TEXT("OrdinalResult3"), OrdinalFormatText.GetSourceText().ToString());
		EnsureValidResult(FText::FormatNamed(OrdinalFormatText, TEXT("Place"), 4).ToString(), TEXT("You came 4th!"), TEXT("OrdinalResult4"), OrdinalFormatText.GetSourceText().ToString());
	}

	{
		const FTextFormat GenderFormatText = INVTEXT("{Gender}|gender(Le,La) {Gender}|gender(guerrier,guerrière) est {Gender}|gender(fort,forte)");
		EnsureValidResult(FText::FormatNamed(GenderFormatText, TEXT("Gender"), ETextGender::Masculine).ToString(), TEXT("Le guerrier est fort"), TEXT("GenderResultM"), GenderFormatText.GetSourceText().ToString());
		EnsureValidResult(FText::FormatNamed(GenderFormatText, TEXT("Gender"), ETextGender::Feminine).ToString(), TEXT("La guerrière est forte"), TEXT("GenderResultF"), GenderFormatText.GetSourceText().ToString());
	}

	{
		const FTextFormat GenderFormatText = INVTEXT("{Gender}|gender(Le guerrier est fort,La guerrière est forte)");
		EnsureValidResult(FText::FormatNamed(GenderFormatText, TEXT("Gender"), ETextGender::Masculine).ToString(), TEXT("Le guerrier est fort"), TEXT("GenderResultM"), GenderFormatText.GetSourceText().ToString());
		EnsureValidResult(FText::FormatNamed(GenderFormatText, TEXT("Gender"), ETextGender::Feminine).ToString(), TEXT("La guerrière est forte"), TEXT("GenderResultF"), GenderFormatText.GetSourceText().ToString());
	}

	{
		const FText Consonant = INVTEXT("\uC0AC\uB78C");/* 사람 */
		const FText ConsonantRieul = INVTEXT("\uC11C\uC6B8");/* 서울 */
		const FText Vowel = INVTEXT("\uC0AC\uC790");/* 사자 */

		{
			const FTextFormat HppFormatText = INVTEXT("{Arg}|hpp(\uC740,\uB294)");/* 은/는 */
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Consonant).ToString(), TEXT("\uC0AC\uB78C\uC740"), TEXT("HppResultConsonant"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), ConsonantRieul).ToString(), TEXT("\uC11C\uC6B8\uC740"), TEXT("HppResultConsonantRieul"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Vowel).ToString(), TEXT("\uC0AC\uC790\uB294"), TEXT("HppResultVowel"), HppFormatText.GetSourceText().ToString());
		}

		{
			const FTextFormat HppFormatText = INVTEXT("{Arg}|hpp(\uC774,\uAC00)");/* 이/가 */
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Consonant).ToString(), TEXT("\uC0AC\uB78C\uC774"), TEXT("HppResultConsonant"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), ConsonantRieul).ToString(), TEXT("\uC11C\uC6B8\uC774"), TEXT("HppResultConsonantRieul"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Vowel).ToString(), TEXT("\uC0AC\uC790\uAC00"), TEXT("HppResultVowel"), HppFormatText.GetSourceText().ToString());
		}

		{
			const FTextFormat HppFormatText = INVTEXT("{Arg}|hpp(\uC744,\uB97C)");/* 을/를 */
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Consonant).ToString(), TEXT("\uC0AC\uB78C\uC744"), TEXT("HppResultConsonant"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), ConsonantRieul).ToString(), TEXT("\uC11C\uC6B8\uC744"), TEXT("HppResultConsonantRieul"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Vowel).ToString(), TEXT("\uC0AC\uC790\uB97C"), TEXT("HppResultVowel"), HppFormatText.GetSourceText().ToString());
		}

		{
			const FTextFormat HppFormatText = INVTEXT("{Arg}|hpp(\uACFC,\uC640)");/* 과/와 */
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Consonant).ToString(), TEXT("\uC0AC\uB78C\uACFC"), TEXT("HppResultConsonant"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), ConsonantRieul).ToString(), TEXT("\uC11C\uC6B8\uACFC"), TEXT("HppResultConsonantRieul"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Vowel).ToString(), TEXT("\uC0AC\uC790\uC640"), TEXT("HppResultVowel"), HppFormatText.GetSourceText().ToString());
		}

		{
			const FTextFormat HppFormatText = INVTEXT("{Arg}|hpp(\uC544,\uC57C)");/* 아/야 */
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Consonant).ToString(), TEXT("\uC0AC\uB78C\uC544"), TEXT("HppResultConsonant"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), ConsonantRieul).ToString(), TEXT("\uC11C\uC6B8\uC544"), TEXT("HppResultConsonantRieul"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Vowel).ToString(), TEXT("\uC0AC\uC790\uC57C"), TEXT("HppResultVowel"), HppFormatText.GetSourceText().ToString());
		}

		{
			const FTextFormat HppFormatText = INVTEXT("{Arg}|hpp(\uC774\uC5B4,\uC5EC)");/* 이어/여 */
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Consonant).ToString(), TEXT("\uC0AC\uB78C\uC774\uC5B4"), TEXT("HppResultConsonant"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), ConsonantRieul).ToString(), TEXT("\uC11C\uC6B8\uC774\uC5B4"), TEXT("HppResultConsonantRieul"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Vowel).ToString(), TEXT("\uC0AC\uC790\uC5EC"), TEXT("HppResultVowel"), HppFormatText.GetSourceText().ToString());
		}

		{
			const FTextFormat HppFormatText = INVTEXT("{Arg}|hpp(\uC774\uC5D0,\uC608)");/* 이에/예 */
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Consonant).ToString(), TEXT("\uC0AC\uB78C\uC774\uC5D0"), TEXT("HppResultConsonant"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), ConsonantRieul).ToString(), TEXT("\uC11C\uC6B8\uC774\uC5D0"), TEXT("HppResultConsonantRieul"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Vowel).ToString(), TEXT("\uC0AC\uC790\uC608"), TEXT("HppResultVowel"), HppFormatText.GetSourceText().ToString());
		}

		{
			const FTextFormat HppFormatText = INVTEXT("{Arg}|hpp(\uC774\uC5C8,\uC601)");/* 이었/영*/
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Consonant).ToString(), TEXT("\uC0AC\uB78C\uC774\uC5C8"), TEXT("HppResultConsonant"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), ConsonantRieul).ToString(), TEXT("\uC11C\uC6B8\uC774\uC5C8"), TEXT("HppResultConsonantRieul"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Vowel).ToString(), TEXT("\uC0AC\uC790\uC601"), TEXT("HppResultVowel"), HppFormatText.GetSourceText().ToString());
		}

		{
			const FTextFormat HppFormatText = INVTEXT("{Arg}|hpp(\uC73C\uB85C,\uB85C)");/* 으로/로 */
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Consonant).ToString(), TEXT("\uC0AC\uB78C\uC73C\uB85C"), TEXT("HppResultConsonant"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), ConsonantRieul).ToString(), TEXT("\uC11C\uC6B8\uB85C"), TEXT("HppResultConsonantRieul"), HppFormatText.GetSourceText().ToString());
			EnsureValidResult(FText::FormatNamed(HppFormatText, TEXT("Arg"), Vowel).ToString(), TEXT("\uC0AC\uC790\uB85C"), TEXT("HppResultVowel"), HppFormatText.GetSourceText().ToString());
		}
	}

	// Restore original culture
	I18N.RestoreCultureState(OriginalCultureState);
}

#if UE_ENABLE_ICU

TEST_CASE_NAMED(FICUSanitizationTest, "System::Core::Misc::ICUSanitization", "[.][EditorContext][ClientContext][EngineFilter]")
{
	// Validate culture code sanitization
	{
		auto TestCultureCodeSanitization = [](const FString& InCode, const FString& InExpectedCode)
		{
			const FString SanitizedCode = ICUUtilities::SanitizeCultureCode(InCode);
			if (!SanitizedCode.Equals(InExpectedCode, ESearchCase::CaseSensitive))
			{
				FAIL_CHECK(FString::Printf(TEXT("SanitizeCultureCode did not produce the expected result (got '%s', expected '%s')"), *SanitizedCode, *InExpectedCode));
			}
		};

		TestCultureCodeSanitization(TEXT("en-US"), TEXT("en-US"));
		TestCultureCodeSanitization(TEXT("en_US_POSIX"), TEXT("en_US_POSIX"));
		TestCultureCodeSanitization(TEXT("en-US{}%"), TEXT("en-US"));
		TestCultureCodeSanitization(TEXT("en{}%-US"), TEXT("en-US"));
	}

	// Validate timezone code sanitization
	{
		auto TestTimezoneCodeSanitization = [](const FString& InCode, const FString& InExpectedCode)
		{
			const FString SanitizedCode = ICUUtilities::SanitizeTimezoneCode(InCode);
			if (!SanitizedCode.Equals(InExpectedCode, ESearchCase::CaseSensitive))
			{
				FAIL_CHECK(FString::Printf(TEXT("SanitizeTimezoneCode did not produce the expected result (got '%s', expected '%s')"), *SanitizedCode, *InExpectedCode));
			}
		};

		TestTimezoneCodeSanitization(TEXT("Etc/Unknown"), TEXT("Etc/Unknown"));
		TestTimezoneCodeSanitization(TEXT("America/Sao_Paulo"), TEXT("America/Sao_Paulo"));
		TestTimezoneCodeSanitization(TEXT("America/Sao_Paulo{}%"), TEXT("America/Sao_Paulo"));
		TestTimezoneCodeSanitization(TEXT("America/Sao{}%_Paulo"), TEXT("America/Sao_Paulo"));
		TestTimezoneCodeSanitization(TEXT("Antarctica/DumontDUrville"), TEXT("Antarctica/DumontDUrville"));
		TestTimezoneCodeSanitization(TEXT("Antarctica/DumontDUrville{}%"), TEXT("Antarctica/DumontDUrville"));
		TestTimezoneCodeSanitization(TEXT("Antarctica/Dumont{}%DUrville"), TEXT("Antarctica/DumontDUrville"));
		TestTimezoneCodeSanitization(TEXT("Antarctica/DumontD'Urville"), TEXT("Antarctica/DumontDUrville"));
		TestTimezoneCodeSanitization(TEXT("Antarctica/DumontDUrville_Dumont"), TEXT("Antarctica/DumontDUrville"));
		TestTimezoneCodeSanitization(TEXT("GMT-8:00"), TEXT("GMT-8:00"));
		TestTimezoneCodeSanitization(TEXT("GMT-8:00{}%"), TEXT("GMT-8:00"));
		TestTimezoneCodeSanitization(TEXT("GMT-{}%8:00"), TEXT("GMT-8:00"));
	}

	// Validate currency code sanitization
	{
		auto TestCurrencyCodeSanitization = [](const FString& InCode, const FString& InExpectedCode)
		{
			const FString SanitizedCode = ICUUtilities::SanitizeCurrencyCode(InCode);
			if (!SanitizedCode.Equals(InExpectedCode, ESearchCase::CaseSensitive))
			{
				FAIL_CHECK(FString::Printf(TEXT("SanitizeCurrencyCode did not produce the expected result (got '%s', expected '%s')"), *SanitizedCode, *InExpectedCode));
			}		
		};

		TestCurrencyCodeSanitization(TEXT("USD"), TEXT("USD"));
		TestCurrencyCodeSanitization(TEXT("USD{}%"), TEXT("USD"));
		TestCurrencyCodeSanitization(TEXT("U{}%SD"), TEXT("USD"));
		TestCurrencyCodeSanitization(TEXT("USDUSD"), TEXT("USD"));
	}

	// Validate canonization of culture names
	{
		auto TestCultureCodeCanonization = [](const FString& InCode, const FString& InExpectedCode)
		{
			const FString CanonizedCode = FCulture::GetCanonicalName(InCode);
			if (!CanonizedCode.Equals(InExpectedCode, ESearchCase::CaseSensitive))
			{
				FAIL_CHECK(FString::Printf(TEXT("GetCanonicalName did not produce the expected result (got '%s', expected '%s')"), *CanonizedCode, *InExpectedCode));
			}
		};

		// Valid codes
		TestCultureCodeCanonization(TEXT(""), TEXT("en-US-POSIX"));
		TestCultureCodeCanonization(TEXT("en"), TEXT("en"));
		TestCultureCodeCanonization(TEXT("en_US"), TEXT("en-US"));
		TestCultureCodeCanonization(TEXT("en_US_POSIX"), TEXT("en-US-POSIX"));
		TestCultureCodeCanonization(TEXT("en_US@POSIX"), TEXT("en-US-POSIX"));
		TestCultureCodeCanonization(TEXT("en_US.utf8"), TEXT("en-US"));
		TestCultureCodeCanonization(TEXT("en_US.utf8@posix"), TEXT("en-US-POSIX"));
		TestCultureCodeCanonization(TEXT("en_IE_PREEURO"), TEXT("en-IE@currency=IEP"));
		TestCultureCodeCanonization(TEXT("en_IE@CURRENCY=IEP"), TEXT("en-IE@currency=IEP"));
		TestCultureCodeCanonization(TEXT("fr@collation=phonebook;calendar=islamic-civil"), TEXT("fr@calendar=islamic-civil;collation=phonebook"));
		TestCultureCodeCanonization(TEXT("sr_Latn_RS_REVISED@currency=USD"), TEXT("sr-Latn-RS-REVISED@currency=USD"));
		
		// Invalid codes
		TestCultureCodeCanonization(TEXT("%%%"), TEXT("en-US-POSIX"));
		TestCultureCodeCanonization(TEXT("en____US_POSIX"), TEXT("en-US-POSIX"));
		TestCultureCodeCanonization(TEXT("en_POSIX"), TEXT("en--POSIX"));
		TestCultureCodeCanonization(TEXT("en__POSIX"), TEXT("en--POSIX"));
		TestCultureCodeCanonization(TEXT("en_US@wooble=USD"), TEXT("en-US"));
		TestCultureCodeCanonization(TEXT("fred_wooble_bob_wibble"), TEXT("en-US-POSIX"));
	}
}

TEST_CASE_NAMED(FICUTextTest, "System::Core::Misc::ICUText", "[.][EditorContext][ClientContext][EngineFilter]")
{
	// Test to make sure that ICUUtilities converts strings correctly

	const FString SourceString("This is a test");
	const FString SourceString2("This is another test");
	icu::UnicodeString ICUString;
	FString ConversionBackStr;

	// ---------------------------------------------------------------------

	ICUUtilities::ConvertString(SourceString, ICUString);
	if (SourceString.Len() != ICUString.countChar32())
	{
		FAIL_CHECK(FString::Printf(TEXT("icu::UnicodeString is the incorrect length (%d; expected %d)."), ICUString.countChar32(), SourceString.Len()));
	}

	ICUUtilities::ConvertString(ICUString, ConversionBackStr);
	if (ICUString.length() != ConversionBackStr.Len())
	{
		FAIL_CHECK(FString::Printf(TEXT("FString is the incorrect length (%d; expected %d)."), ConversionBackStr.Len(), ICUString.countChar32()));
	}
	if (SourceString != ConversionBackStr)
	{
		FAIL_CHECK(FString::Printf(TEXT("FString is has the incorrect converted value ('%s'; expected '%s')."), *ConversionBackStr, *SourceString));
	}

	// ---------------------------------------------------------------------

	ICUUtilities::ConvertString(SourceString2, ICUString);
	if (SourceString2.Len() != ICUString.countChar32())
	{
		FAIL_CHECK(FString::Printf(TEXT("icu::UnicodeString is the incorrect length (%d; expected %d)."), ICUString.countChar32(), SourceString2.Len()));
	}

	ICUUtilities::ConvertString(ICUString, ConversionBackStr);
	if (ICUString.length() != ConversionBackStr.Len())
	{
		FAIL_CHECK(FString::Printf(TEXT("FString is the incorrect length (%d; expected %d)."), ConversionBackStr.Len(), ICUString.countChar32()));
	}
	if (SourceString2 != ConversionBackStr)
	{
		FAIL_CHECK(FString::Printf(TEXT("FString is has the incorrect converted value ('%s'; expected '%s')."), *ConversionBackStr, *SourceString2));
	}

	// ---------------------------------------------------------------------

	ICUUtilities::ConvertString(SourceString, ICUString);
	if (SourceString.Len() != ICUString.countChar32())
	{
		FAIL_CHECK(FString::Printf(TEXT("icu::UnicodeString is the incorrect length (%d; expected %d)."), ICUString.countChar32(), SourceString.Len()));
	}

	ICUUtilities::ConvertString(ICUString, ConversionBackStr);
	if (ICUString.length() != ConversionBackStr.Len())
	{
		FAIL_CHECK(FString::Printf(TEXT("FString is the incorrect length (%d; expected %d)."), ConversionBackStr.Len(), ICUString.countChar32()));
	}
	if (SourceString != ConversionBackStr)
	{
		FAIL_CHECK(FString::Printf(TEXT("FString is has the incorrect converted value ('%s'; expected '%s')."), *ConversionBackStr, *SourceString));
	}
}

TEST_CASE_NAMED(FICUTextUtilities, "System::Core::Misc::ICUTextUtilities", "[.][EditorContext][ClientContext][EngineFilter]")
{
	FInternationalization& I18N = FInternationalization::Get();

	FInternationalization::FCultureStateSnapshot OriginalCultureState;
	I18N.BackupCultureState(OriginalCultureState);

	if (I18N.SetCurrentCulture("fr"))
	{
		if (FTextTransformer::ToLower(FString(TEXT("ÔTE"))) != FString(TEXT("ôte")))
		{
			FAIL_CHECK(FString::Printf(TEXT("FTextTransformer::ToLower doesn't work correctly")));
		}
		
		if (FTextChar::IsWhitespace(TEXT('\u0212')) == true)
		{
			FAIL_CHECK(FString::Printf(TEXT("FTextChar::IsWhitespace can't detect whitespace")));
		}

		if (FTextChar::IsAlpha(TEXT('\u0212')) == false)
		{
			FAIL_CHECK(FString::Printf(TEXT("FTextChar::IsAlpha can't detect fr letter")));
		}

		if (FTextChar::IsAlpha(TEXT('7')) == true || FTextChar::IsAlpha(TEXT('A')) == false)
		{
			FAIL_CHECK(FString::Printf(TEXT("FTextChar::IsAlpha can't detect not letter")));
		}
		
		TCHAR EmojiChars[] = { 0xD83D, 0xDE00, 0x0000 };
		if (FTextChar::IsAlpha(FTextChar::GetCodepoint(EmojiChars)) == true)
		{
			FAIL_CHECK(FString::Printf(TEXT("FTextChar::IsAlpha can't detect surrogate pair")));
		}
	}

	// Restore original culture
	I18N.RestoreCultureState(OriginalCultureState);
}

#endif // #if UE_ENABLE_ICU

#undef LOCTEXT_NAMESPACE 

#endif //WITH_TESTS
