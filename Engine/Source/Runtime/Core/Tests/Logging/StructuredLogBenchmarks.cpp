// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/IntegerSequence.h"
#include "Logging/StructuredLogFormat.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/Invoke.h"
#include "Templates/Overload.h"
#include "Templates/Tuple.h"
#include "Tests/TestHarnessAdapter.h"

#include <catch2/benchmark/catch_benchmark.hpp>

namespace UE
{
static inline constexpr int32 NumTemplateTypes = 10;

// Use the same values for each test for insertion into compact binary
constexpr int32 IntegerValues[NumTemplateTypes] = {
	2118,
	6695,
	15413,
	212350,
	534561,
	3940185,
	4538994,
	22786616,
	96034783,
	226063757,
};

constexpr FUtf8StringView StringValues[NumTemplateTypes] = {
	UTF8TEXTVIEW("Param2118"),
	UTF8TEXTVIEW("Param6695"),
	UTF8TEXTVIEW("Param15413"),
	UTF8TEXTVIEW("Param212350"),
	UTF8TEXTVIEW("Param534561"),
	UTF8TEXTVIEW("Param3940185"),
	UTF8TEXTVIEW("Param4538994"),
	UTF8TEXTVIEW("Param22786616"),
	UTF8TEXTVIEW("Param96034783"),
	UTF8TEXTVIEW("Param226063757"),
};

// Helper to write the same structure into multiple FCbWriter with different leaf field types
template<int32 NumWriters>
class TFieldWriters
{
	TStaticArray<FCbWriter, NumWriters> Writers;

	static void AddValue(FCbWriter& W, int32 Value)
	{
		W.AddInteger(Value);
	}

	static void AddValue(FCbWriter& W, FUtf8StringView Value)
	{
		W.AddString(Value);
	}

public:
	TFieldWriters() = default;
	UE_NONCOPYABLE(TFieldWriters);

	void BeginObject(FUtf8StringView Name)
	{
		for (int32 i=0; i < NumWriters; ++i)
		{
			Writers[i].BeginObject(Name);
		}
	}
	
	void EndObject()
	{
		for (int32 i=0; i < NumWriters; ++i)
		{
			Writers[i].EndObject();
		}
	}

	void BeginArray(FUtf8StringView Name)
	{
		for (int32 i=0; i < NumWriters; ++i)
		{
			Writers[i].BeginArray(Name);
		}
	}
	
	void EndArray()
	{
		for (int32 i=0; i < NumWriters; ++i)
		{
			Writers[i].EndArray();
		}
	}

	template<typename... ArgTypes>
	void AddValues(ArgTypes&&... Args)
	{
		static_assert(sizeof...(Args) == NumWriters);
		([&]<uint32... Indices>(TIntegerSequence<uint32, Indices...>)
		{
			(AddValue(Writers[Indices], Args), ...);
		})(TMakeIntegerSequence<uint32, NumWriters>{});
	}

	template<typename... ArgTypes>
	void AddValues(FUtf8StringView Name, ArgTypes&&... Args)
	{
		static_assert(sizeof...(Args) == NumWriters);
		for (int32 i=0; i < NumWriters; ++i)
		{
			Writers[i].SetName(Name);
		}

		AddValues(Forward<ArgTypes>(Args)...);
	}

	// Save each writer and return a tuple of FCbFieldIterator
	TStaticArray<FCbFieldIterator, NumWriters> Save()
	{
		TStaticArray<FCbFieldIterator, NumWriters> Cb;
		for (int32 i=0; i < NumWriters; ++i)
		{
			Cb[i] = Writers[i].Save();
		}
		return Cb;
	}
};

TEST_CASE_NAMED(FStructuredLogBenchmark_FormatName, "Core::Logging::StructuredLog::Benchmark::FormatName", "[Core][!benchmark]")
{
	TArray<FString> TemplateStrings;
	for (int32 Variant=0; Variant < NumTemplateTypes; ++Variant)
	{
		TStringBuilder<1024> Template;
		Template << TEXTVIEW("Structured log test: ");
		for (int32 ParamIndex=0; ParamIndex <= Variant; ++ParamIndex)
		{
			Template << TEXTVIEW("{Param") << ParamIndex << TEXTVIEW("} ");
		}
		TemplateStrings.Emplace(Template.ToString());
	}
	TArray<UE::FUniqueLogTemplate> Templates;
	Templates.Reserve(NumTemplateTypes);
	for (int32 i=0; i < NumTemplateTypes; ++i)			
	{
		Templates.Emplace(*TemplateStrings[i]);
	}

	TFieldWriters<2> Writers;
	for (int32 i = 0; i < NumTemplateTypes; ++i)
	{
		Writers.AddValues(WriteToUtf8String<64>(TEXTVIEW("Param"), i).ToView(),
			IntegerValues[i],
			StringValues[i]);
	}
	TStaticArray<FCbFieldIterator, 2> Cb = Writers.Save();

	TStringBuilder<1024> Builder;
	BENCHMARK("IntegerFields")
	{
		for (int32 i=0; i < Templates.Num(); ++i)
		{
			TStringBuilder<1024> Builder;
			Templates[i].FormatTo(Builder, Cb[0]);
		}
	};

	BENCHMARK("StringFields")
	{
		for (int32 i=0; i < Templates.Num(); ++i)
		{
			TStringBuilder<1024> Builder;
			Templates[i].FormatTo(Builder, Cb[1]);
		}
	};
}

TEST_CASE_NAMED(FStructuredLogBenchmark_FormatPath, "Core::Logging::StructuredLog::Benchmark::FormatPath", "[Core][!benchmark]")
{
	TArray<FString> TemplateStrings;
	for (int32 Variant=0; Variant < NumTemplateTypes; ++Variant)			
	{
		TStringBuilder<1024> Template;
		Template << TEXTVIEW("Structured log test: ");
		// For variant n, add n fields
		// Each field is nested n levels under an object called Root, e.g. Root/Param0, Root/Param1/Param0
		for (int32 j=0; j < Variant; ++j)
		{
			Template << TEXTVIEW("{Root"); 
			for (int32 k=j; k >= 0; --k)
			{
				Template << TEXTVIEW("/Param") << k;
			}
			Template << TEXTVIEW("} ");
		}
		TemplateStrings.Emplace(Template.ToString());
	}
	TArray<UE::FUniqueLogTemplate> Templates;
	Templates.Reserve(NumTemplateTypes);
	for (int32 i=0; i < NumTemplateTypes; ++i)			
	{
		Templates.Emplace(*TemplateStrings[i], UE::FLogTemplateOptions{ .bAllowSubObjectReferences = true });
	}

	TFieldWriters<2> Writers;
	Writers.BeginObject(UTF8TEXTVIEW("Root"));
	for (int32 i=0; i < NumTemplateTypes; ++i)
	{
		for (int32 j = i; j > 0; --j)
		{
			Writers.BeginObject(WriteToUtf8String<64>(TEXTVIEW("Param"), j).ToView());
		}
		Writers.AddValues(UTF8TEXTVIEW("Param0"), IntegerValues[i], StringValues[i]);
		for (int32 j = i; j > 0; --j)
		{
			Writers.EndObject();
		}
	}
	Writers.EndObject();
	TStaticArray<FCbFieldIterator, 2> Cb = Writers.Save();
	BENCHMARK("IntegerFields")
	{
		for (int32 i=0; i < Templates.Num(); ++i)
		{
			TStringBuilder<1024> Builder;
			Templates[i].FormatTo(Builder, Cb[0]);
		}
	};

	BENCHMARK("StringFields")
	{
		for (int32 i=0; i < Templates.Num(); ++i)
		{
			TStringBuilder<1024> Builder;
			Templates[i].FormatTo(Builder, Cb[1]);
		}
	};
}

TEST_CASE_NAMED(FStructuredLogBenchmark_FormatPath, "Core::Logging::StructuredLog::Benchmark::FormatPathArrayIndex", "[Core][!benchmark]")
{
	TArray<FString> TemplateStrings;
	for (int32 i=0; i < NumTemplateTypes; ++i)			
	{
		int32 Variant = i % NumTemplateTypes;
		TStringBuilder<1024> Template;
		Template << TEXTVIEW("Structured log test: ");
		// For variant n, add n fields
		// Each field is nested n levels under an object called Root and terminates with an array index n, e.g. Root/Param0[0], Root/Param1/Param0[1]
		for (int32 j=0; j < Variant; ++j)
		{
			Template << TEXTVIEW("{Root"); 
			for (int32 k=j; k >= 0; --k)
			{
				Template << TEXTVIEW("/Param") << k;
			}
			Template << TEXTVIEW("[") << j << TEXTVIEW("]} ");
		}
		TemplateStrings.Emplace(Template.ToString());
	}
	TArray<UE::FUniqueLogTemplate> Templates;
	Templates.Reserve(NumTemplateTypes);
	for (int32 i=0; i < NumTemplateTypes; ++i)			
	{
		Templates.Emplace(*TemplateStrings[i], UE::FLogTemplateOptions{ .bAllowSubObjectReferences = true });
	}

	TFieldWriters<2> Writers;
	Writers.BeginObject(UTF8TEXTVIEW("Root"));
	for (int32 i=0; i < NumTemplateTypes; ++i)
	{
		for (int32 j = i; j > 0; --j)
		{
			Writers.BeginObject(WriteToUtf8String<64>(TEXTVIEW("Param"), j).ToView());
		}
		Writers.BeginArray(UTF8TEXTVIEW("Param0"));
		for (int32 j = 0; j < NumTemplateTypes; ++j)
		{
			Writers.AddValues(IntegerValues[j], StringValues[j]);
		}
		Writers.EndArray();
		for (int32 j = i; j > 0; --j)
		{
			Writers.EndObject();
		}
	}
	Writers.EndObject();
	TStaticArray<FCbFieldIterator, 2> Cb= Writers.Save();

	BENCHMARK("IntegerFields")
	{
		for (int32 i=0; i < Templates.Num(); ++i)
		{
			TStringBuilder<1024> Builder;
			Templates[i].FormatTo(Builder, Cb[0]);
		}
	};

	BENCHMARK("StringFields")
	{
		for (int32 i=0; i < Templates.Num(); ++i)
		{
			TStringBuilder<1024> Builder;
			Templates[i].FormatTo(Builder, Cb[1]);
		}
	};
}

}

#endif // WITH_LOW_LEVEL_TESTS