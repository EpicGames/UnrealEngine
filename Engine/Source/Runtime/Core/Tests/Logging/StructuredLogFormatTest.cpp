// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/StructuredLogFormat.h"

#if WITH_TESTS

#include "Containers/UnrealString.h"
#include "Internationalization/Internationalization.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/Timespan.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Tests/TestHarnessAdapter.h"

#define LOCTEXT_NAMESPACE "StructuredLogFormatTest"

namespace UE
{

TEST_CASE_NAMED(FStructuredLogFormatTest, "Core::Logging::StructuredLogFormat", "[Core][SmokeFilter]")
{
	const uint8 Binary[16] = { 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16 };
	const FIoHash ObjectAttachment = FIoHash::HashBuffer(MakeMemoryView(ANSITEXTVIEW("ObjectAttachment")));
	const FIoHash BinaryAttachment = FIoHash::HashBuffer(MakeMemoryView(ANSITEXTVIEW("BinaryAttachment")));
	const FIoHash Hash = FIoHash::HashBuffer(MakeMemoryView(ANSITEXTVIEW("Hash")));
	const FGuid Uuid = FGuid::NewGuidFromHash(FBlake3::HashBuffer(MakeMemoryView(ANSITEXTVIEW("Guid"))));
	const FDateTime DateTime(2025, 4, 10, 11, 15, 30, 123);
	const FTimespan TimeSpan(10, 11, 15, 30, 123456789);
	const FCbObjectId ObjectId = FromGuid(FGuid::NewGuidFromHash(FBlake3::HashBuffer(MakeMemoryView(ANSITEXTVIEW("ObjectId")))));
	const uint8 CustomById[8] = { 17, 18, 19, 20, 21, 22, 23, 24 };
	const uint8 CustomByName[4] = { 25, 26, 27, 28 };

	const auto MakeFields = [&]
	{
		TCbWriter<1024> Fields;

		Fields.BeginObject(ANSITEXTVIEW("Object"));
		Fields.AddInteger(ANSITEXTVIEW("X"), 1);
		Fields.AddInteger(ANSITEXTVIEW("Y"), 2);
		Fields.AddInteger(ANSITEXTVIEW("Z"), 3);
		Fields.EndObject();

		Fields.BeginArray(ANSITEXTVIEW("Array"));
		Fields.AddInteger(1);
		Fields.AddInteger(2);
		Fields.AddInteger(3);
		Fields.EndArray();

		Fields.AddNull(ANSITEXTVIEW("Null"));
		Fields.AddBinary(ANSITEXTVIEW("Binary"), MakeMemoryView(Binary));
		Fields.AddString(ANSITEXTVIEW("String"), ANSITEXTVIEW("\"Quote\" with 4 words."));
		Fields.AddInteger(ANSITEXTVIEW("IntegerNegative"), -64);
		Fields.AddInteger(ANSITEXTVIEW("IntegerPositive"), 63);
		Fields.AddFloat(ANSITEXTVIEW("Float"), 128.25f);
		Fields.AddFloat(ANSITEXTVIEW("Double"), 123.456);
		Fields.AddBool(ANSITEXTVIEW("False"), false);
		Fields.AddBool(ANSITEXTVIEW("True"), true);
		Fields.AddObjectAttachment(ANSITEXTVIEW("ObjectAttachment"), ObjectAttachment);
		Fields.AddBinaryAttachment(ANSITEXTVIEW("BinaryAttachment"), BinaryAttachment);
		Fields.AddHash(ANSITEXTVIEW("Hash"), Hash);
		Fields.AddUuid(ANSITEXTVIEW("Uuid"), Uuid);
		Fields.AddDateTime(ANSITEXTVIEW("DateTime"), DateTime);
		Fields.AddTimeSpan(ANSITEXTVIEW("TimeSpan"), TimeSpan);
		Fields.AddObjectId(ANSITEXTVIEW("ObjectId"), ObjectId);
		Fields.AddCustom(ANSITEXTVIEW("CustomById"), 128, MakeMemoryView(CustomById));
		Fields.AddCustom(ANSITEXTVIEW("CustomByName"), ANSITEXTVIEW("Custom"), MakeMemoryView(CustomByName));

		Fields.BeginObject(ANSITEXTVIEW("ObjectText"));
		Fields.AddInteger(ANSITEXTVIEW("X"), 1);
		Fields.AddInteger(ANSITEXTVIEW("Y"), 2);
		Fields.AddInteger(ANSITEXTVIEW("Z"), 3);
		Fields.AddString(ANSITEXTVIEW("$text"), ANSITEXTVIEW("X=1;Y=2;Z=3"));
		Fields.EndObject();

		Fields.BeginObject(ANSITEXTVIEW("ObjectWithNestedText"));
		Fields.BeginObject(ANSITEXTVIEW("X"));
		Fields.AddInteger(ANSITEXTVIEW("$value"), 1);
		Fields.AddString(ANSITEXTVIEW("$text"), ANSITEXTVIEW("0001"));
		Fields.EndObject();
		Fields.BeginObject(ANSITEXTVIEW("Y"));
		Fields.AddInteger(ANSITEXTVIEW("$value"), 2);
		Fields.AddString(ANSITEXTVIEW("$text"), ANSITEXTVIEW("0002"));
		Fields.EndObject();
		Fields.BeginObject(ANSITEXTVIEW("Z"));
		Fields.AddInteger(ANSITEXTVIEW("$value"), 3);
		Fields.AddString(ANSITEXTVIEW("$text"), ANSITEXTVIEW("0003"));
		Fields.EndObject();
		Fields.EndObject();

		Fields.BeginArray(ANSITEXTVIEW("ArrayWithNestedText"));
		Fields.BeginObject();
		Fields.AddInteger(ANSITEXTVIEW("$value"), 1);
		Fields.AddString(ANSITEXTVIEW("$text"), ANSITEXTVIEW("0001"));
		Fields.EndObject();
		Fields.BeginObject();
		Fields.AddInteger(ANSITEXTVIEW("$value"), 2);
		Fields.AddString(ANSITEXTVIEW("$text"), ANSITEXTVIEW("0002"));
		Fields.EndObject();
		Fields.BeginObject();
		Fields.AddInteger(ANSITEXTVIEW("$value"), 3);
		Fields.AddString(ANSITEXTVIEW("$text"), ANSITEXTVIEW("0003"));
		Fields.EndObject();
		Fields.EndArray();

		Fields.BeginObject(ANSITEXTVIEW("ObjectFormat"));
		Fields.AddInteger(ANSITEXTVIEW("X"), 1);
		Fields.AddInteger(ANSITEXTVIEW("Y"), 2);
		Fields.AddInteger(ANSITEXTVIEW("Z"), 3);
		Fields.AddString(ANSITEXTVIEW("$format"), ANSITEXTVIEW("X={X};Y={Y};Z={Z}"));
		Fields.EndObject();

		Fields.BeginObject(ANSITEXTVIEW("ObjectWithNestedFormat"));
		Fields.BeginObject(ANSITEXTVIEW("Point"));
		Fields.AddInteger(ANSITEXTVIEW("X"), 1);
		Fields.AddInteger(ANSITEXTVIEW("Y"), 2);
		Fields.AddInteger(ANSITEXTVIEW("Z"), 3);
		Fields.AddString(ANSITEXTVIEW("$format"), ANSITEXTVIEW("X={X};Y={Y};Z={Z}"));
		Fields.EndObject();
		Fields.AddString(ANSITEXTVIEW("$format"), ANSITEXTVIEW("Target=({Point}); X={Point/X}"));
		Fields.EndObject();

		Fields.BeginArray(ANSITEXTVIEW("ArrayWithNestedFormat"));
		Fields.BeginObject();
		Fields.AddInteger(ANSITEXTVIEW("X"), 1);
		Fields.AddString(ANSITEXTVIEW("$format"), ANSITEXTVIEW("X={X}"));
		Fields.EndObject();
		Fields.BeginObject();
		Fields.AddInteger(ANSITEXTVIEW("Y"), 2);
		Fields.AddString(ANSITEXTVIEW("$format"), ANSITEXTVIEW("Y={Y}"));
		Fields.EndObject();
		Fields.BeginObject();
		Fields.AddInteger(ANSITEXTVIEW("Z"), 3);
		Fields.AddString(ANSITEXTVIEW("$format"), ANSITEXTVIEW("Z={Z}"));
		Fields.EndObject();
		Fields.EndArray();

		Fields.BeginObject(ANSITEXTVIEW("ObjectLocFormat"));
		Fields.AddInteger(ANSITEXTVIEW("X"), 1);
		Fields.AddInteger(ANSITEXTVIEW("Y"), 2);
		Fields.AddInteger(ANSITEXTVIEW("Z"), 3);
		SerializeLogFormat(Fields, LOCTEXT("ObjectLocFormat", "X={X};Y={Y};Z={Z}"));
		Fields.EndObject();

		Fields.BeginObject(ANSITEXTVIEW("ObjectWithNestedLocFormat"));
		Fields.BeginObject(ANSITEXTVIEW("Point"));
		Fields.AddInteger(ANSITEXTVIEW("X"), 1);
		Fields.AddInteger(ANSITEXTVIEW("Y"), 2);
		Fields.AddInteger(ANSITEXTVIEW("Z"), 3);
		SerializeLogFormat(Fields, LOCTEXT("ObjectWithNestedLocFormatPoint", "X={X};Y={Y};Z={Z}"));
		Fields.EndObject();
		SerializeLogFormat(Fields, LOCTEXT("ObjectWithNestedLocFormat", "Target=({Point}); X={Point/X}"));
		Fields.EndObject();

		Fields.BeginArray(ANSITEXTVIEW("ArrayWithNestedLocFormat"));
		Fields.BeginObject();
		Fields.AddInteger(ANSITEXTVIEW("X"), 1);
		Fields.AddString(ANSITEXTVIEW("$format"), ANSITEXTVIEW("X={X}"));
		Fields.EndObject();
		Fields.BeginObject();
		Fields.AddInteger(ANSITEXTVIEW("Y"), 2);
		Fields.AddString(ANSITEXTVIEW("$format"), ANSITEXTVIEW("Y={Y}"));
		Fields.EndObject();
		Fields.BeginObject();
		Fields.AddInteger(ANSITEXTVIEW("Z"), 3);
		Fields.AddString(ANSITEXTVIEW("$format"), ANSITEXTVIEW("Z={Z}"));
		Fields.EndObject();
		Fields.EndArray();

		return Fields.Save();
	};

	const FCbFieldIterator Fields = MakeFields();

	const auto Test = [&Fields](const TCHAR* Format, FUtf8StringView Expected, const FLogTemplateOptions& Options = {})
	{
		{
			FInlineLogTemplate Template(Format, Options);
			TUtf8StringBuilder<1024> Message;
			Template.FormatTo(Message, Fields);
			// TODO: Requires a fix for CaptureExpressionsAndValues in LowLevelTestAdapter.h
			//CAPTURE(Format, Expected, Message);
			CHECK(Message.ToView().Equals(Expected));
		}
		{
			TUtf8StringBuilder<64> Utf8Format(InPlace, Format);
			FInlineLogTemplate Template(*Utf8Format, Options);
			TUtf8StringBuilder<1024> Message;
			Template.FormatTo(Message, Fields);
			// TODO: Requires a fix for CaptureExpressionsAndValues in LowLevelTestAdapter.h
			//CAPTURE(Format, Expected, Message);
			CHECK(Message.ToView().Equals(Expected));
		}
	};

	// Invalid
	//Test(TEXT("Text } Text"), ANSITEXTVIEW(""));
	//Test(TEXT("Text { Text"), ANSITEXTVIEW(""));
	//Test(TEXT("{Missing}"), ANSITEXTVIEW(""));

	// Test each of the types that can be represented by compact binary.
	Test(TEXT("{Object}"), ANSITEXTVIEW(R"({"X": 1, "Y": 2, "Z": 3})"));
	Test(TEXT("{Array}"), ANSITEXTVIEW(R"([1, 2, 3])"));
	Test(TEXT("{Null}"), ANSITEXTVIEW(R"(null)"));
	Test(TEXT("{Binary}"), ANSITEXTVIEW(R"("AQIDBAUGBwgJCgsMDQ4PEA==")"));
	Test(TEXT("{String}"), ANSITEXTVIEW(R"("Quote" with 4 words.)"));
	Test(TEXT("{IntegerNegative}"), ANSITEXTVIEW(R"(-64)"));
	Test(TEXT("{IntegerPositive}"), ANSITEXTVIEW(R"(63)"));
	Test(TEXT("{Float}"), ANSITEXTVIEW(R"(128.25)"));
	Test(TEXT("{Double}"), ANSITEXTVIEW(R"(123.456)"));
	Test(TEXT("{False}"), ANSITEXTVIEW(R"(false)"));
	Test(TEXT("{True}"), ANSITEXTVIEW(R"(true)"));
	Test(TEXT("{ObjectAttachment}"), ANSITEXTVIEW(R"(cb42395cfe025324d80c31c88746d3392f330e58)"));
	Test(TEXT("{BinaryAttachment}"), ANSITEXTVIEW(R"(0ba7b01905a760046bacb86f092f291924c1f24a)"));
	Test(TEXT("{Hash}"), ANSITEXTVIEW(R"(700b0783bebf169c6d473141e82dd88c67f31ce2)"));
	Test(TEXT("{Uuid}"), ANSITEXTVIEW(R"(b1718295-2b3d-3379-9948-ef46ee9de6c3)"));
	Test(TEXT("{DateTime}"), ANSITEXTVIEW(R"(2025-04-10T11:15:30.123Z)"));
	Test(TEXT("{TimeSpan}"), ANSITEXTVIEW(R"(+10.11:15:30.123456700)"));
	Test(TEXT("{ObjectId}"), ANSITEXTVIEW(R"(354fa969c46eb24d1ea03026)"));
	Test(TEXT("{CustomById}"), ANSITEXTVIEW(R"({"Id":128,"Data":"ERITFBUWFxg="})"));
	Test(TEXT("{CustomByName}"), ANSITEXTVIEW(R"({"Name":"Custom","Data":"GRobHA=="})"));

	// Test an object with a $text field.
	Test(TEXT("{ObjectText}"), ANSITEXTVIEW(R"(X=1;Y=2;Z=3)"));

	// Test an object and an array with nested $text fields.
	Test(TEXT("{ObjectWithNestedText}"), ANSITEXTVIEW(R"({"X": 0001, "Y": 0002, "Z": 0003})"));
	Test(TEXT("{ArrayWithNestedText}"), ANSITEXTVIEW(R"([0001, 0002, 0003])"));

	// Test $format fields.
	Test(TEXT("{ObjectFormat}"), ANSITEXTVIEW(R"(X=1;Y=2;Z=3)"));
	Test(TEXT("{ObjectWithNestedFormat}"), ANSITEXTVIEW(R"(Target=(X=1;Y=2;Z=3); X=1)"));
	Test(TEXT("{ArrayWithNestedFormat}"), ANSITEXTVIEW(R"([X=1, Y=2, Z=3])"));

	// Test $locformat fields.
	Test(TEXT("{ObjectLocFormat}"), ANSITEXTVIEW(R"(X=1;Y=2;Z=3)"));
	Test(TEXT("{ObjectWithNestedLocFormat}"), ANSITEXTVIEW(R"(Target=(X=1;Y=2;Z=3); X=1)"));
	Test(TEXT("{ArrayWithNestedLocFormat}"), ANSITEXTVIEW(R"([X=1, Y=2, Z=3])"));

	// Test accessing sub-object fields.
	Test(TEXT("{{{Object/X}, {Object/Y}, {Object/Z}}}"), ANSITEXTVIEW("{1, 2, 3}"), {.bAllowSubObjectReferences = true});
	Test(TEXT("{ObjectText/X}"), ANSITEXTVIEW("1"), {.bAllowSubObjectReferences = true});
	Test(TEXT("{ObjectWithNestedText/X}"), ANSITEXTVIEW("0001"), {.bAllowSubObjectReferences = true});

	// Test array indexing
	Test(TEXT("{Array[0]}"), ANSITEXTVIEW(R"(1)"), {.bAllowSubObjectReferences = true});
	Test(TEXT("{Array[1]}"), ANSITEXTVIEW(R"(2)"), {.bAllowSubObjectReferences = true});
	Test(TEXT("{Array[2]}"), ANSITEXTVIEW(R"(3)"), {.bAllowSubObjectReferences = true});
	Test(TEXT("{ArrayWithNestedFormat[0]}"), ANSITEXTVIEW(R"(X=1)"), {.bAllowSubObjectReferences = true});
	Test(TEXT("{ArrayWithNestedFormat[1]}"), ANSITEXTVIEW(R"(Y=2)"), {.bAllowSubObjectReferences = true});
	Test(TEXT("{ArrayWithNestedFormat[2]}"), ANSITEXTVIEW(R"(Z=3)"), {.bAllowSubObjectReferences = true});
	Test(TEXT("{ArrayWithNestedFormat[0]/X}"), ANSITEXTVIEW(R"(1)"), {.bAllowSubObjectReferences = true});
	Test(TEXT("{ArrayWithNestedFormat[1]/Y}"), ANSITEXTVIEW(R"(2)"), {.bAllowSubObjectReferences = true});
	Test(TEXT("{ArrayWithNestedFormat[2]/Z}"), ANSITEXTVIEW(R"(3)"), {.bAllowSubObjectReferences = true});

	const auto TestLoc = [&Fields](const FText& FormatText, FUtf8StringView Expected, const FLogTemplateOptions& Options = {})
	{
		const TOptional<FString> Namespace = FTextInspector::GetNamespace(FormatText);
		const TOptional<FString> Key = FTextInspector::GetKey(FormatText);
		const FString* Format = FTextInspector::GetSourceString(FormatText);
		FInlineLogTemplate Template(**Namespace, **Key, **Format, Options);
		TUtf8StringBuilder<1024> Message;
		Template.FormatTo(Message, Fields);
		// TODO: Requires a fix for CaptureExpressionsAndValues in LowLevelTestAdapter.h
		//CAPTURE(Format, Expected, Message);
		CHECK(Message.ToView().Equals(Expected));
	};

	// Invalid
	//TestLoc(LOCTEXT("Invalid1", "Text ` Text"), ANSITEXTVIEW(""));
	//TestLoc(LOCTEXT("Invalid2", "Text } Text"), ANSITEXTVIEW(""));
	//TestLoc(LOCTEXT("Invalid3", "Text { Text"), ANSITEXTVIEW(""));
	//TestLoc(LOCTEXT("Invalid4", "{Missing}"), ANSITEXTVIEW(""));

	// Test with an argument modifier.
	TestLoc(LOCTEXT("EmitterSubheaderText", "Found {IntegerPositive} {IntegerPositive}|plural(one=error,other=errors)!"), ANSITEXTVIEW("Found 63 errors!"));

	// Test each of the types that can be represented by compact binary.
	TestLoc(LOCTEXT("Object", "{Object}"), ANSITEXTVIEW(R"({"X": 1, "Y": 2, "Z": 3})"));
	TestLoc(LOCTEXT("Array", "{Array}"), ANSITEXTVIEW(R"([1, 2, 3])"));
	TestLoc(LOCTEXT("Null", "{Null}"), ANSITEXTVIEW(R"(null)"));
	TestLoc(LOCTEXT("Binary", "{Binary}"), ANSITEXTVIEW(R"("AQIDBAUGBwgJCgsMDQ4PEA==")"));
	TestLoc(LOCTEXT("String", "{String}"), ANSITEXTVIEW(R"("Quote" with 4 words.)"));
	TestLoc(LOCTEXT("IntegerNegative", "{IntegerNegative}"), ANSITEXTVIEW(R"(-64)"));
	TestLoc(LOCTEXT("IntegerPositive", "{IntegerPositive}"), ANSITEXTVIEW(R"(63)"));
	TestLoc(LOCTEXT("Float", "{Float}"), ANSITEXTVIEW(R"(128.25)"));
	TestLoc(LOCTEXT("Double", "{Double}"), ANSITEXTVIEW(R"(123.456)"));
	TestLoc(LOCTEXT("False", "{False}"), ANSITEXTVIEW(R"(false)"));
	TestLoc(LOCTEXT("True", "{True}"), ANSITEXTVIEW(R"(true)"));
	TestLoc(LOCTEXT("ObjectAttachment", "{ObjectAttachment}"), ANSITEXTVIEW(R"(cb42395cfe025324d80c31c88746d3392f330e58)"));
	TestLoc(LOCTEXT("BinaryAttachment", "{BinaryAttachment}"), ANSITEXTVIEW(R"(0ba7b01905a760046bacb86f092f291924c1f24a)"));
	TestLoc(LOCTEXT("Hash", "{Hash}"), ANSITEXTVIEW(R"(700b0783bebf169c6d473141e82dd88c67f31ce2)"));
	TestLoc(LOCTEXT("Uuid", "{Uuid}"), ANSITEXTVIEW(R"(b1718295-2b3d-3379-9948-ef46ee9de6c3)"));
	TestLoc(LOCTEXT("DateTime", "{DateTime}"), ANSITEXTVIEW(R"(2025-04-10T11:15:30.123Z)"));
	TestLoc(LOCTEXT("TimeSpan", "{TimeSpan}"), ANSITEXTVIEW(R"(+10.11:15:30.123456700)"));
	TestLoc(LOCTEXT("ObjectId", "{ObjectId}"), ANSITEXTVIEW(R"(354fa969c46eb24d1ea03026)"));
	TestLoc(LOCTEXT("CustomById", "{CustomById}"), ANSITEXTVIEW(R"({"Id":128,"Data":"ERITFBUWFxg="})"));
	TestLoc(LOCTEXT("CustomByName", "{CustomByName}"), ANSITEXTVIEW(R"({"Name":"Custom","Data":"GRobHA=="})"));

	// Test an object with a $text field.
	TestLoc(LOCTEXT("ObjectText", "{ObjectText}"), ANSITEXTVIEW(R"(X=1;Y=2;Z=3)"));

	// Test an object and an array with nested $text fields.
	TestLoc(LOCTEXT("ObjectWithNestedText", "{ObjectWithNestedText}"), ANSITEXTVIEW(R"({"X": 0001, "Y": 0002, "Z": 0003})"));
	TestLoc(LOCTEXT("ArrayWithNestedText", "{ArrayWithNestedText}"), ANSITEXTVIEW(R"([0001, 0002, 0003])"));

	// Test $format fields.
	TestLoc(LOCTEXT("ObjectWithFormat", "{ObjectFormat}"), ANSITEXTVIEW(R"(X=1;Y=2;Z=3)"));
	TestLoc(LOCTEXT("ObjectWithNestedFormat", "{ObjectWithNestedFormat}"), ANSITEXTVIEW(R"(Target=(X=1;Y=2;Z=3); X=1)"));
	TestLoc(LOCTEXT("ArrayWithNestedFormat", "{ArrayWithNestedFormat}"), ANSITEXTVIEW(R"([X=1, Y=2, Z=3])"));

	// Test $locformat fields.
	TestLoc(LOCTEXT("ObjectWithLocFormatFmt", "{ObjectLocFormat}"), ANSITEXTVIEW(R"(X=1;Y=2;Z=3)"));
	TestLoc(LOCTEXT("ObjectWithNestedLocFormatFmt", "{ObjectWithNestedLocFormat}"), ANSITEXTVIEW(R"(Target=(X=1;Y=2;Z=3); X=1)"));
	TestLoc(LOCTEXT("ArrayWithNestedLocFormatFmt", "{ArrayWithNestedLocFormat}"), ANSITEXTVIEW(R"([X=1, Y=2, Z=3])"));

	// Test accessing sub-object fields.
	TestLoc(LOCTEXT("ObjectX", "`{{Object/X}, {Object/Y}, {Object/Z}`}"), ANSITEXTVIEW("{1, 2, 3}"), {.bAllowSubObjectReferences = true});
	TestLoc(LOCTEXT("ObjectTextX", "{ObjectText/X}"), ANSITEXTVIEW("1"), {.bAllowSubObjectReferences = true});
	TestLoc(LOCTEXT("ObjectWithNestedTextX", "{ObjectWithNestedText/X}"), ANSITEXTVIEW("0001"), {.bAllowSubObjectReferences = true});

	// Test array indexing
	TestLoc(LOCTEXT("ArrayIndex0", "{Array[0]}"), ANSITEXTVIEW(R"(1)"), {.bAllowSubObjectReferences = true});
	TestLoc(LOCTEXT("ArrayIndex1", "{Array[1]}"), ANSITEXTVIEW(R"(2)"), {.bAllowSubObjectReferences = true});
	TestLoc(LOCTEXT("ArrayIndex2", "{Array[2]}"), ANSITEXTVIEW(R"(3)"), {.bAllowSubObjectReferences = true});
	TestLoc(LOCTEXT("ArrayWithNestedFormatIndex0", "{ArrayWithNestedFormat[0]}"), ANSITEXTVIEW(R"(X=1)"), {.bAllowSubObjectReferences = true});
	TestLoc(LOCTEXT("ArrayWithNestedFormatIndex1", "{ArrayWithNestedFormat[1]}"), ANSITEXTVIEW(R"(Y=2)"), {.bAllowSubObjectReferences = true});
	TestLoc(LOCTEXT("ArrayWithNestedFormatIndex2", "{ArrayWithNestedFormat[2]}"), ANSITEXTVIEW(R"(Z=3)"), {.bAllowSubObjectReferences = true});
	TestLoc(LOCTEXT("ArrayWithNestedFormatIndex0X", "{ArrayWithNestedFormat[0]/X}"), ANSITEXTVIEW(R"(1)"), {.bAllowSubObjectReferences = true});
	TestLoc(LOCTEXT("ArrayWithNestedFormatIndex1Y", "{ArrayWithNestedFormat[1]/Y}"), ANSITEXTVIEW(R"(2)"), {.bAllowSubObjectReferences = true});
	TestLoc(LOCTEXT("ArrayWithNestedFormatIndex2Z", "{ArrayWithNestedFormat[2]/Z}"), ANSITEXTVIEW(R"(3)"), {.bAllowSubObjectReferences = true});

	// Test constructing a template from FText.
	{
		FUniqueLogTemplate Template(LOCTEXT("TextFormat", "FText bWorks={True}"));
		TUtf8StringBuilder<24> Message;
		Template.FormatTo(Message, Fields);
		CHECK(Message.ToView().Equals(ANSITEXTVIEW(R"(FText bWorks=true)")));
	}
}

} // UE

#undef LOCTEXT_NAMESPACE

#endif // WITH_TESTS
