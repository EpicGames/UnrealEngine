// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreMinimal.h"
#include "Math/Color.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FFLinearColorSmokeTest, "System::Core::Math::FLinearColor::Smoke Test", "[Core][Math][SmokeFilter]")
{
	SECTION("FLinearColor")
	{
		FLinearColor black(FLinearColor::Black);
		FLinearColor white(FLinearColor::White);
		FLinearColor red(FLinearColor::Red);

		CHECK(red.R == 1.0);
		CHECK(red.G == 0.0); 
		CHECK(red.B == 0.0);
		CHECK(red.A == 1.0);
		CHECK(red == FLinearColor::Red);
		CHECK(red != FLinearColor::Green);

		CHECK(white.IsAlmostBlack() == false);
		CHECK(black.IsAlmostBlack() == true);

		FLinearColor yellow(FColor::FromHex(FString("FFFF00FF")));
		FLinearColor green(FColor::FromHex(FString("00FF00FF")));
		FLinearColor blue(FColor::FromHex(FString("0000FFFF")));

		CHECK(yellow == FLinearColor::Yellow);
		CHECK(green == FLinearColor::Green);
		CHECK(blue == FLinearColor::Blue);
		CHECK(blue != yellow);
		CHECK(green != yellow);
		CHECK(yellow == yellow);
	}
}

TEST_CASE_NAMED(FFColorSmokeTest, "System::Core::Math::FColor::Smoke Test", "[Core][Math][SmokeFilter]")
{
	SECTION("FColor")
	{
		FColor black(FColor::Black);
		FColor white(FColor::White);
		FColor red(FColor::Red);

		CHECK(red.R == 0xFF);
		CHECK(red.G == 0x00);
		CHECK(red.B == 0x00);
		CHECK(red.A == 0xFF);
		CHECK(red == FColor::Red);
		CHECK(red != FColor::Green);

		FColor yellow(FColor::FromHex(FString("FFFF00FF")));
		FColor green(FColor::FromHex(FString("00FF00FF")));
		FColor blue(FColor::FromHex(FString("0000FFFF")));

		CHECK(yellow == FColor::Yellow);
		CHECK(green == FColor::Green);
		CHECK(blue == FColor::Blue);
		CHECK(blue != yellow);
		CHECK(green != yellow);
		CHECK(yellow == yellow);

		CHECK(1 == 1);
	}
}

#endif //WITH_TESTS