// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Math/GenericOctree.h"
#include "Misc/AutomationTest.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_TESTS

namespace UE::MathTest
{
	struct FTestOctreeElement
	{
		FTestOctreeElement() = default;

		explicit FTestOctreeElement(int InValue, const FBoxCenterAndExtent& InBounds)
			: Value(InValue)
			, Bounds(InBounds)
		{
		}

		int32 Value = 0;
		FBoxCenterAndExtent Bounds = {};
	};

	struct TTestOctreeSemantics
	{
		enum { MaxElementsPerLeaf = 2 };
		enum { MinInclusiveElementsPerNode = 7 };
		enum { MaxNodeDepth = 12 };

		typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

		static FBoxCenterAndExtent GetBoundingBox(const FTestOctreeElement& InPoint)
		{
			return InPoint.Bounds;
		}

		static const bool AreElementsEqual(const FTestOctreeElement& Lhs, const FTestOctreeElement& Rhs)
		{
			return Lhs.Value == Rhs.Value;
		}

		static void ApplyOffset(const FTestOctreeElement& Element, const FVector& Offset)
		{
		}

		static void SetElementId(const FTestOctreeElement& Element, FOctreeElementId2 OctreeElementID)
		{
		}
	};

	TEST_CASE_NAMED(FGenericOctreeTest, "System::Core::Math::GenericOctree::ApplyOffset", "[ApplicationContextMask][EngineFilter]")
	{
		// Ensure that no assert fires when applying an offset
		TOctree2<FTestOctreeElement, TTestOctreeSemantics> Octree;
		for (int32 Count = 20; Count != 0; --Count)
		{
			FVector Center((float)Count, (float)Count, (float)Count);
			FVector Extent = Center + FVector(1., 1., 1.);
			Octree.AddElement(FTestOctreeElement(5, FBoxCenterAndExtent(Center, Extent)));
		}
		Octree.ApplyOffset(FVector{});
	}
}

#endif // WITH_TESTS
