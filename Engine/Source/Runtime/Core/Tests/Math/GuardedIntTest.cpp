// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/GuardedInt.h"
#include "Misc/AutomationTest.h"
#include <limits>
#include <type_traits>

#if WITH_DEV_AUTOMATION_TESTS

namespace {
template<typename IntType>
bool DoGuardedIntTestForType(FAutomationTestBase& Test)
{
	// IntType may only be int8 or uint8.
	static_assert(std::is_same_v<IntType, int8> || std::is_same_v<IntType, uint8>);

	constexpr bool bIsSigned = std::is_signed_v<IntType>;
	using IntType32 = std::conditional_t<bIsSigned, int32, uint32>;
	constexpr IntType32 IntTypeMin = std::numeric_limits<IntType>::min();
	constexpr IntType32 IntTypeMax = std::numeric_limits<IntType>::max();

	// Do an exhaustive test of all checked 8-bit int operations. This is a small enough space of possibilities
	// that it's easy to try all options exhaustively, and it's likewise easy to compute reference results in int32 arithmetic
	// without having to worry about overflows. TGuardedInt uses the same logic and implementation for all signed type
	// instantiations (as of this writing), so this is a good way to get coverage on all relevant code paths.
	using FGuardedIntType = TGuardedInt<IntType>;
	int NumErrorsFound = 0;
	const int NumErrorsMax = 100; // Stop reporting errors after this in case something is completely busted.

	// Helper functions to log test results
	auto CheckArithResult = [&Test, &NumErrorsFound, NumErrorsMax](const TCHAR* Operation, IntType32 A, IntType32 B, FGuardedIntType CheckedResult, IntType32 UncheckedValue)
	{
		FGuardedIntType UncheckedResult{ UncheckedValue };

		// If either operand was invalid, the unchecked result is also invalid
		if (!FGuardedIntType(A).IsValid() || !FGuardedIntType(B).IsValid())
		{
			UncheckedResult = FGuardedIntType();
		}

		if (CheckedResult != UncheckedResult && NumErrorsFound < NumErrorsMax)
		{
			NumErrorsFound++;

			const IntType32 UncheckedNum = UncheckedResult.IsValid() ? UncheckedResult.GetChecked() : (IntTypeMax + 1);
			const IntType32 CheckedNum = CheckedResult.IsValid() ? CheckedResult.GetChecked() : (IntTypeMax + 1);

			Test.AddError(FString::Printf(TEXT("Arith %s A=%d B=%d Expected=(%d,Valid=%d) Got=(%d,Valid=%d)"),
				Operation, A, B, UncheckedNum, UncheckedResult.IsValid(), CheckedNum, CheckedResult.IsValid()));
		}
	};

	auto CheckCompareResult = [&Test, &NumErrorsFound, NumErrorsMax](const TCHAR* Operation, IntType32 A, IntType32 B, bool bAcceptInvalid, bool bTestResult, bool bRefResult)
	{
		// If either operands was invalid, override the ref result
		if (!FGuardedIntType(A).IsValid() || !FGuardedIntType(B).IsValid())
			bRefResult = bAcceptInvalid;

		if (bTestResult != bRefResult && NumErrorsFound < NumErrorsMax)
		{
			NumErrorsFound++;

			Test.AddError(FString::Printf(TEXT("Cmp %s A=%d B=%d ref=%d test=%d\n"), Operation, A, B, bRefResult, bTestResult));
		}
	};

	// Note: we include IntTypeMax + 1 in the ranges here to explicitly try out-of-range (invalid) values.
	for (IntType32 A = IntTypeMin; A <= IntTypeMax + 1; ++A)
	{
		for (IntType32 B = IntTypeMin; B <= IntTypeMax + 1; ++B)
		{
			FGuardedIntType CheckedA{ A };
			FGuardedIntType CheckedB{ B };

			// This doesn't test much by itself but ensures we have at least compiled GetChecked.
			if (A >= IntTypeMin && A <= IntTypeMax && B == 0)
			{
				if (CheckedA.GetChecked() != A)
				{
					Test.AddError(FString::Printf(TEXT("GetChecked A=%d"), A));
				}
			}

			// Compute reference results for the operations that are not defined everywhere
			IntType32 ADivB = -256;
			IntType32 AModB = -256;

			// -128 / -1 is undefined in int8 arithmetic because it overflows; when we're
			// computing it with int32s, it of course works just fine, but we want to make sure
			// our test results are correct.
			if (B != 0 && !(bIsSigned && A == IntTypeMin && B == -1))
			{
				ADivB = A / B;
				AModB = A % B;
			}

			// Shifts are only defined for in-range shift amounts, which for our checked ints
			// is [0,NumBits-1]. This is tighter than usual C++ arithmetic for the FGuardedIntType we
			// use for testing since normal C++ promotes to int and shifts down (so usually [0,31]
			// count limit) so we need to test for these explicitly.
			IntType32 ALshB = -256;
			IntType32 ARshB = -256;
			if (B >= 0 && B <= 7)
			{
				ALshB = A << B;
				ARshB = A >> B;
			}

			CheckArithResult(TEXT("add1"), A, B, CheckedA + CheckedB, A + B);
			CheckArithResult(TEXT("sub1"), A, B, CheckedA - CheckedB, A - B);
			CheckArithResult(TEXT("mul1"), A, B, CheckedA * CheckedB, A * B);
			CheckArithResult(TEXT("div1"), A, B, CheckedA / CheckedB, ADivB);
			CheckArithResult(TEXT("mod1"), A, B, CheckedA % CheckedB, AModB);
			CheckArithResult(TEXT("lsh1"), A, B, CheckedA << CheckedB, ALshB);
			CheckArithResult(TEXT("rsh1"), A, B, CheckedA >> CheckedB, ARshB);
			if (bIsSigned || A == 0)
			{
				CheckArithResult(TEXT("neg"), A, A, -CheckedA, static_cast<IntType32>(-static_cast<int32>(A)));
			}
			CheckArithResult(TEXT("abs"), A, A, CheckedA.Abs(), static_cast<IntType32>(FMath::Abs(static_cast<int32>(A))));

			// Mixed-type arithmetic needs int8 arguments, which we can only actually do when the values in question _are_ int8s.
			// (i.e. we can't do this with our intentional out-of-range value of 128.)
			if (CheckedB.IsValid())
			{
				IntType BAs8 = (IntType)B;

				CheckArithResult(TEXT("add2"), A, B, CheckedA + BAs8, A + B);
				CheckArithResult(TEXT("sub2"), A, B, CheckedA - BAs8, A - B);
				CheckArithResult(TEXT("mul2"), A, B, CheckedA * BAs8, A * B);
				CheckArithResult(TEXT("div2"), A, B, CheckedA / BAs8, ADivB);
				CheckArithResult(TEXT("mod2"), A, B, CheckedA % BAs8, AModB);
				CheckArithResult(TEXT("lsh2"), A, B, CheckedA << BAs8, ALshB);
				CheckArithResult(TEXT("rsh2"), A, B, CheckedA >> BAs8, ARshB);
			}

			if (CheckedA.IsValid())
			{
				IntType AAs8 = (IntType)A;

				CheckArithResult(TEXT("add3"), A, B, AAs8 + CheckedB, A + B);
				CheckArithResult(TEXT("sub3"), A, B, AAs8 - CheckedB, A - B);
				CheckArithResult(TEXT("mul3"), A, B, AAs8 * CheckedB, A * B);
				CheckArithResult(TEXT("div3"), A, B, AAs8 / CheckedB, ADivB);
				CheckArithResult(TEXT("mod3"), A, B, AAs8 % CheckedB, AModB);
				CheckArithResult(TEXT("lsh3"), A, B, AAs8 << CheckedB, ALshB);
				CheckArithResult(TEXT("rsh3"), A, B, AAs8 >> CheckedB, ARshB);
			}

			CheckCompareResult(TEXT("valid_lt1"), A, B, false, CheckedA.ValidAndLessThan(CheckedB), A < B);
			CheckCompareResult(TEXT("valid_lt2"), A, B, false, CheckedA.ValidAndLessThan(B), A < B);
			CheckCompareResult(TEXT("valid_le1"), A, B, false, CheckedA.ValidAndLessOrEqual(CheckedB), A <= B);
			CheckCompareResult(TEXT("valid_le1"), A, B, false, CheckedA.ValidAndLessOrEqual(B), A <= B);
			CheckCompareResult(TEXT("valid_gt1"), A, B, false, CheckedA.ValidAndGreaterThan(CheckedB), A > B);
			CheckCompareResult(TEXT("valid_gt1"), A, B, false, CheckedA.ValidAndGreaterThan(B), A > B);
			CheckCompareResult(TEXT("valid_ge1"), A, B, false, CheckedA.ValidAndGreaterOrEqual(CheckedB), A >= B);
			CheckCompareResult(TEXT("valid_ge2"), A, B, false, CheckedA.ValidAndGreaterOrEqual(B), A >= B);

			CheckCompareResult(TEXT("invalid_lt1"), A, B, true, CheckedA.InvalidOrLessThan(CheckedB), A < B);
			CheckCompareResult(TEXT("invalid_lt2"), A, B, true, CheckedA.InvalidOrLessThan(B), A < B);
			CheckCompareResult(TEXT("invalid_le1"), A, B, true, CheckedA.InvalidOrLessOrEqual(CheckedB), A <= B);
			CheckCompareResult(TEXT("invalid_le2"), A, B, true, CheckedA.InvalidOrLessOrEqual(B), A <= B);
			CheckCompareResult(TEXT("invalid_gt1"), A, B, true, CheckedA.InvalidOrGreaterThan(CheckedB), A > B);
			CheckCompareResult(TEXT("invalid_gt2"), A, B, true, CheckedA.InvalidOrGreaterThan(B), A > B);
			CheckCompareResult(TEXT("invalid_ge1"), A, B, true, CheckedA.InvalidOrGreaterOrEqual(CheckedB), A >= B);
			CheckCompareResult(TEXT("invalid_ge2"), A, B, true, CheckedA.InvalidOrGreaterOrEqual(B), A >= B);
		}
	}

	if (NumErrorsFound > NumErrorsMax)
	{
		Test.AddError(FString::Printf(TEXT("Too many errors (%d), stopped reporting after %d instances.\n"), NumErrorsFound, NumErrorsMax));
	}

	return NumErrorsFound == 0;
}
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGuardedIntTest, "System.Core.GuardedInt", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGuardedIntTest::RunTest(const FString&)
{
	bool bResult = true;
	bResult &= DoGuardedIntTestForType<int8>(*this);
	bResult &= DoGuardedIntTestForType<uint8>(*this);
	return bResult;
}

#endif