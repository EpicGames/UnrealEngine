// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Math/IntVector.h"
#include "Tests/TestHarnessAdapter.h"

// UE::Math::TIntVector2

template <typename IntType>
struct TIntVector2TestData
{
};

template <>
struct TIntVector2TestData<int32>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0");
	static inline FStringView MinMaxString = TEXTVIEW("X=-2147483648 Y=2147483647");
};

template <>
struct TIntVector2TestData<int64>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0");
	static inline FStringView MinMaxString = TEXTVIEW("X=-9223372036854775808 Y=9223372036854775807");
};

template <>
struct TIntVector2TestData<uint32>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0");
	static inline FStringView MinMaxString = TEXTVIEW("X=0 Y=4294967295");
};

template <>
struct TIntVector2TestData<uint64>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0");
	static inline FStringView MinMaxString = TEXTVIEW("X=0 Y=18446744073709551615");
};

template <typename IntType>
void TestIntVector2()
{
	using FIntVector2Type = UE::Math::TIntVector2<IntType>;
	using FIntVector2TestData = TIntVector2TestData<IntType>;

	constexpr IntType MinValue = TNumericLimits<IntType>::Min();
	constexpr IntType MaxValue = TNumericLimits<IntType>::Max();

	// GetMax
	CHECK(FIntVector2Type(1, 2).GetMax() == 2);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector2Type( 2, -2).GetMax() == 2);
		CHECK(FIntVector2Type(-2, -4).GetMax() == -2);
	}

	// GetAbsMax
	CHECK(FIntVector2Type(1, 2).GetAbsMax() == 2);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector2Type( 2, -2).GetAbsMax() == 2);
		CHECK(FIntVector2Type(-2, -4).GetAbsMax() == 4);
	}

	// GetMin
	CHECK(FIntVector2Type(1, 2).GetMin() == 1);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector2Type( 2, -2).GetMin() == -2);
		CHECK(FIntVector2Type(-2, -4).GetMin() == -4);
	}

	// GetAbsMin
	CHECK(FIntVector2Type(1, 2).GetAbsMin() == 1);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector2Type( 2, -2).GetAbsMin() == 2);
		CHECK(FIntVector2Type(-2, -4).GetAbsMin() == 2);
	}

	// ComponentMax
	CHECK(FIntVector2Type(1, 2).ComponentMax(FIntVector2Type(2, 1)) == FIntVector2Type(2, 2));
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector2Type(-1, -2).ComponentMax(FIntVector2Type(-2, -1)) == FIntVector2Type(-1, -1));
	}

	// ComponentMin
	CHECK(FIntVector2Type(1, 2).ComponentMin(FIntVector2Type(2, 1)) == FIntVector2Type(1, 1));
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector2Type(-1, -2).ComponentMin(FIntVector2Type(-2, -1)) == FIntVector2Type(-2, -2));
	}

	// AppendString(TStringBuilderBase<TCHAR>) via operator<<(TStringBuilderBase<TCHAR>, FIntVector2Type)
	TStringBuilder<128> Builder;
	CHECK((Builder << FIntVector2Type(0, 0)) == FIntVector2TestData::ZeroString);
	Builder.Reset();
	CHECK((Builder << FIntVector2Type(MinValue, MaxValue)) == FIntVector2TestData::MinMaxString);

	// ToString
	CHECK(FIntVector2Type(0, 0).ToString() == FIntVector2TestData::ZeroString);
	CHECK(FIntVector2Type(MinValue, MaxValue).ToString() == FIntVector2TestData::MinMaxString);

	// InitFromString
	FIntVector2Type InitFromStringVec;
	CHECK((InitFromStringVec.InitFromString(FString(FIntVector2TestData::ZeroString)) && InitFromStringVec == FIntVector2Type(0, 0)));
	CHECK((InitFromStringVec.InitFromString(FString(FIntVector2TestData::MinMaxString)) && InitFromStringVec == FIntVector2Type(MinValue, MaxValue)));
}

TEST_CASE_NAMED(FInt32Vector2Test, "System::Core::Math::FInt32Vector2", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector2<int32>();
}

TEST_CASE_NAMED(FInt64Vector2Test, "System::Core::Math::FInt64Vector2", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector2<int64>();
}

TEST_CASE_NAMED(FUint32Vector2Test, "System::Core::Math::FUint32Vector2", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector2<uint32>();
}

TEST_CASE_NAMED(FUint64Vector2Test, "System::Core::Math::FUint64Vector2", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector2<uint64>();
}

// UE::Math::TIntVector3

template <typename IntType>
struct TIntVector3TestData
{
};

template <>
struct TIntVector3TestData<int32>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0 Z=0");
	static inline FStringView MinZeroMaxString = TEXTVIEW("X=-2147483648 Y=0 Z=2147483647");
};

template <>
struct TIntVector3TestData<int64>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0 Z=0");
	static inline FStringView MinZeroMaxString = TEXTVIEW("X=-9223372036854775808 Y=0 Z=9223372036854775807");
};

template <>
struct TIntVector3TestData<uint32>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0 Z=0");
	static inline FStringView MinZeroMaxString = TEXTVIEW("X=0 Y=0 Z=4294967295");
};

template <>
struct TIntVector3TestData<uint64>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0 Z=0");
	static inline FStringView MinZeroMaxString = TEXTVIEW("X=0 Y=0 Z=18446744073709551615");
};

template <typename IntType>
void TestIntVector3()
{
	using FIntVector3Type = UE::Math::TIntVector3<IntType>;
	using FIntVector3TestData = TIntVector3TestData<IntType>;

	constexpr IntType MinValue = TNumericLimits<IntType>::Min();
	constexpr IntType MaxValue = TNumericLimits<IntType>::Max();

	// GetMax
	CHECK(FIntVector3Type(0, 1, 2).GetMax() == 2);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector3Type( 2,  0, -2).GetMax() == 2);
		CHECK(FIntVector3Type(-2, -4, -6).GetMax() == -2);
	}

	// GetAbsMax
	CHECK(FIntVector3Type(0, 1, 2).GetAbsMax() == 2);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector3Type( 2,  0, -2).GetAbsMax() == 2);
		CHECK(FIntVector3Type(-2, -4, -6).GetAbsMax() == 6);
	}

	// GetMin
	CHECK(FIntVector3Type(0, 1, 2).GetMin() == 0);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector3Type( 2,  0, -2).GetMin() == -2);
		CHECK(FIntVector3Type(-2, -4, -6).GetMin() == -6);
	}

	// GetAbsMin
	CHECK(FIntVector3Type(0, 1, 2).GetAbsMin() == 0);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector3Type( 2,  0, -2).GetAbsMin() == 0);
		CHECK(FIntVector3Type(-2, -4, -6).GetAbsMin() == 2);
	}

	// ComponentMax
	CHECK(FIntVector3Type(0, 1, 2).ComponentMax(FIntVector3Type(2, 1, 0)) == FIntVector3Type(2, 1, 2));
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector3Type(-1, -2, -3).ComponentMax(FIntVector3Type(-3, -2, -1)) == FIntVector3Type(-1, -2, -1));
	}

	// ComponentMin
	CHECK(FIntVector3Type(0, 1, 2).ComponentMin(FIntVector3Type(2, 1, 0)) == FIntVector3Type(0, 1, 0));
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector3Type(-1, -2, -3).ComponentMin(FIntVector3Type(-3, -2, -1)) == FIntVector3Type(-3, -2, -3));
	}

	// AppendString(TStringBuilderBase<TCHAR>) via operator<<(TStringBuilderBase<TCHAR>, FIntVector3Type)
	TStringBuilder<128> Builder;
	CHECK((Builder << FIntVector3Type(0, 0, 0)) == FIntVector3TestData::ZeroString);
	Builder.Reset();
	CHECK((Builder << FIntVector3Type(MinValue, 0, MaxValue)) == FIntVector3TestData::MinZeroMaxString);

	// ToString
	CHECK(FIntVector3Type(0, 0, 0).ToString() == FIntVector3TestData::ZeroString);
	CHECK(FIntVector3Type(MinValue, 0, MaxValue).ToString() == FIntVector3TestData::MinZeroMaxString);

	// InitFromString
	FIntVector3Type InitFromStringVec;
	CHECK((InitFromStringVec.InitFromString(FString(FIntVector3TestData::ZeroString)) && InitFromStringVec == FIntVector3Type(0, 0, 0)));
	CHECK((InitFromStringVec.InitFromString(FString(FIntVector3TestData::MinZeroMaxString)) && InitFromStringVec == FIntVector3Type(MinValue, 0, MaxValue)));
}

TEST_CASE_NAMED(FInt32Vector3Test, "System::Core::Math::FInt32Vector3", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector3<int32>();
}

TEST_CASE_NAMED(FInt64Vector3Test, "System::Core::Math::FInt64Vector3", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector3<int64>();
}

TEST_CASE_NAMED(FUint32Vector3Test, "System::Core::Math::FUint32Vector3", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector3<uint32>();
}

TEST_CASE_NAMED(FUint64Vector3Test, "System::Core::Math::FUint64Vector3", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector3<uint64>();
}

// UE::Math::TIntVector4

template <typename IntType>
struct TIntVector4TestData
{
};

template <>
struct TIntVector4TestData<int32>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0 Z=0 W=0");
	static inline FStringView MinZeroMaxOneString = TEXTVIEW("X=-2147483648 Y=0 Z=2147483647 W=1");
};

template <>
struct TIntVector4TestData<int64>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0 Z=0 W=0");
	static inline FStringView MinZeroMaxOneString = TEXTVIEW("X=-9223372036854775808 Y=0 Z=9223372036854775807 W=1");
};

template <>
struct TIntVector4TestData<uint32>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0 Z=0 W=0");
	static inline FStringView MinZeroMaxOneString = TEXTVIEW("X=0 Y=0 Z=4294967295 W=1");
};

template <>
struct TIntVector4TestData<uint64>
{
	static inline FStringView ZeroString = TEXTVIEW("X=0 Y=0 Z=0 W=0");
	static inline FStringView MinZeroMaxOneString = TEXTVIEW("X=0 Y=0 Z=18446744073709551615 W=1");
};

template <typename IntType>
void TestIntVector4()
{
	using FIntVector4Type = UE::Math::TIntVector4<IntType>;
	using FIntVector4TestData = TIntVector4TestData<IntType>;

	constexpr IntType MinValue = TNumericLimits<IntType>::Min();
	constexpr IntType MaxValue = TNumericLimits<IntType>::Max();

	// GetMax
	CHECK(FIntVector4Type(0, 1, 2, 3).GetMax() == 3);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector4Type( 2,  0, -2,  0).GetMax() == 2);
		CHECK(FIntVector4Type(-2, -4, -6, -8).GetMax() == -2);
	}

	// GetAbsMax
	CHECK(FIntVector4Type(0, 1, 2, 3).GetAbsMax() == 3);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector4Type( 2,  0, -2,  0).GetAbsMax() == 2);
		CHECK(FIntVector4Type(-2, -4, -6, -8).GetAbsMax() == 8);
	}

	// GetMin
	CHECK(FIntVector4Type(0, 1, 2, 3).GetMin() == 0);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector4Type( 2,  0, -2,  0).GetMin() == -2);
		CHECK(FIntVector4Type(-2, -4, -6, -8).GetMin() == -8);
	}

	// GetAbsMin
	CHECK(FIntVector4Type(0, 1, 2, 3).GetAbsMin() == 0);
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector4Type( 2,  0, -2,  0).GetAbsMin() == 0);
		CHECK(FIntVector4Type(-2, -4, -6, -8).GetAbsMin() == 2);
	}

	// ComponentMax
	CHECK(FIntVector4Type(0, 1, 2, 3).ComponentMax(FIntVector4Type(3, 2, 1, 0)) == FIntVector4Type(3, 2, 2, 3));
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector4Type(-1, -2, -3, -4).ComponentMax(FIntVector4Type(-4, -3, -2, -1)) == FIntVector4Type(-1, -2, -2, -1));
	}

	// ComponentMin
	CHECK(FIntVector4Type(0, 1, 2, 3).ComponentMin(FIntVector4Type(3, 2, 1, 0)) == FIntVector4Type(0, 1, 1, 0));
	if constexpr (std::is_signed_v<IntType>)
	{
		CHECK(FIntVector4Type(-1, -2, -3, -4).ComponentMin(FIntVector4Type(-4, -3, -2, -1)) == FIntVector4Type(-4, -3, -3, -4));
	}

	// AppendString(TStringBuilderBase<TCHAR>) via operator<<(TStringBuilderBase<TCHAR>, FIntVector4Type)
	TStringBuilder<128> Builder;
	CHECK((Builder << FIntVector4Type(0, 0, 0, 0)) == FIntVector4TestData::ZeroString);
	Builder.Reset();
	CHECK((Builder << FIntVector4Type(MinValue, 0, MaxValue, 1)) == FIntVector4TestData::MinZeroMaxOneString);

	// ToString
	CHECK(FIntVector4Type(0, 0, 0, 0).ToString() == FIntVector4TestData::ZeroString);
	CHECK(FIntVector4Type(MinValue, 0, MaxValue, 1).ToString() == FIntVector4TestData::MinZeroMaxOneString);

	// InitFromString
	FIntVector4Type InitFromStringVec;
	CHECK((InitFromStringVec.InitFromString(FString(FIntVector4TestData::ZeroString)) && InitFromStringVec == FIntVector4Type(0, 0, 0, 0)));
	CHECK((InitFromStringVec.InitFromString(FString(FIntVector4TestData::MinZeroMaxOneString)) && InitFromStringVec == FIntVector4Type(MinValue, 0, MaxValue, 1)));
}

TEST_CASE_NAMED(FInt32Vector4Test, "System::Core::Math::FInt32Vector4", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector4<int32>();
}

TEST_CASE_NAMED(FInt64Vector4Test, "System::Core::Math::FInt64Vector4", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector4<int64>();
}

TEST_CASE_NAMED(FUint32Vector4Test, "System::Core::Math::FUint32Vector4", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector4<uint32>();
}

TEST_CASE_NAMED(FUint64Vector4Test, "System::Core::Math::FUint64Vector4", "[ApplicationContextMask][SmokeFilter]")
{
	TestIntVector4<uint64>();
}

#endif // WITH_TESTS
