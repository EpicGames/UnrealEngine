// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Math/Range.h"

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FRangeBoundTest, "System::Core::Math::RangeBound", "[EditorContext][ClientContext][SmokeFilter]")
{
	// ensure template instantiation works
	FDateRange DateRange;
	FDoubleRange DoubleRange;
	FFloatRange FloatRange;
	FInt8Range Int8Range;
	FInt16Range Int16Range;
	FInt32Range Int32Range;
	FInt64Range Int64Range;

	// bound types must be correct after construction
	FFloatRangeBound b1_1 = FFloatRangeBound::Exclusive(2.0f);
	FFloatRangeBound b1_2 = FFloatRangeBound::Inclusive(2.0f);
	FFloatRangeBound b1_3 = FFloatRangeBound::Open();
	FFloatRangeBound b1_4 = 2;

	CHECK_MESSAGE(TEXT("Exclusive bound constructor must create exclusive bound"), b1_1.IsExclusive());
	CHECK_MESSAGE(TEXT("Exclusive bound constructor must create closed bound"), b1_1.IsClosed());
	CHECK_FALSE_MESSAGE(TEXT("Exclusive bound constructor must not create inclusive bound"), b1_1.IsInclusive());
	CHECK_FALSE_MESSAGE(TEXT("Exclusive bound constructor must not create open bound"), b1_1.IsOpen());
	CHECK_MESSAGE(TEXT("Exclusive bound constructor must create the correct value"), b1_1.GetValue() == 2.0f);

	CHECK_MESSAGE(TEXT("Inclusive bound constructor must create exclusive bound"), b1_2.IsInclusive());
	CHECK_MESSAGE(TEXT("Inclusive bound constructor must create closed bound"), b1_2.IsClosed());
	CHECK_FALSE_MESSAGE(TEXT("Inclusive bound constructor must not create exclusive bound"), b1_2.IsExclusive());
	CHECK_FALSE_MESSAGE(TEXT("Inclusive bound constructor must not create open bound"), b1_2.IsOpen());
	CHECK_MESSAGE(TEXT("Inclusive bound constructor must create the correct value"), b1_2.GetValue() == 2.0f);

	CHECK_MESSAGE(TEXT("Open bound constructor must create open bound"), b1_3.IsOpen());
	CHECK_FALSE_MESSAGE(TEXT("Open bound constructor must not create closed bound"), b1_3.IsClosed());
	CHECK_FALSE_MESSAGE(TEXT("Open bound constructor must not create exclusive bound"), b1_3.IsExclusive());
	CHECK_FALSE_MESSAGE(TEXT("Open bound constructor must not create inclusive bound"), b1_3.IsInclusive());

	CHECK_MESSAGE(TEXT("Implicit constructor must create an inclusive bound"), b1_4.IsInclusive());
	CHECK_MESSAGE(TEXT("Implicit constructor must create the correct value"), b1_4 == b1_2);

	// comparisons must be correct
	FFloatRangeBound b2_1 = FFloatRangeBound::Exclusive(2.0f);
	FFloatRangeBound b2_2 = FFloatRangeBound::Exclusive(2.0f);
	FFloatRangeBound b2_3 = FFloatRangeBound::Inclusive(2.0f);
	FFloatRangeBound b2_4 = FFloatRangeBound::Inclusive(2.0f);
	FFloatRangeBound b2_5 = FFloatRangeBound::Open();
	FFloatRangeBound b2_6 = FFloatRangeBound::Open();

	CHECK_MESSAGE(TEXT("Equal exclusive bounds must be equal"), b2_1 == b2_2);
	CHECK_MESSAGE(TEXT("Equal inclusive bounds must be equal"), b2_3 == b2_4);
	CHECK_MESSAGE(TEXT("Open bounds must be equal"), b2_5 == b2_6);

	CHECK_FALSE_MESSAGE(TEXT("Equal exclusive bounds must not be unequal"), b2_1 != b2_2);
	CHECK_FALSE_MESSAGE(TEXT("Equal inclusive bounds must not be unequal"), b2_3 != b2_4);
	CHECK_FALSE_MESSAGE(TEXT("Open bounds must not be unequal"), b2_5 != b2_6);

	FFloatRangeBound b2_7 = FFloatRangeBound::Exclusive(3.0f);
	FFloatRangeBound b2_8 = FFloatRangeBound::Inclusive(3.0f);

	CHECK_MESSAGE(TEXT("Unequal exclusive bounds must be unequal"), b2_1 != b2_7);
	CHECK_MESSAGE(TEXT("Unequal inclusive bounds must be unequal"), b2_2 != b2_8);

	CHECK_FALSE_MESSAGE(TEXT("Unequal exclusive bounds must not be equal"), b2_1 == b2_7);
	CHECK_FALSE_MESSAGE(TEXT("Unequal inclusive bounds must not be equal"), b2_2 == b2_8);

	// min-max comparisons between bounds must be correct
	FFloatRangeBound b3_1 = FFloatRangeBound::Exclusive(2.0f);
	FFloatRangeBound b3_2 = FFloatRangeBound::Inclusive(2.0f);
	FFloatRangeBound b3_3 = FFloatRangeBound::Open();

	CHECK_MESSAGE(TEXT("'[2' must be less than '(2' <1>"), FFloatRangeBound::MinLower(b3_2, b3_1) == b3_2);
	CHECK_MESSAGE(TEXT("'[2' must be less than '(2' <2>"), FFloatRangeBound::MinLower(b3_1, b3_2) == b3_2);
	CHECK_MESSAGE(TEXT("Open lower bound must be less than '(2' <1>"), FFloatRangeBound::MinLower(b3_3, b3_1) == b3_3);
	CHECK_MESSAGE(TEXT("Open lower bound must be less than '(2' <2>"), FFloatRangeBound::MinLower(b3_1, b3_3) == b3_3);
	CHECK_MESSAGE(TEXT("Open lower bound must be less than '[2' <1>"), FFloatRangeBound::MinLower(b3_3, b3_2) == b3_3);
	CHECK_MESSAGE(TEXT("Open lower bound must be less than '[2' <2>"), FFloatRangeBound::MinLower(b3_2, b3_3) == b3_3);

	CHECK_MESSAGE(TEXT("'(2' must be greater than '[2' <1>"), FFloatRangeBound::MaxLower(b3_2, b3_1) == b3_1);
	CHECK_MESSAGE(TEXT("'(2' must be greater than '[2' <2>"), FFloatRangeBound::MaxLower(b3_1, b3_2) == b3_1);
	CHECK_MESSAGE(TEXT("'(2' must be greater than open lower bound <1>"), FFloatRangeBound::MaxLower(b3_3, b3_1) == b3_1);
	CHECK_MESSAGE(TEXT("'(2' must be greater than open lower bound <2>"), FFloatRangeBound::MaxLower(b3_1, b3_3) == b3_1);
	CHECK_MESSAGE(TEXT("'[2' must be greater than open lower bound <1>"), FFloatRangeBound::MaxLower(b3_3, b3_2) == b3_2);
	CHECK_MESSAGE(TEXT("'[2' must be greater than open lower bound <2>"), FFloatRangeBound::MaxLower(b3_2, b3_3) == b3_2);

	CHECK_MESSAGE(TEXT("'2)' must be less than '2]' <1>"), FFloatRangeBound::MinUpper(b3_2, b3_1) == b3_1);
	CHECK_MESSAGE(TEXT("'2)' must be less than '2]' <2>"), FFloatRangeBound::MinUpper(b3_1, b3_2) == b3_1);
	CHECK_MESSAGE(TEXT("'2)' must be less than open upper bound <1>"), FFloatRangeBound::MinUpper(b3_3, b3_1) == b3_1);
	CHECK_MESSAGE(TEXT("'2)' must be less than open upper bound <2>"), FFloatRangeBound::MinUpper(b3_1, b3_3) == b3_1);
	CHECK_MESSAGE(TEXT("'2]' must be less than open upper bound <1>"), FFloatRangeBound::MinUpper(b3_3, b3_2) ==b3_2);
	CHECK_MESSAGE(TEXT("'2]' must be less than open upper bound"), FFloatRangeBound::MinUpper(b3_2, b3_3) ==b3_2);

	CHECK_MESSAGE(TEXT("'2]' must be greater than '2)' <1>"), FFloatRangeBound::MaxUpper(b3_2, b3_1) == b3_2);
	CHECK_MESSAGE(TEXT("'2]' must be greater than '2)' <2>"), FFloatRangeBound::MaxUpper(b3_1, b3_2) == b3_2);
	CHECK_MESSAGE(TEXT("Open upper bound must be greater than '2)' <1>"), FFloatRangeBound::MaxUpper(b3_3, b3_1) == b3_3);
	CHECK_MESSAGE(TEXT("Open upper bound must be greater than '2)' <2>"), FFloatRangeBound::MaxUpper(b3_1, b3_3) == b3_3);
	CHECK_MESSAGE(TEXT("Open upper bound must be greater than '2]' <1>"), FFloatRangeBound::MaxUpper(b3_3, b3_2) == b3_3);
	CHECK_MESSAGE(TEXT("Open upper bound must be greater than '2]' <2>"), FFloatRangeBound::MaxUpper(b3_2, b3_3) ==b3_3);

	FFloatRangeBound b4_1 = FFloatRangeBound::Exclusive(3);
	FFloatRangeBound b4_2 = FFloatRangeBound::Inclusive(3);

	CHECK_MESSAGE(TEXT("'(2' must be less than '[3' <1>"), FFloatRangeBound::MinLower(b3_1, b4_2) == b3_1);
	CHECK_MESSAGE(TEXT("'(2' must be less than '[3' <2>"), FFloatRangeBound::MinLower(b4_2, b3_1) == b3_1);
	CHECK_MESSAGE(TEXT("'[2' must be less than '[3' <1>"), FFloatRangeBound::MinLower(b3_2, b4_2) == b3_2);
	CHECK_MESSAGE(TEXT("'[2' must be less than '[3' <2>"), FFloatRangeBound::MinLower(b4_2, b3_2) == b3_2);

	CHECK_MESSAGE(TEXT("'[3' must be greater than '(2' <1>"), FFloatRangeBound::MaxLower(b3_1, b4_2) == b4_2);
	CHECK_MESSAGE(TEXT("'[3' must be greater than '(2' <2>"), FFloatRangeBound::MaxLower(b4_2, b3_1) == b4_2);
	CHECK_MESSAGE(TEXT("'[3' must be greater than '[2' <1>"), FFloatRangeBound::MaxLower(b3_2, b4_2) == b4_2);
	CHECK_MESSAGE(TEXT("'[3' must be greater than '[2' <2>"), FFloatRangeBound::MaxLower(b4_2, b3_2) == b4_2);
}

#endif //WITH_TESTS
