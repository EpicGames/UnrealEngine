// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Math/Range.h"
#include "Math/RangeSet.h"
#include "Misc/Timespan.h"

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FRangeSetTest, "System::Core::Math::RangeSet", "[EditorContext][ClientContext][SmokeFilter]")
{
	{
		TRangeSet<int32> RangeSet;
		RangeSet.Add(TRange<int32>::Inclusive(0, 1));
		RangeSet.Add(TRange<int32>::Inclusive(1, 2));
		RangeSet.Add(TRange<int32>::Inclusive(3, 4));

		int32 Value = RangeSet.GetMinBoundValue();
		REQUIRE(Value == 0);
		
		Value = RangeSet.GetMaxBoundValue();
		REQUIRE(Value == 4);
	}

	{
		TRangeSet<FTimespan> RangeSet;
		RangeSet.Add(TRange<FTimespan>::Inclusive(FTimespan(0), FTimespan(1)));
		RangeSet.Add(TRange<FTimespan>::Inclusive(FTimespan(1), FTimespan(2)));
		RangeSet.Add(TRange<FTimespan>::Inclusive(FTimespan(3), FTimespan(4)));

		FTimespan Value = RangeSet.GetMinBoundValue();
		REQUIRE(Value == FTimespan::Zero());

		Value = RangeSet.GetMaxBoundValue();
		REQUIRE(Value == FTimespan(4));
	}
}

#endif //WITH_TESTS
