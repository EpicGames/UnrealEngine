// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Math/Range.h"

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FRangeTest, "System::Core::Math::Range", "[EditorContext][ClientContext][SmokeFilter]")
{
	// single element constructor
	FFloatRange r1_1 = FFloatRange(3.0f);

	CHECK_MESSAGE(TEXT("Single element constructor must create the correct lower bound value"), r1_1.GetLowerBoundValue() == 3.0f);
	CHECK_MESSAGE(TEXT("Single element constructor must create the correct upper bound value"), r1_1.GetUpperBoundValue() == 3.0f);
	CHECK_MESSAGE(TEXT("Single element constructor must create an inclusive lower bound"), r1_1.GetLowerBound().IsInclusive());
	CHECK_MESSAGE(TEXT("Single element constructor must create an inclusive upper bound"), r1_1.GetUpperBound().IsInclusive());
	CHECK_FALSE_MESSAGE(TEXT("Single element constructor must create non-empty range"), r1_1.IsEmpty());

	// explicit element pair constructor
	FFloatRange r2_1 = FFloatRange(1.0f, 4.0f);

	CHECK_MESSAGE(TEXT("Explicit element pair constructor must create the correct lower bound value"), r2_1.GetLowerBoundValue() == 1.0f);
	CHECK_MESSAGE(TEXT("Explicit element pair constructor must create the correct upper bound value"), r2_1.GetUpperBoundValue() == 4.0f);
	CHECK_MESSAGE(TEXT("Explicit element pair constructor must create an inclusive lower bound"), r2_1.GetLowerBound().IsInclusive());
	CHECK_MESSAGE(TEXT("Explicit element pair constructor must create an exclusive upper bound"), r2_1.GetUpperBound().IsExclusive());
	CHECK_FALSE_MESSAGE(TEXT("Explicit element pair constructor must create non-empty range"), r2_1.IsEmpty());

	// bound pair constructors
	FFloatRange r3_1 = FFloatRange(3.0f);
	FFloatRange r3_2 = FFloatRange(1.0f, 4.0f);
	FFloatRange r3_3 = FFloatRange(FFloatRangeBound(3.0f), FFloatRangeBound(3.0f));
	FFloatRange r3_4 = FFloatRange(FFloatRangeBound::Inclusive(1.0f), FFloatRangeBound::Exclusive(4.0f));

	CHECK_MESSAGE(TEXT("Default bound pair constructor must create the correct range"), r3_3 == r3_1);
	CHECK_FALSE_MESSAGE(TEXT("Default bound pair constructor must create non-empty range"), r3_3.IsEmpty());
	CHECK_MESSAGE(TEXT("Specific bound pair constructor must create the correct range"), r3_4 == r3_2);
	CHECK_FALSE_MESSAGE(TEXT("Specific bound pair constructor must create non-empty range"), r3_4.IsEmpty());

	// empty ranges
	FFloatRange r4_1 = FFloatRange::Empty();
	FFloatRange r4_2 = FFloatRange(4.0f, 1.0f);
	FFloatRange r4_3 = FFloatRange(FFloatRangeBound::Inclusive(6.0f), FFloatRangeBound::Exclusive(2.0f));
	FFloatRange r4_4 = FFloatRange(FFloatRangeBound::Exclusive(4.0f), FFloatRangeBound::Exclusive(4.0f));
	FFloatRange r4_5 = FFloatRange(FFloatRangeBound::Exclusive(4.0f), FFloatRangeBound::Inclusive(4.0f));
	FFloatRange r4_6 = FFloatRange(FFloatRangeBound::Inclusive(4.0f), FFloatRangeBound::Exclusive(4.0f));

	CHECK_MESSAGE(TEXT("Empty range must be empty <1>"), r4_1.IsEmpty());
	CHECK_MESSAGE(TEXT("Empty range must be empty <2>"), r4_2.IsEmpty());
	CHECK_MESSAGE(TEXT("Empty range must be empty <3>"), r4_3.IsEmpty());
	CHECK_MESSAGE(TEXT("Empty range must be empty <4>"), r4_4.IsEmpty());
	CHECK_MESSAGE(TEXT("Empty range must be empty <5>"), r4_5.IsEmpty());
	CHECK_MESSAGE(TEXT("Empty range must be empty <6>"), r4_6.IsEmpty());

	CHECK_MESSAGE(TEXT("Empty ranges must be equal <1>"), r4_1 == r4_2);
	CHECK_MESSAGE(TEXT("Empty ranges must be equal <2>"), r4_1 == r4_3);
	CHECK_MESSAGE(TEXT("Empty ranges must be equal <3>"), r4_1 == r4_4);
	CHECK_MESSAGE(TEXT("Empty ranges must be equal <4>"), r4_1 == r4_5);
	CHECK_MESSAGE(TEXT("Empty ranges must be equal <5>"), r4_1 == r4_6);

	// inclusive-exclusive comparison
	FFloatRange r5_1 = FFloatRange(FFloatRangeBound::Exclusive(1.0f), FFloatRangeBound::Exclusive(4.0f));
	FFloatRange r5_2 = FFloatRange(FFloatRangeBound::Exclusive(1.0f), FFloatRangeBound::Inclusive(4.0f));
	FFloatRange r5_3 = FFloatRange(FFloatRangeBound::Inclusive(1.0f), FFloatRangeBound::Exclusive(4.0f));
	FFloatRange r5_4 = FFloatRange(FFloatRangeBound::Inclusive(1.0f), FFloatRangeBound::Inclusive(4.0f));

	CHECK_MESSAGE(TEXT("Same inclusive-exclusive ranges must be equal <1>"), r5_1 == r5_1);
	CHECK_MESSAGE(TEXT("Same inclusive-exclusive ranges must be equal <2>"), r5_2 == r5_2);
	CHECK_MESSAGE(TEXT("Same inclusive-exclusive ranges must be equal <3>"), r5_3 == r5_3);
	CHECK_MESSAGE(TEXT("Same inclusive-exclusive ranges must be equal <4>"), r5_4 == r5_4);

	CHECK_FALSE_MESSAGE(TEXT("Different inclusive-exclusive ranges must be different <1>"), r5_1 == r5_2);
	CHECK_FALSE_MESSAGE(TEXT("Different inclusive-exclusive ranges must be different <2>"), r5_1 == r5_3);
	CHECK_FALSE_MESSAGE(TEXT("Different inclusive-exclusive ranges must be different <3>"), r5_1 == r5_4);
	CHECK_FALSE_MESSAGE(TEXT("Different inclusive-exclusive ranges must be different <4>"), r5_2 == r5_3);
	CHECK_FALSE_MESSAGE(TEXT("Different inclusive-exclusive ranges must be different <5>"), r5_2 == r5_4);
	CHECK_FALSE_MESSAGE(TEXT("Different inclusive-exclusive ranges must be different <6>"), r5_3 == r5_4);

	// adjoined ranges
	FFloatRange r6_1 = FFloatRange(FFloatRangeBound::Exclusive(1.0f), FFloatRangeBound::Exclusive(4.0f));
	FFloatRange r6_2 = FFloatRange(FFloatRangeBound::Exclusive(1.0f), FFloatRangeBound::Inclusive(4.0f));
	FFloatRange r6_3 = FFloatRange(FFloatRangeBound::Exclusive(4.0f), FFloatRangeBound::Inclusive(8.0f));
	FFloatRange r6_4 = FFloatRange(FFloatRangeBound::Inclusive(4.0f), FFloatRangeBound::Inclusive(8.0f));
	FFloatRange r6_5 = FFloatRange(FFloatRangeBound::Inclusive(3.0f), FFloatRangeBound::Inclusive(8.0f));
	FFloatRange r6_6 = FFloatRange(FFloatRangeBound::Inclusive(0.0f), FFloatRangeBound::Inclusive(2.0f));

	CHECK_MESSAGE(TEXT("Adjoined ranges must be adjoined"), r6_1.Adjoins(r6_4));
	CHECK_MESSAGE(TEXT("Adjoined ranges must be adjoined"), r6_2.Adjoins(r6_3));
	CHECK_FALSE_MESSAGE(TEXT("Separated ranges must not be adjoined"), r6_1.Adjoins(r6_3));
	CHECK_FALSE_MESSAGE(TEXT("Separated ranges must not be adjoined"), r6_2.Adjoins(r6_4));
	CHECK_FALSE_MESSAGE(TEXT("Overlapped ranges must not be adjoined <1>"), r6_1.Adjoins(r6_5));
	CHECK_FALSE_MESSAGE(TEXT("Overlapped ranges must not be adjoined <2>"), r6_1.Adjoins(r6_6));

	FFloatRange r6_7 = FFloatRange(FFloatRangeBound::Exclusive(4.0f), FFloatRangeBound::Exclusive(2.0f));
	FFloatRange r6_8 = FFloatRange(FFloatRangeBound::Inclusive(4.0f), FFloatRangeBound::Inclusive(2.0f));

	CHECK_FALSE_MESSAGE(TEXT("A non-empty and an empty range must not be adjoined <1>"), r6_1.Adjoins(r6_8));
	CHECK_FALSE_MESSAGE(TEXT("A non-empty and an empty range must not be adjoined <2>"), r6_2.Adjoins(r6_7));

	FFloatRange r6_9 = FFloatRange(FFloatRangeBound::Inclusive(2.0f), FFloatRangeBound::Exclusive(0.0f));
	FFloatRange r6_10 = FFloatRange(FFloatRangeBound::Exclusive(2.0f), FFloatRangeBound::Exclusive(0.0f));

	CHECK_FALSE_MESSAGE(TEXT("Two empty ranges must not be adjoined <1>"), r6_7.Adjoins(r6_9));
	CHECK_FALSE_MESSAGE(TEXT("Two empty ranges must not be adjoined <2>"), r6_8.Adjoins(r6_10));

	// @todo gmp: conjoined ranges

	// @todo gmp: element containment

	// @todo gmp: range containment

	// @todo gmp: contiguous ranges

	// bounds access
	FFloatRange r11_1 = FFloatRange(FFloatRangeBound::Inclusive(1.0f), FFloatRangeBound::Exclusive(4.0f));
	FFloatRange r11_2 = FFloatRange(FFloatRangeBound::Open(), FFloatRangeBound::Open());

	CHECK_MESSAGE(TEXT("A closed range must have a lower bound"), r11_1.HasLowerBound());
	CHECK_MESSAGE(TEXT("A closed range must have an upper bound"), r11_1.HasUpperBound());
	CHECK_FALSE_MESSAGE(TEXT("An open range must not have a lower bound"), r11_2.HasLowerBound());
	CHECK_FALSE_MESSAGE(TEXT("An open range must not have an upper bound"), r11_2.HasUpperBound());

	CHECK_MESSAGE(TEXT("The lower bound value of [1, 4) must be 1"), r11_1.GetLowerBoundValue() == 1.0f);
	CHECK_MESSAGE(TEXT("The upper bound value of [1, 4) must be 4"), r11_1.GetUpperBoundValue() == 4.0f);

	// degenerate ranges
	FFloatRange r12_1 = FFloatRange(3.0f);
	FFloatRange r12_2 = FFloatRange(FFloatRangeBound::Inclusive(3.0f), FFloatRangeBound::Inclusive(3.0f));

	CHECK_MESSAGE(TEXT("A range with a single element must be degenerate <1>"), r12_1.IsDegenerate());
	CHECK_MESSAGE(TEXT("A range with a single element must be degenerate <2>"), r12_2.IsDegenerate());

	// @todo gmp: overlapping ranges
	FFloatRange r14_1 = FFloatRange(FFloatRangeBound::Exclusive(0.0f), FFloatRangeBound::Exclusive(2.0f));
	FFloatRange r14_2 = FFloatRange(FFloatRangeBound::Inclusive(0.0f), FFloatRangeBound::Inclusive(2.0f));
	FFloatRange r14_3 = FFloatRange(FFloatRangeBound::Exclusive(2.0f), FFloatRangeBound::Exclusive(4.0f));
	FFloatRange r14_4 = FFloatRange(FFloatRangeBound::Inclusive(2.0f), FFloatRangeBound::Inclusive(4.0f));
	FFloatRange r14_5 = FFloatRange(FFloatRangeBound::Inclusive(3.0f), FFloatRangeBound::Inclusive(5.0f));
	FFloatRange r14_6 = FFloatRange(FFloatRangeBound::Inclusive(1.0f), FFloatRangeBound::Inclusive(3.0f));
	FFloatRange r14_7 = FFloatRange(FFloatRangeBound::Exclusive(1.0f), FFloatRangeBound::Inclusive(3.0f));
	FFloatRange r14_8 = FFloatRange(FFloatRangeBound::Inclusive(1.0f), FFloatRangeBound::Exclusive(3.0f));
	FFloatRange r14_9 = FFloatRange(FFloatRangeBound::Exclusive(6.0f), FFloatRangeBound::Exclusive(10.0f));
	FFloatRange r14_10 = FFloatRange(FFloatRangeBound::Inclusive(4.0f), FFloatRangeBound::Open());
	FFloatRange r14_11 = FFloatRange(FFloatRangeBound::Open(), FFloatRangeBound::Exclusive(3.0f));

	CHECK_MESSAGE(TEXT("(2, 4) must overlap [3, 5]"), r14_3.Overlaps(r14_5));
	CHECK_MESSAGE(TEXT("(2, 4) must overlap [1, 3]"), r14_3.Overlaps(r14_6));
	CHECK_MESSAGE(TEXT("(2, 4) must overlap (1, 3]"), r14_3.Overlaps(r14_7));
	CHECK_MESSAGE(TEXT("(2, 4) must overlap [1, 3)"), r14_3.Overlaps(r14_8));
	CHECK_MESSAGE(TEXT("(2, 4) must overlap [2, 4]"), r14_3.Overlaps(r14_4));
	CHECK_MESSAGE(TEXT("(2, 4) must overlap itself"), r14_3.Overlaps(r14_3));
	CHECK_MESSAGE(TEXT("(2, 4) must not overlap (0, 2)"), !r14_3.Overlaps(r14_1));
	CHECK_MESSAGE(TEXT("(2, 4) must not overlap [0, 2]"), !r14_3.Overlaps(r14_2));
	CHECK_MESSAGE(TEXT("[2, 4] must not overlap (0, 2)"), !r14_4.Overlaps(r14_1));
	CHECK_MESSAGE(TEXT("[2, 4] must overlap [0, 2]"), r14_4.Overlaps(r14_2));
	CHECK_MESSAGE(TEXT("[2, 4] must not overlap (6, 10)"), !r14_4.Overlaps(r14_9));
	CHECK_MESSAGE(TEXT("[2, 4] must overlap [4, inf)"), r14_4.Overlaps(r14_10));
	CHECK_MESSAGE(TEXT("(2, 4) must not overlap [4, inf)"), !r14_3.Overlaps(r14_10));
	CHECK_MESSAGE(TEXT("[1, 3] must not overlap [4, inf)"), !r14_6.Overlaps(r14_10));
	CHECK_MESSAGE(TEXT("[1, 3] must overlap (inf, 3)"), r14_6.Overlaps(r14_11));
	CHECK_MESSAGE(TEXT("(6, 10) must not overlap (inf, 3)"), !r14_9.Overlaps(r14_11));

	// @todo gmp: range splitting

	// difference
	FFloatRange r15_1 = FFloatRange(FFloatRangeBound::Exclusive(7.0f), FFloatRangeBound::Exclusive(14.0f));		// X
	FFloatRange r15_2 = FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Inclusive(14.0f));

	CHECK_MESSAGE(TEXT("The difference between a range and itself must be an empty set <1>"), FFloatRange::Difference(r15_1, r15_1).Num() == 0);
	CHECK_MESSAGE(TEXT("The difference between a range and itself must be an empty set <2>"), FFloatRange::Difference(r15_2, r15_2).Num() == 0);

	TArray<FFloatRange > d15_0 = FFloatRange::Difference(r15_2, r15_1);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - (7, 14) must result in two ranges"), d15_0.Num() == 2);

		bool b15_1 = (d15_0[0] == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Inclusive(7.0f)));
		bool b15_2 = (d15_0[1] == FFloatRange(FFloatRangeBound::Inclusive(14.0f), FFloatRangeBound::Inclusive(14.0f)));

		CHECK_MESSAGE(TEXT("[7, 14] - (7, 14) must be {[7, 7], [14, 14]}"), b15_1 && b15_2);
	}

	FFloatRange r15_3 = FFloatRange(FFloatRangeBound::Inclusive(2.0f), FFloatRangeBound::Exclusive(9.0f));		// Y overlapping left
	FFloatRange r15_4 = FFloatRange(FFloatRangeBound::Inclusive(2.0f), FFloatRangeBound::Inclusive(9.0f));
	FFloatRange r15_5 = FFloatRange(FFloatRangeBound::Exclusive(8.0f), FFloatRangeBound::Exclusive(17.0f));		// Y overlapping right
	FFloatRange r15_6 = FFloatRange(FFloatRangeBound::Inclusive(8.0f), FFloatRangeBound::Exclusive(17.0f));

	TArray<FFloatRange > d15_1 = FFloatRange::Difference(r15_1, r15_3);
	{
		CHECK_MESSAGE(TEXT("(7, 14) - [2, 9) must result in one range"), d15_1.Num() == 1);
		CHECK_MESSAGE(TEXT("(7, 14) - [2, 9) must be {[9, 14)}"), d15_1[0] == FFloatRange(FFloatRangeBound::Inclusive(9.0f), FFloatRangeBound::Exclusive(14.0f)));
	}

	TArray<FFloatRange > d15_2 = FFloatRange::Difference(r15_1, r15_4);
	{
		CHECK_MESSAGE(TEXT("(7, 14) - [2, 9] must result in one range"), d15_2.Num() == 1);
		CHECK_MESSAGE(TEXT("(7, 14) - [2, 9] must be {(9, 14)}"), d15_2[0] == FFloatRange(FFloatRangeBound::Exclusive(9.0f), FFloatRangeBound::Exclusive(14.0f)));
	}

	TArray<FFloatRange > d15_3 = FFloatRange::Difference(r15_1, r15_5);
	{
		CHECK_MESSAGE(TEXT("(7, 14) - (8, 17) must result in one range"), d15_3.Num() == 1);
		CHECK_MESSAGE(TEXT("(7, 14) - (8, 17) must be {(7, 8]}"), d15_3[0] == FFloatRange(FFloatRangeBound::Exclusive(7.0f), FFloatRangeBound::Inclusive(8.0f)));
	}

	TArray<FFloatRange > d15_4 = FFloatRange::Difference(r15_1, r15_6);
	{
		CHECK_MESSAGE(TEXT("(7, 14) - [8, 17) must result in one range"), d15_4.Num() == 1);
		CHECK_MESSAGE(TEXT("(7, 14) - [8, 17) must be {(7, 8)}"), d15_4[0] == FFloatRange(FFloatRangeBound::Exclusive(7.0f), FFloatRangeBound::Exclusive(8.0f)));
	}

	TArray<FFloatRange > d15_5 = FFloatRange::Difference(r15_2, r15_3);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - [2, 9) must result in one range"), d15_5.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] - [2, 9) must be {[9, 14]}"), d15_5[0] == FFloatRange(FFloatRangeBound::Inclusive(9.0f), FFloatRangeBound::Inclusive(14.0f)));
	}

	TArray<FFloatRange > d15_6 = FFloatRange::Difference(r15_2, r15_4);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - [2, 9] must result in one range"), d15_6.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] - [2, 9] must be {(9, 14]}"), d15_6[0] == FFloatRange(FFloatRangeBound::Exclusive(9.0f), FFloatRangeBound::Inclusive(14.0f)));
	}

	TArray<FFloatRange > d15_7 = FFloatRange::Difference(r15_2, r15_5);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - (8, 17) must result in one range"), d15_7.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] - (8, 17) must be {[7, 8]}"), d15_7[0] == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Inclusive(8.0f)));
	}

	TArray<FFloatRange > d15_8 = FFloatRange::Difference(r15_2, r15_6);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - [8, 17) must result in one range"), d15_8.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] - [8, 17) must be {[7, 8)}"), d15_8[0] == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Exclusive(8.0f)));
	}

	FFloatRange r15_7 = FFloatRange(FFloatRangeBound::Inclusive(2.0f), FFloatRangeBound::Exclusive(7.0f));		// Y adjoint left
	FFloatRange r15_8 = FFloatRange(FFloatRangeBound::Inclusive(2.0f), FFloatRangeBound::Inclusive(7.0f));

	TArray<FFloatRange > d15_9 = FFloatRange::Difference(r15_1, r15_8);
	{
		CHECK_MESSAGE(TEXT("(7, 14) - [2, 7] must result in one set"), d15_9.Num() == 1);
		CHECK_MESSAGE(TEXT("(7, 14) - [2, 7] must be {(7, 14)}"), d15_9[0] == FFloatRange(FFloatRangeBound::Exclusive(7.0f), FFloatRangeBound::Exclusive(14.0f)));
	}

	TArray<FFloatRange > d15_10 = FFloatRange::Difference(r15_2, r15_7);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - [2, 7) must result in one set"), d15_10.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] - [2, 7) must be {[7, 14]}"), d15_10[0] == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Inclusive(14.0f)));
	}

	TArray<FFloatRange > d15_11 = FFloatRange::Difference(r15_2, r15_8);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - [2, 7] must result in one range"), d15_11.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] - [2, 7] must be {(7, 14]}"), d15_11[0] == FFloatRange(FFloatRangeBound::Exclusive(7.0f), FFloatRangeBound::Inclusive(14.0f)));
	}

	FFloatRange r15_9 = FFloatRange(FFloatRangeBound::Exclusive(14.0f), FFloatRangeBound::Exclusive(16.0f));	// Y adjoint right
	FFloatRange r15_10 = FFloatRange(FFloatRangeBound::Inclusive(14.0f), FFloatRangeBound::Inclusive(16.0f));

	TArray<FFloatRange > d15_12 = FFloatRange::Difference(r15_1, r15_10);
	{
		CHECK_MESSAGE(TEXT("(7, 14) - [14, 18] must result in one range"), d15_12.Num() == 1);
		CHECK_MESSAGE(TEXT("(7, 14) - [14, 18] must be {(7, 14)}"), d15_12[0] == FFloatRange(FFloatRangeBound::Exclusive(7.0f), FFloatRangeBound::Exclusive(14.0f)));
	}

	TArray<FFloatRange > d15_13 = FFloatRange::Difference(r15_2, r15_9);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - (14, 18) must result in one range"), d15_13.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] - (14, 18) must be {[7, 14]}"), d15_13[0] == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Inclusive(14.0f)));
	}

	TArray<FFloatRange > d15_14 = FFloatRange::Difference(r15_2, r15_10);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - [14, 18] must result in one range"), d15_14.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] - [14, 18] must be {(7, 14)}"), d15_14[0] == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Exclusive(14.0f)));
	}

	FFloatRange r15_11 = FFloatRange(FFloatRangeBound::Inclusive(8.0f), FFloatRangeBound::Inclusive(13.0f));	// Y enclosed

	TArray<FFloatRange > d15_15 = FFloatRange::Difference(r15_2, r15_11);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - [8, 13] must result in two ranges"), d15_15.Num() == 2);

		bool b15_1 = (d15_15[0] == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Exclusive(8.0f)));
		bool b15_2 = (d15_15[1] == FFloatRange(FFloatRangeBound::Exclusive(13.0f), FFloatRangeBound::Inclusive(14.0f)));

		CHECK_MESSAGE(TEXT("[7, 14] - [8, 13] must be {[7, 8), (13, 14]}"), b15_1 && b15_2);
	}

	FFloatRange r15_12 = FFloatRange(FFloatRangeBound::Inclusive(2.0f), FFloatRangeBound::Inclusive(4.0f));

	TArray<FFloatRange > d15_16 = FFloatRange::Difference(r15_2, r15_12);
	{
		CHECK_MESSAGE(TEXT("[7, 14] - [2, 4) must result in one set"), d15_16.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] - [2, 4) must be {[7, 14]}"), d15_16[0] == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Inclusive(14.0f)));
	}

	// range hulls
	FFloatRange r16_1 = FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Exclusive(9.0f));
	FFloatRange r16_2 = FFloatRange(FFloatRangeBound::Inclusive(11.0f), FFloatRangeBound::Exclusive(14.0f));
	FFloatRange r16_3 = FFloatRange(4.0f, 1.0f);
	FFloatRange r16_4 = FFloatRange(8.0f, 5.0f);

	CHECK_MESSAGE(TEXT("The hull of [7, 9) and [11, 14) must be [7, 14)"), FFloatRange::Hull(r16_1, r16_2) == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Exclusive(14.0f)));
	CHECK_MESSAGE(TEXT("The hull of [7, 9) and an empty range must be [7, 9)"), FFloatRange::Hull(r16_1, r16_3) == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Exclusive(9.0f)));
	CHECK_MESSAGE(TEXT("The hull of two empty ranges must be empty"), FFloatRange::Hull(r16_3, r16_4).IsEmpty());

	// intersections
	FFloatRange r17_1 = FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Exclusive(14.0f));
	FFloatRange r17_2 = FFloatRange(FFloatRangeBound::Inclusive(2.0f), FFloatRangeBound::Exclusive(8.0f));		// left overlap
	FFloatRange r17_3 = FFloatRange(FFloatRangeBound::Inclusive(2.0f), FFloatRangeBound::Inclusive(8.0f));
	FFloatRange r17_4 = FFloatRange(FFloatRangeBound::Exclusive(8.0f), FFloatRangeBound::Exclusive(16.0f));		// right overlap
	FFloatRange r17_5 = FFloatRange(FFloatRangeBound::Inclusive(8.0f), FFloatRangeBound::Exclusive(16.0f));
	FFloatRange r17_6 = FFloatRange(4.0f, 1.0f);
	FFloatRange r17_7 = FFloatRange(8.0f, 5.0f);

	CHECK_MESSAGE(TEXT("The intersection of [7, 14) and [2, 8) must be [7, 8)"), FFloatRange::Intersection(r17_1, r17_2) == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Exclusive(8.0f)));
	CHECK_MESSAGE(TEXT("The intersection of [7, 14) and [2, 8] must be [7, 8]"), FFloatRange::Intersection(r17_1, r17_3) == FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Inclusive(8.0f)));
	CHECK_MESSAGE(TEXT("The intersection of [7, 14) and (8, 16) must be (8, 14)"), FFloatRange::Intersection(r17_1, r17_4) == FFloatRange(FFloatRangeBound::Exclusive(8.0f), FFloatRangeBound::Exclusive(14.0f)));
	CHECK_MESSAGE(TEXT("The intersection of [7, 14) and [8, 16) must be [8, 14)"), FFloatRange::Intersection(r17_1, r17_5) == FFloatRange(FFloatRangeBound::Inclusive(8.0f), FFloatRangeBound::Exclusive(14.0f)));
	CHECK_MESSAGE(TEXT("The intersection of a non-empty range and an empty range must be empty"), FFloatRange::Intersection(r17_1, r17_6).IsEmpty());
	CHECK_MESSAGE(TEXT("The intersection of two empty ranges must be empty"), FFloatRange::Intersection(r17_6, r17_7).IsEmpty());

	// union
	FFloatRange r18_1 = FFloatRange(FFloatRangeBound::Inclusive(7.0f), FFloatRangeBound::Inclusive(14.0f));
	FFloatRange r18_2 = FFloatRange(FFloatRangeBound::Exclusive(7.0f), FFloatRangeBound::Exclusive(14.0f));
	FFloatRange r18_3 = FFloatRange(FFloatRangeBound::Inclusive(2.0f), FFloatRangeBound::Inclusive(8.0f));		// left overlap
	FFloatRange r18_4 = FFloatRange(FFloatRangeBound::Exclusive(15.0f), FFloatRangeBound::Inclusive(16.0f));
	FFloatRange r18_5 = FFloatRange(FFloatRangeBound::Inclusive(8.0f), FFloatRangeBound::Exclusive(16.0f));		// right overlap
	FFloatRange r18_6 = FFloatRange(FFloatRangeBound::Inclusive(6.0f), FFloatRangeBound::Exclusive(7.0f));

	TArray<FFloatRange > u18_0 = FFloatRange::Union(r18_1, r18_1);
	{
		CHECK_MESSAGE(TEXT("[7, 14] unioned with itself must result in one range"), u18_0.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] unioned with itself must result in itself"), u18_0[0] == r18_1);
	}

	TArray<FFloatRange > u18_1 = FFloatRange::Union(r18_1, r18_2);
	{
		CHECK_MESSAGE(TEXT("[7, 14] unioned with (7, 14) must result in one range"), u18_1.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] unioned with (7, 14) must result in {[7, 14]}"), u18_1[0] == r18_1);
	}

	TArray<FFloatRange > u18_2 = FFloatRange::Union(r18_1, r18_3);
	{
		CHECK_MESSAGE(TEXT("[7, 14] unioned with [2, 8] must result in one range"), u18_2.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] unioned with [2, 8] must result in {[2, 14]}"), u18_2[0] == FFloatRange(FFloatRangeBound::Inclusive(2.f), FFloatRangeBound::Inclusive(14.f)));
	}

	TArray<FFloatRange > u18_3 = FFloatRange::Union(r18_1, r18_4);
	{
		CHECK_MESSAGE(TEXT("[7, 14] unioned with (15, 16] must result in two ranges"), u18_3.Num() == 2);

		bool b18_1 = (u18_3[0] == r18_1);
		bool b18_2 = (u18_3[1] == r18_4);

		CHECK_MESSAGE(TEXT("[7, 14] unioned with (15, 16] must result in {[7, 14], (15, 16]}"), b18_1 && b18_2);
	}

	TArray<FFloatRange > u18_4 = FFloatRange::Union(r18_1, FFloatRange::Empty());
	{
		CHECK_MESSAGE(TEXT("A non-empty range unioned with an empty range must result in one range"), u18_4.Num() == 1);
		CHECK_MESSAGE(TEXT("A non-empty range unioned with an empty range must result in the non-empty range"), u18_4[0] == r18_1);
	}

	TArray<FFloatRange > u18_5 = FFloatRange::Union(r18_1, r18_5);
	{
		CHECK_MESSAGE(TEXT("[7, 14] unioned with [8, 16) must result in one range"), u18_5.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] unioned with [8, 16) must result in {[7, 16)}"), u18_5[0] == FFloatRange(FFloatRangeBound::Inclusive(7.f), FFloatRangeBound::Exclusive(16.f)));
	}

	TArray<FFloatRange > u18_6 = FFloatRange::Union(r18_1, r18_6);
	{
		CHECK_MESSAGE(TEXT("[7, 14] unioned with [6, 7) must result in one range"), u18_6.Num() == 1);
		CHECK_MESSAGE(TEXT("[7, 14] unioned with [6, 7) must result in {[6, 14]}"), u18_6[0] == FFloatRange(FFloatRangeBound::Inclusive(6.f), FFloatRangeBound::Inclusive(14.f)));
	}

	TArray<FFloatRange > u18_7 = FFloatRange::Union(r18_2, r18_6);
	{
		CHECK_MESSAGE(TEXT("(7, 14) unioned with [6, 7) must result in two ranges"), u18_7.Num() == 2);

		bool b18_1 = (u18_7[0] == r18_2);
		bool b18_2 = (u18_7[1] == r18_6);

		CHECK_MESSAGE(TEXT("(7, 14) unioned with [6, 7) must result in {(7, 14), [6, 7)}"), b18_1 && b18_2);
	}

	CHECK_MESSAGE(TEXT("An pair of empty ranges unioned must yield no ranges."), FFloatRange::Union(FFloatRange::Empty(), FFloatRange::Empty()).Num() == 0);
}

#endif //WITH_TESTS
