// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Math/UnitConversion.h"
#include "Tests/TestHarnessAdapter.h"

PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

bool IsRoughlyEqual(double One, double Two, float Epsilon)
{
	return FMath::Abs(One-Two) <= Epsilon;
}

TEST_CASE_NAMED(FUnitUnitTests, "System::Core::Math::Unit Conversion", "[ApplicationContextMask][SmokeFilter]")
{
	struct FTestStruct
	{
		double Source;
		double ExpectedResult;
		double AccuracyEpsilon;

		EUnit FromUnit, ToUnit;
	};
	static FTestStruct Tests[] = {
		{ 0.025,	2.3651e11,	1e7, 	EUnit::Lightyears, 									EUnit::Kilometers	 								},
		{ 0.5,		80467.2,	0.1,	EUnit::Miles, 										EUnit::Centimeters	 								},
		{ 0.2,		182.88,		0.01,	EUnit::Yards, 										EUnit::Millimeters	 								},
		{ 0.2,		60960,		0.1,	EUnit::Feet, 										EUnit::Micrometers	 								},
		{ 10,		0.254,		0.001,	EUnit::Inches, 										EUnit::Meters		 								},
		{ 0.75,		2460.6299,	1e-4,	EUnit::Kilometers, 									EUnit::Feet											},
		{ 1,		39.37,		0.01,	EUnit::Meters, 										EUnit::Inches										},
		{ 2750,		27.5,		1e-6,	EUnit::Centimeters, 								EUnit::Meters										},
		{ 1000,		1.0936,		1e-4,	EUnit::Millimeters, 								EUnit::Yards		 								},
		{ 2000,		0.0787,		1e-4,	EUnit::Micrometers, 								EUnit::Inches		 								},

		{ 90,		UE_PI/2,	1e-3,	EUnit::Degrees, 									EUnit::Radians		 								},
		{ UE_PI,	180,		1e-3,	EUnit::Radians, 									EUnit::Degrees		 								},

		{ 12,		43.2,		0.1,	EUnit::MetersPerSecond,								EUnit::KilometersPerHour							},
		{ 1,		0.6214,		1e-4,	EUnit::KilometersPerHour,							EUnit::MilesPerHour									},
		{ 15,		6.7056,		1e-4,	EUnit::MilesPerHour,								EUnit::MetersPerSecond								},

		{ 90.0,			UE_DOUBLE_HALF_PI,	1e-3,	EUnit::DegreesPerSecond, 				EUnit::RadiansPerSecond								},
		{ UE_DOUBLE_PI,	180.0,				1e-3,	EUnit::RadiansPerSecond, 				EUnit::DegreesPerSecond								},

		{ 100.0,		1.0,		1e-3,	EUnit::CentimetersPerSecondSquared, 	EUnit::MetersPerSecondSquared								},
		{ 1.0,		100.0,	1e-3,	EUnit::MetersPerSecondSquared, 		EUnit::CentimetersPerSecondSquared							},

		{ 100, 		212,		0.1,	EUnit::Celsius,										EUnit::Farenheit									},
		{ 400, 		477.594,	1e-3,	EUnit::Farenheit,									EUnit::Kelvin										},
		{ 72, 		-201.15,	0.01,	EUnit::Kelvin,										EUnit::Celsius										},

		{ 1000, 	3.5274e-5,	1e-6,	EUnit::Micrograms,									EUnit::Ounces,										},
		{ 1000,		1,			0.1,	EUnit::Milligrams,									EUnit::Grams,										},
		{ 200,		0.4409,		1e-4,	EUnit::Grams,										EUnit::Pounds,										},
		{ 0.15,		150000,		0.1,	EUnit::Kilograms,									EUnit::Milligrams,									},
		{ 1,		157.473,	1e-3,	EUnit::MetricTons,									EUnit::Stones,										},
		{ 0.001,	28349.5,	0.1,	EUnit::Ounces,										EUnit::Micrograms,									},
		{ 500,		226.796,	1e-3,	EUnit::Pounds,										EUnit::Kilograms,									},
		{ 100,		0.6350,		1e-4,	EUnit::Stones,										EUnit::MetricTons,									},

		{ 100,		10.1972,	1e-4,	EUnit::Newtons,										EUnit::KilogramsForce,								},
		{ 2,		4.4092,		1e-4,	EUnit::KilogramsForce,								EUnit::PoundsForce,									},
		{ 15,		6672.33,	1e-2,	EUnit::PoundsForce,									EUnit::KilogramCentimetersPerSecondSquared,			},
		{ 500,		5,			0.1,	EUnit::KilogramCentimetersPerSecondSquared,			EUnit::Newtons,										},

		{ 10,		100000,		0.1,	EUnit::NewtonMeters, 								EUnit::KilogramCentimetersSquaredPerSecondSquared	},
		{ 2000,		0.2,		0.1,	EUnit::KilogramCentimetersSquaredPerSecondSquared, 	EUnit::NewtonMeters		 							},

		{ 1000,		1,			0.1,	EUnit::Hertz,										EUnit::Kilohertz,									},
		{ 0.25,		250*60,		1e-3,	EUnit::Kilohertz,									EUnit::RevolutionsPerMinute,						},
		{ 1000,		1,			1e-3,	EUnit::Megahertz,									EUnit::Gigahertz,									},
		{ 0.001,	1000000,	1e-3,	EUnit::Gigahertz,									EUnit::Hertz,										},
		{ 100,		100.0/60,	1e-3,	EUnit::RevolutionsPerMinute,						EUnit::Hertz,										},
		
		{ 1024,		1,			1e-3,	EUnit::Bytes,										EUnit::Kilobytes,									},
		{ 1.5,		1536,		1e-3,	EUnit::Kilobytes,									EUnit::Bytes,										},
		{ 1000,		9.5367e-4,	1e-5,	EUnit::Megabytes,									EUnit::Terabytes,									},
		{ 0.5,		512,		1e-3,	EUnit::Gigabytes,									EUnit::Megabytes,									},
		{ 0.25,		256,		1e-3,	EUnit::Terabytes,									EUnit::Gigabytes,									},
		
		{ 10000,	0.166667,	1e-6,	EUnit::Milliseconds,								EUnit::Minutes,										},
		{ 0.5,		500,		1e-6,	EUnit::Seconds,										EUnit::Milliseconds,								},
		{ 30,		60*30,		1e-6,	EUnit::Minutes,										EUnit::Seconds,										},
		{ 5,		5.0/24,		1e-6,	EUnit::Hours,										EUnit::Days,										},
		{ 0.75,		18,			1e-6,	EUnit::Days,										EUnit::Hours,										},
		{ 3,		0.25,		1e-6,	EUnit::Months,										EUnit::Years,										},
		{ 0.5,		6,			1e-6,	EUnit::Years,										EUnit::Months,										},

		{ 22.5,		0.225,		1e-6,	EUnit::Percentage,									EUnit::Multiplier,									},
		{ 22.5,		2250,		0.f,	EUnit::Multiplier,									EUnit::Percentage,									},

	};

	for (auto& Test : Tests)
	{
		const double ActualResult = FUnitConversion::Convert(Test.Source, Test.FromUnit, Test.ToUnit);
		if (!IsRoughlyEqual(ActualResult, Test.ExpectedResult, Test.AccuracyEpsilon))
		{

			const TCHAR* FromUnitString	= FUnitConversion::GetUnitDisplayString(Test.FromUnit);
			const TCHAR* ToUnitString 	= FUnitConversion::GetUnitDisplayString(Test.ToUnit);

			FAIL_CHECK(FString::Printf(TEXT("Conversion from %s to %s was incorrect. Converting %.10f%s to %s resulted in %.15f%s, expected %.15f%s (threshold = %.15f)"),
				FromUnitString, ToUnitString,
				Test.Source, FromUnitString, ToUnitString,
				ActualResult, ToUnitString,
				Test.ExpectedResult, ToUnitString,
				Test.AccuracyEpsilon));
		}
	}
}

TEST_CASE_NAMED(FParsingUnitTests, "System::Core::Math::Unit Parsing", "[ApplicationContextMask][SmokeFilter]")
{
	struct FTestCases
	{
		const TCHAR*					Expression;
		double							ExpectedValue;
		EUnit							UnderlyingUnit;
		TOptional<FNumericUnit<double>>	ExistingValue;
	};

	FTestCases Tests[] = {
		{TEXT("10.7cm"),						10.7,		EUnit::Centimeters,			TOptional<FNumericUnit<double>>()},
		{TEXT("0.7 m"),							70.0,		EUnit::Centimeters,			TOptional<FNumericUnit<double>>()},
		{TEXT("2m - 1m"),						100.0,		EUnit::Centimeters,			TOptional<FNumericUnit<double>>()},
		{TEXT("10.4865 MetersPerSecond"),		10.4865,	EUnit::MetersPerSecond,		TOptional<FNumericUnit<double>>()},
		{TEXT("10.8 cd"),						10.8,		EUnit::Candela,				TOptional<FNumericUnit<double>>()},
		{TEXT("4.8 cd + 1.2cd"),				6.0,		EUnit::Candela,				TOptional<FNumericUnit<double>>()},
		{TEXT("1cd/m2 + 0.5 CandelaPerMeter2"),	1.5,		EUnit::CandelaPerMeter2,	TOptional<FNumericUnit<double>>()},
		{TEXT("+=0.7 m"),						140.0,		EUnit::Centimeters,			TOptional<FNumericUnit<double>>(70.0)},
	};

	for (FTestCases& Test : Tests)
	{
		const FNumericUnit<double> ExistingValue = Test.ExistingValue.IsSet() ? Test.ExistingValue.GetValue() : FNumericUnit<double>(0.0, EUnit::Unspecified);
		TValueOrError<FNumericUnit<double>, FText> Result = UnitConversion::TryParseExpression(Test.Expression, Test.UnderlyingUnit, ExistingValue);
		if (Result.IsValid())
		{
			const bool IsEqual = IsRoughlyEqual(Result.GetValue().ConvertTo(Test.UnderlyingUnit).GetValue().Value, Test.ExpectedValue, 1e-6f);
			CHECK_MESSAGE(FString::Printf(TEXT("Parsing of expression \"%s\" failed. Expected %f but got %f."),
					Test.Expression,
					Test.ExpectedValue,
					Result.GetValue().Value
				), IsEqual);
		}
		else
		{
			FAIL_CHECK(FString::Printf(TEXT("Parsing of expression \"%s\" was incorrect (%s). Expected %f."),
				Test.Expression,
				*(Result.GetError().ToString()),
				Test.ExpectedValue));
		}
	}
}

PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS

#endif //WITH_TESTS
