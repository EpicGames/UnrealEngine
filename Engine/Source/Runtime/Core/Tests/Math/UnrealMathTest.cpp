// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/UnrealTemplate.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Containers/Map.h"
#include "Math/Vector.h"
#include "Math/VectorRegister.h"
#include "Math/Plane.h"
#include "Math/Rotator.h"
#include "Math/Matrix.h"
#include "Math/RotationMatrix.h"
#include "Math/Quat.h"
#include "Math/QuatRotationTranslationMatrix.h"
#include "Math/Color.h"
#include "Math/Box.h"
#include "Math/Bounds.h"
#include "Async/ParallelFor.h"
#include "Misc/ScopeLock.h"
#include <limits>
#include <cmath>

#include "Tests/TestHarnessAdapter.h"

DEFINE_LOG_CATEGORY_STATIC(LogUnrealMathTest, Log, All);

// Create some temporary storage variables
alignas(alignof(VectorRegister4Float)) static float GScratch[16];
alignas(alignof(VectorRegister4Double)) static double GScratchDouble[16];
static float GSum;
static double GSumDouble;
static bool GPassing = true;

#define MATHTEST_INLINE FORCENOINLINE // if you want to do performance testing change to FORCEINLINE or FORCEINLINE_DEBUGGABLE

FORCENOINLINE void CheckPassing(bool Passing)
{
	// Convenient for setting breakpoints to catch any failures.
	if (!Passing)
	{
		GPassing = false;
	}
}

FORCENOINLINE void ResetPassing()
{
	GPassing = true;
}

/**
 * Tests if two vectors (xyzw) are bitwise equal
 *
 * @param Vec0 First vector
 * @param Vec1 Second vector
 *
 * @return true if equal
 */
FORCENOINLINE bool TestVectorsEqualBitwise(const VectorRegister4Float& Vec0, const VectorRegister4Float& Vec1)
{
	VectorStoreAligned(Vec0, GScratch + 0);
	VectorStoreAligned(Vec1, GScratch + 4);
	GSum = 0.0f;

	const bool Passed = (memcmp(GScratch + 0, GScratch + 4, sizeof(float) * 4) == 0);

	CheckPassing(Passed);
	return Passed;
}

FORCENOINLINE bool TestVectorsEqualBitwise(const VectorRegister4Double& Vec0, const VectorRegister4Double& Vec1)
{
	VectorStoreAligned(Vec0, GScratchDouble + 0);
	VectorStoreAligned(Vec1, GScratchDouble + 4);
	GSumDouble = 0.0;

	const bool Passed = (memcmp(GScratchDouble + 0, GScratchDouble + 4, sizeof(double) * 4) == 0);

	CheckPassing(Passed);
	return Passed;
}


/**
 * Tests if two vectors (xyzw) are equal within an optional tolerance
 *
 * @param Vec0 First vector
 * @param Vec1 Second vector
 * @param Tolerance Error allowed for the comparison
 *
 * @return true if equal(ish)
 */
FORCENOINLINE bool TestVectorsEqual(const VectorRegister4Float& Vec0, const VectorRegister4Float& Vec1, float Tolerance = 0.0f)
{
	VectorStoreAligned(Vec0, GScratch + 0);
	VectorStoreAligned(Vec1, GScratch + 4);
	GSum = 0.f;
	for (int32 Component = 0; Component < 4; Component++)
	{
		float Diff = GScratch[Component + 0] - GScratch[Component + 4];
		GSum += FMath::Abs(Diff);
	}
	CheckPassing(GSum <= Tolerance);
	return GSum <= Tolerance;
}

FORCENOINLINE bool TestVectorsEqual(const VectorRegister4Double& Vec0, const VectorRegister4Double& Vec1, double Tolerance = 0.0)
{
	VectorStoreAligned(Vec0, GScratchDouble + 0);
	VectorStoreAligned(Vec1, GScratchDouble + 4);
	GSumDouble = 0.0;
	for (int32 Component = 0; Component < 4; Component++)
	{
		double Diff = GScratchDouble[Component + 0] - GScratchDouble[Component + 4];
		GSumDouble += FMath::Abs(Diff);
	}
	CheckPassing(GSumDouble <= Tolerance);
	return GSumDouble <= Tolerance;
}

/**
 * Enforce tolerance per-component, not summed error
 */
FORCENOINLINE bool TestVectorsEqual_ComponentWiseError(const VectorRegister4Float& Vec0, const VectorRegister4Float& Vec1, float Tolerance = 0.0f)
{
	VectorStoreAligned(Vec0, GScratch + 0);
	VectorStoreAligned(Vec1, GScratch + 4);
	GSum = 0.0f;
	bool bPassing = true;

	for (int32 Component = 0; Component < 4; Component++)
	{
		float Diff = GScratch[Component + 0] - GScratch[Component + 4];
		bPassing &= FMath::IsNearlyZero(Diff, Tolerance);
		GSum += FMath::Abs(Diff);
	}
	CheckPassing(bPassing);
	return bPassing;
}

FORCENOINLINE bool TestVectorsEqual_ComponentWiseError(const VectorRegister4Double& Vec0, const VectorRegister4Double& Vec1, double Tolerance = 0.0f)
{
	VectorStoreAligned(Vec0, GScratchDouble + 0);
	VectorStoreAligned(Vec1, GScratchDouble + 4);
	GSumDouble = 0.0f;
	bool bPassing = true;

	for (int32 Component = 0; Component < 4; Component++)
	{
		double Diff = GScratchDouble[Component + 0] - GScratchDouble[Component + 4];
		bPassing &= FMath::IsNearlyZero(Diff, Tolerance);
		GSumDouble += FMath::Abs(Diff);
	}
	CheckPassing(bPassing);
	return bPassing;
}


/**
 * Tests if two vectors (xyz) are equal within an optional tolerance
 *
 * @param Vec0 First vector
 * @param Vec1 Second vector
 * @param Tolerance Error allowed for the comparison
 *
 * @return true if equal(ish)
 */
FORCENOINLINE bool TestVectorsEqual3(const VectorRegister4Float& Vec0, const VectorRegister4Float& Vec1, float Tolerance = 0.0f)
{
	VectorStoreAligned(Vec0, GScratch + 0);
	VectorStoreAligned(Vec1, GScratch + 4);
	GSum = 0.f;
	for (int32 Component = 0; Component < 3; Component++)
	{
		GSum += FMath::Abs<float>(GScratch[Component + 0] - GScratch[Component + 4]);
	}
	CheckPassing(GSum <= Tolerance);
	return GSum <= Tolerance;
}

FORCENOINLINE bool TestVectorsEqual3(const VectorRegister4Double& Vec0, const VectorRegister4Double& Vec1, double Tolerance = 0.0)
{
	VectorStoreAligned(Vec0, GScratchDouble + 0);
	VectorStoreAligned(Vec1, GScratchDouble + 4);
	GSumDouble = 0.f;
	for (int32 Component = 0; Component < 3; Component++)
	{
		GSumDouble += FMath::Abs<double>(GScratchDouble[Component + 0] - GScratchDouble[Component + 4]);
	}
	CheckPassing(GSumDouble <= Tolerance);
	return GSumDouble <= Tolerance;
}

/**
 * Tests if two vectors of floats (xyz) are equal within an optional tolerance
 *
 * @param Vec0 First vector
 * @param Vec1 Second vector
 * @param Tolerance Error allowed for the comparison
 *
 * @return true if equal(ish)
 */
FORCENOINLINE bool TestFVector3Equal(const FVector3f& Vec0, const FVector3f& Vec1, float Tolerance = 0.0f)
{
	GScratch[0] = Vec0.X;
	GScratch[1] = Vec0.Y;
	GScratch[2] = Vec0.Z;
	GScratch[3] = 0.0f;
	GScratch[4] = Vec1.X;
	GScratch[5] = Vec1.Y;
	GScratch[6] = Vec1.Z;
	GScratch[7] = 0.0f;
	GSum = 0.f;

	for (int32 Component = 0; Component < 3; Component++)
	{
		GSum += FMath::Abs<float>(GScratch[Component + 0] - GScratch[Component + 4]);
	}
	CheckPassing(GSum <= Tolerance);
	return GSum <= Tolerance;
}

/**
 * Tests if two vectors of doubles (xyz) are equal within an optional tolerance
 *
 * @param Vec0 First vector
 * @param Vec1 Second vector
 * @param Tolerance Error allowed for the comparison
 *
 * @return true if equal(ish)
 */
FORCENOINLINE bool TestFVector3Equal(const FVector3d& Vec0, const FVector3d& Vec1, double Tolerance = 0.0l)
{
	GScratchDouble[0] = Vec0.X;
	GScratchDouble[1] = Vec0.Y;
	GScratchDouble[2] = Vec0.Z;
	GScratchDouble[3] = 0.0f;
	GScratchDouble[4] = Vec1.X;
	GScratchDouble[5] = Vec1.Y;
	GScratchDouble[6] = Vec1.Z;
	GScratchDouble[7] = 0.0f;
	GSumDouble = 0.f;

	for (int32 Component = 0; Component < 3; Component++)
	{
		GSumDouble += FMath::Abs<double>(GScratchDouble[Component + 0] - GScratchDouble[Component + 4]);
	}
	CheckPassing(GSumDouble <= Tolerance);
	return GSumDouble <= Tolerance;
}

FORCENOINLINE bool TestQuatsEqual(const FQuat4f& Q0, const FQuat4f& Q1, float Tolerance)
{
	GScratch[0] = Q0.X;
	GScratch[1] = Q0.Y;
	GScratch[2] = Q0.Z;
	GScratch[3] = Q0.W;
	GScratch[4] = Q1.X;
	GScratch[5] = Q1.Y;
	GScratch[6] = Q1.Z;
	GScratch[7] = Q1.W;
	GSum = 0.f;

	const bool bEqual = Q0.Equals(Q1, Tolerance);
	CheckPassing(bEqual);
	return bEqual;
}

FORCENOINLINE bool TestQuatsEqual(const FQuat4d& Q0, const FQuat4d& Q1, double Tolerance)
{
	GScratchDouble[0] = Q0.X;
	GScratchDouble[1] = Q0.Y;
	GScratchDouble[2] = Q0.Z;
	GScratchDouble[3] = Q0.W;
	GScratchDouble[4] = Q1.X;
	GScratchDouble[5] = Q1.Y;
	GScratchDouble[6] = Q1.Z;
	GScratchDouble[7] = Q1.W;
	GSumDouble = 0.f;

	const bool bEqual = Q0.Equals(Q1, Tolerance);
	CheckPassing(bEqual);
	return bEqual;
}

/**
 * Tests if a vector (xyz) is normalized (length 1) within a tolerance
 *
 * @param Vec0 Vector
 * @param Tolerance Error allowed for the comparison
 *
 * @return true if normalized(ish)
 */
FORCENOINLINE bool TestFVector3Normalized(const FVector3f& Vec0, float Tolerance)
{
	GScratch[0] = Vec0.X;
	GScratch[1] = Vec0.Y;
	GScratch[2] = Vec0.Z;
	GScratch[3] = 0.0f;
	GScratch[4] = 0.0f;
	GScratch[5] = 0.0f;
	GScratch[6] = 0.0f;
	GScratch[7] = 0.0f;
	GSum = FMath::Sqrt(Vec0.X * Vec0.X + Vec0.Y * Vec0.Y + Vec0.Z * Vec0.Z);

	const bool bNormalized = FMath::IsNearlyEqual(GSum, 1.0f, Tolerance);
	CheckPassing(bNormalized);
	return bNormalized;
}

/**
 * Tests if a quaternion (xyzw) is normalized (length 1) within a tolerance
 *
 * @param Q0 Quaternion
 * @param Tolerance Error allowed for the comparison
 *
 * @return true if normalized(ish)
 */
FORCENOINLINE bool TestQuatNormalized(const FQuat4f& Q0, float Tolerance)
{
	GScratch[0] = Q0.X;
	GScratch[1] = Q0.Y;
	GScratch[2] = Q0.Z;
	GScratch[3] = Q0.W;
	GScratch[4] = 0.0f;
	GScratch[5] = 0.0f;
	GScratch[6] = 0.0f;
	GScratch[7] = 0.0f;
	GSum = FMath::Sqrt(Q0.X * Q0.X + Q0.Y * Q0.Y + Q0.Z * Q0.Z + Q0.W * Q0.W);

	const bool bNormalized = FMath::IsNearlyEqual(GSum, 1.0f, Tolerance);
	CheckPassing(bNormalized);
	return bNormalized;
}

/**
 * Tests if two matrices (4x4 xyzw) are equal within an optional tolerance
 *
 * @param Mat0 First Matrix
 * @param Mat1 Second Matrix
 * @param Tolerance Error per column allowed for the comparison
 *
 * @return true if equal(ish)
 */
FORCENOINLINE bool TestMatricesEqual(const FMatrix44f& Mat0, const FMatrix44f& Mat1, float Tolerance = 0.0f)
{
	for (int32 Row = 0; Row < 4; ++Row)
	{
		GSum = 0.f;
		for (int32 Column = 0; Column < 4; ++Column)
		{
			float Diff = Mat0.M[Row][Column] - Mat1.M[Row][Column];
			GSum += FMath::Abs(Diff);
		}
		if (GSum > Tolerance)
		{
			CheckPassing(false);
			return false;
		}
	}
	return true;
}

FORCENOINLINE bool TestMatricesEqual(const FMatrix44d& Mat0, const FMatrix44d& Mat1, double Tolerance = 0.0)
{
	for (int32 Row = 0; Row < 4; ++Row)
	{
		GSumDouble = 0.f;
		for (int32 Column = 0; Column < 4; ++Column)
		{
			double Diff = Mat0.M[Row][Column] - Mat1.M[Row][Column];
			GSumDouble += FMath::Abs(Diff);
		}
		if (GSumDouble > Tolerance)
		{
			CheckPassing(false);
			return false;
		}
	}
	return true;
}


/**
 * Multiplies two 4x4 matrices.
 *
 * @param Result	Pointer to where the result should be stored
 * @param Matrix1	Pointer to the first matrix
 * @param Matrix2	Pointer to the second matrix
 */
template<typename FReal>
void TestVectorMatrixMultiply(UE::Math::TMatrix<FReal>* Result, const UE::Math::TMatrix<FReal>* Matrix1, const UE::Math::TMatrix<FReal>* Matrix2)
{
	typedef FReal Float4x4[4][4];
	const Float4x4& A = *((const Float4x4*)Matrix1);
	const Float4x4& B = *((const Float4x4*)Matrix2);
	Float4x4 Temp;
	Temp[0][0] = A[0][0] * B[0][0] + A[0][1] * B[1][0] + A[0][2] * B[2][0] + A[0][3] * B[3][0];
	Temp[0][1] = A[0][0] * B[0][1] + A[0][1] * B[1][1] + A[0][2] * B[2][1] + A[0][3] * B[3][1];
	Temp[0][2] = A[0][0] * B[0][2] + A[0][1] * B[1][2] + A[0][2] * B[2][2] + A[0][3] * B[3][2];
	Temp[0][3] = A[0][0] * B[0][3] + A[0][1] * B[1][3] + A[0][2] * B[2][3] + A[0][3] * B[3][3];

	Temp[1][0] = A[1][0] * B[0][0] + A[1][1] * B[1][0] + A[1][2] * B[2][0] + A[1][3] * B[3][0];
	Temp[1][1] = A[1][0] * B[0][1] + A[1][1] * B[1][1] + A[1][2] * B[2][1] + A[1][3] * B[3][1];
	Temp[1][2] = A[1][0] * B[0][2] + A[1][1] * B[1][2] + A[1][2] * B[2][2] + A[1][3] * B[3][2];
	Temp[1][3] = A[1][0] * B[0][3] + A[1][1] * B[1][3] + A[1][2] * B[2][3] + A[1][3] * B[3][3];

	Temp[2][0] = A[2][0] * B[0][0] + A[2][1] * B[1][0] + A[2][2] * B[2][0] + A[2][3] * B[3][0];
	Temp[2][1] = A[2][0] * B[0][1] + A[2][1] * B[1][1] + A[2][2] * B[2][1] + A[2][3] * B[3][1];
	Temp[2][2] = A[2][0] * B[0][2] + A[2][1] * B[1][2] + A[2][2] * B[2][2] + A[2][3] * B[3][2];
	Temp[2][3] = A[2][0] * B[0][3] + A[2][1] * B[1][3] + A[2][2] * B[2][3] + A[2][3] * B[3][3];

	Temp[3][0] = A[3][0] * B[0][0] + A[3][1] * B[1][0] + A[3][2] * B[2][0] + A[3][3] * B[3][0];
	Temp[3][1] = A[3][0] * B[0][1] + A[3][1] * B[1][1] + A[3][2] * B[2][1] + A[3][3] * B[3][1];
	Temp[3][2] = A[3][0] * B[0][2] + A[3][1] * B[1][2] + A[3][2] * B[2][2] + A[3][3] * B[3][2];
	Temp[3][3] = A[3][0] * B[0][3] + A[3][1] * B[1][3] + A[3][2] * B[2][3] + A[3][3] * B[3][3];
	memcpy(Result, &Temp, 16 * sizeof(FReal));
}


/**
 * Calculate the inverse of an FMatrix44f.
 *
 * @param DstMatrix		FMatrix44f pointer to where the result should be stored
 * @param SrcMatrix		FMatrix44f pointer to the Matrix to be inversed
 */
template<typename FReal>
void TestVectorMatrixInverse(UE::Math::TMatrix<FReal>* DstMatrix, const UE::Math::TMatrix<FReal>* SrcMatrix)
{
	typedef FReal Float4x4[4][4];
	const Float4x4& M = *((const Float4x4*)SrcMatrix);
	Float4x4 Result;
	FReal Det[4];
	Float4x4 Tmp;

	Tmp[0][0] = M[2][2] * M[3][3] - M[2][3] * M[3][2];
	Tmp[0][1] = M[1][2] * M[3][3] - M[1][3] * M[3][2];
	Tmp[0][2] = M[1][2] * M[2][3] - M[1][3] * M[2][2];

	Tmp[1][0] = M[2][2] * M[3][3] - M[2][3] * M[3][2];
	Tmp[1][1] = M[0][2] * M[3][3] - M[0][3] * M[3][2];
	Tmp[1][2] = M[0][2] * M[2][3] - M[0][3] * M[2][2];

	Tmp[2][0] = M[1][2] * M[3][3] - M[1][3] * M[3][2];
	Tmp[2][1] = M[0][2] * M[3][3] - M[0][3] * M[3][2];
	Tmp[2][2] = M[0][2] * M[1][3] - M[0][3] * M[1][2];

	Tmp[3][0] = M[1][2] * M[2][3] - M[1][3] * M[2][2];
	Tmp[3][1] = M[0][2] * M[2][3] - M[0][3] * M[2][2];
	Tmp[3][2] = M[0][2] * M[1][3] - M[0][3] * M[1][2];

	Det[0] = M[1][1] * Tmp[0][0] - M[2][1] * Tmp[0][1] + M[3][1] * Tmp[0][2];
	Det[1] = M[0][1] * Tmp[1][0] - M[2][1] * Tmp[1][1] + M[3][1] * Tmp[1][2];
	Det[2] = M[0][1] * Tmp[2][0] - M[1][1] * Tmp[2][1] + M[3][1] * Tmp[2][2];
	Det[3] = M[0][1] * Tmp[3][0] - M[1][1] * Tmp[3][1] + M[2][1] * Tmp[3][2];

	FReal Determinant = M[0][0] * Det[0] - M[1][0] * Det[1] + M[2][0] * Det[2] - M[3][0] * Det[3];
	const FReal	RDet = 1.0f / Determinant;

	Result[0][0] = RDet * Det[0];
	Result[0][1] = -RDet * Det[1];
	Result[0][2] = RDet * Det[2];
	Result[0][3] = -RDet * Det[3];
	Result[1][0] = -RDet * (M[1][0] * Tmp[0][0] - M[2][0] * Tmp[0][1] + M[3][0] * Tmp[0][2]);
	Result[1][1] = RDet * (M[0][0] * Tmp[1][0] - M[2][0] * Tmp[1][1] + M[3][0] * Tmp[1][2]);
	Result[1][2] = -RDet * (M[0][0] * Tmp[2][0] - M[1][0] * Tmp[2][1] + M[3][0] * Tmp[2][2]);
	Result[1][3] = RDet * (M[0][0] * Tmp[3][0] - M[1][0] * Tmp[3][1] + M[2][0] * Tmp[3][2]);
	Result[2][0] = RDet * (
		M[1][0] * (M[2][1] * M[3][3] - M[2][3] * M[3][1]) -
		M[2][0] * (M[1][1] * M[3][3] - M[1][3] * M[3][1]) +
		M[3][0] * (M[1][1] * M[2][3] - M[1][3] * M[2][1])
		);
	Result[2][1] = -RDet * (
		M[0][0] * (M[2][1] * M[3][3] - M[2][3] * M[3][1]) -
		M[2][0] * (M[0][1] * M[3][3] - M[0][3] * M[3][1]) +
		M[3][0] * (M[0][1] * M[2][3] - M[0][3] * M[2][1])
		);
	Result[2][2] = RDet * (
		M[0][0] * (M[1][1] * M[3][3] - M[1][3] * M[3][1]) -
		M[1][0] * (M[0][1] * M[3][3] - M[0][3] * M[3][1]) +
		M[3][0] * (M[0][1] * M[1][3] - M[0][3] * M[1][1])
		);
	Result[2][3] = -RDet * (
		M[0][0] * (M[1][1] * M[2][3] - M[1][3] * M[2][1]) -
		M[1][0] * (M[0][1] * M[2][3] - M[0][3] * M[2][1]) +
		M[2][0] * (M[0][1] * M[1][3] - M[0][3] * M[1][1])
		);
	Result[3][0] = -RDet * (
		M[1][0] * (M[2][1] * M[3][2] - M[2][2] * M[3][1]) -
		M[2][0] * (M[1][1] * M[3][2] - M[1][2] * M[3][1]) +
		M[3][0] * (M[1][1] * M[2][2] - M[1][2] * M[2][1])
		);
	Result[3][1] = RDet * (
		M[0][0] * (M[2][1] * M[3][2] - M[2][2] * M[3][1]) -
		M[2][0] * (M[0][1] * M[3][2] - M[0][2] * M[3][1]) +
		M[3][0] * (M[0][1] * M[2][2] - M[0][2] * M[2][1])
		);
	Result[3][2] = -RDet * (
		M[0][0] * (M[1][1] * M[3][2] - M[1][2] * M[3][1]) -
		M[1][0] * (M[0][1] * M[3][2] - M[0][2] * M[3][1]) +
		M[3][0] * (M[0][1] * M[1][2] - M[0][2] * M[1][1])
		);
	Result[3][3] = RDet * (
		M[0][0] * (M[1][1] * M[2][2] - M[1][2] * M[2][1]) -
		M[1][0] * (M[0][1] * M[2][2] - M[0][2] * M[2][1]) +
		M[2][0] * (M[0][1] * M[1][2] - M[0][2] * M[1][1])
		);

	memcpy(DstMatrix, &Result, 16 * sizeof(FReal));
}


/**
 * Calculate Homogeneous transform.
 *
 * @param VecP			VectorRegister4Float
 * @param MatrixM		FMatrix44f pointer to the Matrix to apply transform
 * @return VectorRegister4Float = VecP*MatrixM
 */
VectorRegister4Float TestVectorTransformVector(const VectorRegister4Float& VecP, const FMatrix44f* MatrixM)
{
	typedef float Float4x4[4][4];
	union U {
		VectorRegister4Float v; float f[4];
		FORCEINLINE U() : v() {}
	} Tmp, Result;
	Tmp.v = VecP;
	const Float4x4& M = *((const Float4x4*)MatrixM);

	Result.f[0] = Tmp.f[0] * M[0][0] + Tmp.f[1] * M[1][0] + Tmp.f[2] * M[2][0] + Tmp.f[3] * M[3][0];
	Result.f[1] = Tmp.f[0] * M[0][1] + Tmp.f[1] * M[1][1] + Tmp.f[2] * M[2][1] + Tmp.f[3] * M[3][1];
	Result.f[2] = Tmp.f[0] * M[0][2] + Tmp.f[1] * M[1][2] + Tmp.f[2] * M[2][2] + Tmp.f[3] * M[3][2];
	Result.f[3] = Tmp.f[0] * M[0][3] + Tmp.f[1] * M[1][3] + Tmp.f[2] * M[2][3] + Tmp.f[3] * M[3][3];

	return Result.v;
}

VectorRegister4Double TestVectorTransformVector(const VectorRegister4Double& VecP, const FMatrix44d* MatrixM)
{
	typedef double Float4x4[4][4];
	union U {
		VectorRegister4Double v; double f[4];
		FORCEINLINE U() : v() {}
	} Tmp, Result;
	Tmp.v = VecP;
	const Float4x4& M = *((const Float4x4*)MatrixM);

	Result.f[0] = Tmp.f[0] * M[0][0] + Tmp.f[1] * M[1][0] + Tmp.f[2] * M[2][0] + Tmp.f[3] * M[3][0];
	Result.f[1] = Tmp.f[0] * M[0][1] + Tmp.f[1] * M[1][1] + Tmp.f[2] * M[2][1] + Tmp.f[3] * M[3][1];
	Result.f[2] = Tmp.f[0] * M[0][2] + Tmp.f[1] * M[1][2] + Tmp.f[2] * M[2][2] + Tmp.f[3] * M[3][2];
	Result.f[3] = Tmp.f[0] * M[0][3] + Tmp.f[1] * M[1][3] + Tmp.f[2] * M[2][3] + Tmp.f[3] * M[3][3];

	return Result.v;
}

/**
* Get Rotation as a quaternion.
* @param Rotator FRotator3f
* @return Rotation as a quaternion.
*/
MATHTEST_INLINE FQuat4f TestRotatorToQuaternion(const FRotator3f& Rotator)
{
	const float Pitch = FMath::Fmod(Rotator.Pitch, 360.f);
	const float Yaw = FMath::Fmod(Rotator.Yaw, 360.f);
	const float Roll = FMath::Fmod(Rotator.Roll, 360.f);

	const float CR = FMath::Cos(FMath::DegreesToRadians(Roll * 0.5f));
	const float CP = FMath::Cos(FMath::DegreesToRadians(Pitch * 0.5f));
	const float CY = FMath::Cos(FMath::DegreesToRadians(Yaw * 0.5f));
	const float SR = FMath::Sin(FMath::DegreesToRadians(Roll * 0.5f));
	const float SP = FMath::Sin(FMath::DegreesToRadians(Pitch * 0.5f));
	const float SY = FMath::Sin(FMath::DegreesToRadians(Yaw * 0.5f));

	FQuat4f RotationQuat;
	RotationQuat.W = CR * CP * CY + SR * SP * SY;
	RotationQuat.X = CR * SP * SY - SR * CP * CY;
	RotationQuat.Y = -CR * SP * CY - SR * CP * SY;
	RotationQuat.Z = CR * CP * SY - SR * SP * CY;
	return RotationQuat;
}

MATHTEST_INLINE FQuat4d TestRotatorToQuaternion(const FRotator3d& Rotator)
{
	const double Pitch = FMath::Fmod(Rotator.Pitch, 360.f);
	const double Yaw = FMath::Fmod(Rotator.Yaw, 360.f);
	const double Roll = FMath::Fmod(Rotator.Roll, 360.f);

	const double CR = FMath::Cos(FMath::DegreesToRadians(Roll * 0.5f));
	const double CP = FMath::Cos(FMath::DegreesToRadians(Pitch * 0.5));
	const double CY = FMath::Cos(FMath::DegreesToRadians(Yaw * 0.5));
	const double SR = FMath::Sin(FMath::DegreesToRadians(Roll * 0.5));
	const double SP = FMath::Sin(FMath::DegreesToRadians(Pitch * 0.5));
	const double SY = FMath::Sin(FMath::DegreesToRadians(Yaw * 0.5));

	FQuat4d RotationQuat;
	RotationQuat.W = CR * CP * CY + SR * SP * SY;
	RotationQuat.X = CR * SP * SY - SR * CP * CY;
	RotationQuat.Y = -CR * SP * CY - SR * CP * SY;
	RotationQuat.Z = CR * CP * SY - SR * SP * CY;
	return RotationQuat;
}

MATHTEST_INLINE FVector3f TestQuaternionRotateVectorScalar(const FQuat4f& Quat, const FVector3f& Vector)
{
	// (q.W*q.W-qv.qv)v + 2(qv.v)qv + 2 q.W (qv x v)
	const FVector3f qv(Quat.X, Quat.Y, Quat.Z);
	FVector3f vOut = (2.f * Quat.W) * (qv ^ Vector);
	vOut += ((Quat.W * Quat.W) - (qv | qv)) * Vector;
	vOut += (2.f * (qv | Vector)) * qv;

	return vOut;
}

// Q * V * Q^-1
MATHTEST_INLINE FVector3f TestQuaternionMultiplyVector(const FQuat4f& Quat, const FVector3f& Vector)
{
	FQuat4f VQ(Vector.X, Vector.Y, Vector.Z, 0.f);
	FQuat4f VT, VR;
	FQuat4f I = Quat.Inverse();
	VectorQuaternionMultiply(&VT, &Quat, &VQ);
	VectorQuaternionMultiply(&VR, &VT, &I);

	return FVector3f(VR.X, VR.Y, VR.Z);
}

MATHTEST_INLINE FVector3f TestQuaternionRotateVectorRegister(const FQuat4f& Quat, const FVector3f& V)
{
	const VectorRegister4Float Rotation = *((const VectorRegister4Float*)(&Quat));
	const VectorRegister4Float InputVectorW0 = VectorLoadFloat3_W0(&V);
	const VectorRegister4Float RotatedVec = VectorQuaternionRotateVector(Rotation, InputVectorW0);

	FVector3f Result;
	VectorStoreFloat3(RotatedVec, &Result);
	return Result;
}


/**
* Multiplies two quaternions: The order matters.
*
* @param Result	Pointer to where the result should be stored
* @param Quat1	Pointer to the first quaternion (must not be the destination)
* @param Quat2	Pointer to the second quaternion (must not be the destination)
*/
template<typename FloatType>
void TestVectorQuaternionMultiply(UE::Math::TQuat<FloatType>* Result, const UE::Math::TQuat<FloatType>* Quat1, const UE::Math::TQuat<FloatType>* Quat2)
{
	typedef FloatType Float4[4];
	const Float4& A = *((const Float4*)Quat1);
	const Float4& B = *((const Float4*)Quat2);
	Float4& R = *((Float4*)Result);

	// store intermediate results in temporaries
	const FloatType TX = A[3] * B[0] + A[0] * B[3] + A[1] * B[2] - A[2] * B[1];
	const FloatType TY = A[3] * B[1] - A[0] * B[2] + A[1] * B[3] + A[2] * B[0];
	const FloatType TZ = A[3] * B[2] + A[0] * B[1] - A[1] * B[0] + A[2] * B[3];
	const FloatType TW = A[3] * B[3] - A[0] * B[0] - A[1] * B[1] - A[2] * B[2];

	// copy intermediate result to R
	R[0] = TX;
	R[1] = TY;
	R[2] = TZ;
	R[3] = TW;
}

/**
 * Converts a Quaternion to a Rotator.
 */
FORCENOINLINE FRotator3f TestQuaternionToRotator(const FQuat4f& Quat)
{
	const float X = Quat.X;
	const float Y = Quat.Y;
	const float Z = Quat.Z;
	const float W = Quat.W;

	const float SingularityTest = Z * X - W * Y;
	const float YawY = 2.f * (W * Z + X * Y);
	const float YawX = (1.f - 2.f * (FMath::Square(Y) + FMath::Square(Z)));
	const float SINGULARITY_THRESHOLD = 0.4999995f;

	static const float RAD_TO_DEG = (180.f) / UE_PI;
	FRotator3f RotatorFromQuat;

	// Note: using stock C functions for some trig functions since this is the "reference" implementation
	// and we don't want fast approximations to be used here.
	if (SingularityTest < -SINGULARITY_THRESHOLD)
	{
		RotatorFromQuat.Pitch = -90.f;
		RotatorFromQuat.Yaw = (-2.0f * atan2f(X, W) * RAD_TO_DEG);
		RotatorFromQuat.Roll = 0.f;
	}
	else if (SingularityTest > SINGULARITY_THRESHOLD)
	{
		RotatorFromQuat.Pitch = 90.f;
		RotatorFromQuat.Yaw = (2.0f * atan2f(X, W) * RAD_TO_DEG);
		RotatorFromQuat.Roll = 0.f;
	}
	else
	{
		RotatorFromQuat.Pitch = FMath::Asin(2.f * (SingularityTest)) * RAD_TO_DEG;
		RotatorFromQuat.Yaw = atan2f(YawY, YawX) * RAD_TO_DEG;
		RotatorFromQuat.Roll = atan2f(-2.f * (W * X + Y * Z), (1.f - 2.f * (FMath::Square(X) + FMath::Square(Y)))) * RAD_TO_DEG;
	}

	RotatorFromQuat.Pitch = FRotator3f::NormalizeAxis(RotatorFromQuat.Pitch);
	RotatorFromQuat.Yaw = FRotator3f::NormalizeAxis(RotatorFromQuat.Yaw);
	RotatorFromQuat.Roll = FRotator3f::NormalizeAxis(RotatorFromQuat.Roll);

	return RotatorFromQuat;
}


FORCENOINLINE FQuat4f FindBetween_Old(const FVector3f& vec1, const FVector3f& vec2)
{
	const FVector3f cross = vec1 ^ vec2;
	const float crossMag = cross.Size();

	// See if vectors are parallel or anti-parallel
	if (crossMag < UE_KINDA_SMALL_NUMBER)
	{
		// If these vectors are parallel - just return identity quaternion (ie no rotation).
		const float Dot = vec1 | vec2;
		if (Dot > -UE_KINDA_SMALL_NUMBER)
		{
			return FQuat4f::Identity; // no rotation
		}
		// Exactly opposite..
		else
		{
			// ..rotation by 180 degrees around a vector orthogonal to vec1 & vec2
			FVector3f Vec = vec1.SizeSquared() > vec2.SizeSquared() ? vec1 : vec2;
			Vec.Normalize();

			FVector3f AxisA, AxisB;
			Vec.FindBestAxisVectors(AxisA, AxisB);

			return FQuat4f(AxisA.X, AxisA.Y, AxisA.Z, 0.f); // (axis*sin(pi/2), cos(pi/2)) = (axis, 0)
		}
	}

	// Not parallel, so use normal code
	float angle = FMath::Asin(crossMag);

	const float dot = vec1 | vec2;
	if (dot < 0.0f)
	{
		angle = UE_PI - angle;
	}

	float sinHalfAng, cosHalfAng;
	FMath::SinCos(&sinHalfAng, &cosHalfAng, 0.5f * angle);
	const FVector3f axis = cross / crossMag;

	return FQuat4f(
		sinHalfAng * axis.X,
		sinHalfAng * axis.Y,
		sinHalfAng * axis.Z,
		cosHalfAng);
}

FQuat4f FindBetween_Helper_5_2(const FVector3f& A, const FVector3f& B, float NormAB)
{
	float W = NormAB + FVector3f::DotProduct(A, B);
	FQuat4f Result;

	if (W >= 1e-6f * NormAB)
	{
		//Axis = FVector::CrossProduct(A, B);
		Result = FQuat4f(
			A.Y * B.Z - A.Z * B.Y,
			A.Z * B.X - A.X * B.Z,
			A.X * B.Y - A.Y * B.X,
			W);
	}
	else
	{
		// A and B point in opposite directions
		W = 0.f;
		Result = FMath::Abs(A.X) > FMath::Abs(A.Y)
			? FQuat4f(-A.Z, 0.f, A.X, W)
			: FQuat4f(0.f, -A.Z, A.Y, W);
	}

	Result.Normalize();
	return Result;
}

FQuat4f FindBetweenNormals_5_2(const FVector3f& A, const FVector3f& B)
{
	const float NormAB = 1.f;
	return FindBetween_Helper_5_2(A, B, NormAB);
}


template<typename T>
UE::Math::TQuat<T> Old_Slerp_NotNormalized(const UE::Math::TQuat<T>& Quat1, const UE::Math::TQuat<T>& Quat2, T Slerp)
{
	// Get cosine of angle between quats.
	const T RawCosom =
		Quat1.X * Quat2.X +
		Quat1.Y * Quat2.Y +
		Quat1.Z * Quat2.Z +
		Quat1.W * Quat2.W;
	// Unaligned quats - compensate, results in taking shorter route.
	const T Cosom = FMath::FloatSelect(RawCosom, RawCosom, -RawCosom);

	T Scale0, Scale1;

	if (Cosom < T(0.9999f))
	{
		const T Omega = FMath::Acos(Cosom);
		const T InvSin = T(1.f) / FMath::Sin(Omega);
		Scale0 = FMath::Sin((T(1.f) - Slerp) * Omega) * InvSin;
		Scale1 = FMath::Sin(Slerp * Omega) * InvSin;
	}
	else
	{
		// Use linear interpolation.
		Scale0 = T(1.0f) - Slerp;
		Scale1 = Slerp;
	}

	// In keeping with our flipped Cosom:
	Scale1 = FMath::FloatSelect(RawCosom, Scale1, -Scale1);

	UE::Math::TQuat<T> Result;

	Result.X = Scale0 * Quat1.X + Scale1 * Quat2.X;
	Result.Y = Scale0 * Quat1.Y + Scale1 * Quat2.Y;
	Result.Z = Scale0 * Quat1.Z + Scale1 * Quat2.Z;
	Result.W = Scale0 * Quat1.W + Scale1 * Quat2.W;

	return Result;
}

template<typename T>
UE::Math::TQuat<T> Old_Slerp(const UE::Math::TQuat<T>& Quat1, const UE::Math::TQuat<T>& Quat2, T Slerp)
{
	return Old_Slerp_NotNormalized(Quat1, Quat2, Slerp).GetNormalized();
}

// ROTATOR TESTS

bool TestRotatorEqual0(const FRotator3f& A, const FRotator3f& B, const float Tolerance)
{
	// This is the version used for a few years (known working version).
	return (FMath::Abs(FRotator3f::NormalizeAxis(A.Pitch - B.Pitch)) <= Tolerance)
		&& (FMath::Abs(FRotator3f::NormalizeAxis(A.Yaw - B.Yaw)) <= Tolerance)
		&& (FMath::Abs(FRotator3f::NormalizeAxis(A.Roll - B.Roll)) <= Tolerance);
}

bool TestRotatorEqual1(const FRotator3f& A, const FRotator3f& B, const float Tolerance)
{
	// Test the vectorized method.
	const VectorRegister4Float RegA = VectorLoadFloat3_W0(&A);
	const VectorRegister4Float RegB = VectorLoadFloat3_W0(&B);
	const VectorRegister4Float NormDelta = VectorNormalizeRotator(VectorSubtract(RegA, RegB));
	const VectorRegister4Float AbsNormDelta = VectorAbs(NormDelta);
	return !VectorAnyGreaterThan(AbsNormDelta, VectorLoadFloat1(&Tolerance));
}

bool TestRotatorEqual2(const FRotator3f& A, const FRotator3f& B, const float Tolerance)
{
	// Test the FRotator3f method itself. It will likely be an equivalent implementation as 0 or 1 above.
	return A.Equals(B, Tolerance);
}

bool TestRotatorEqual3(const FRotator3f& A, const FRotator3f& B, const float Tolerance)
{
	// Logically equivalent to tests above. Also tests IsNearlyZero().
	return (A - B).IsNearlyZero(Tolerance);
}

// Report an error if bComparison is not equal to bExpected.
void LogRotatorTest(bool bExpected, const TCHAR* TestName, const FRotator3f& A, const FRotator3f& B, bool bComparison)
{
	const bool bHasPassed = (bComparison == bExpected);
	if (bHasPassed == false)
	{
		UE_LOGF(LogUnrealMathTest, Log, "%ls: %ls", bHasPassed ? TEXT("PASSED") : TEXT("FAILED"), TestName);
		UE_LOGF(LogUnrealMathTest, Log, "(%ls).Equals(%ls) = %d", *A.ToString(), *B.ToString(), bComparison);
		CheckPassing(false);
	}
}


void LogRotatorTest(const TCHAR* TestName, const FRotator3f& A, const FRotator3f& B, bool bComparison)
{
	if (bComparison == false)
	{
		UE_LOGF(LogUnrealMathTest, Log, "%ls: %ls", bComparison ? TEXT("PASSED") : TEXT("FAILED"), TestName);
		UE_LOGF(LogUnrealMathTest, Log, "(%ls).Equals(%ls) = %d", *A.ToString(), *B.ToString(), bComparison);
		CheckPassing(false);
	}
}

void LogQuaternionTest(const TCHAR* TestName, const FQuat4f& A, const FQuat4f& B, bool bComparison)
{
	if (bComparison == false)
	{
		UE_LOGF(LogUnrealMathTest, Log, "%ls: %ls", bComparison ? TEXT("PASSED") : TEXT("FAILED"), TestName);
		UE_LOGF(LogUnrealMathTest, Log, "(%ls).Equals(%ls) = %d", *A.ToString(), *B.ToString(), bComparison);
		CheckPassing(false);
	}
}

// Normalize tests

MATHTEST_INLINE VectorRegister4Float TestVectorNormalize_Sqrt(const VectorRegister4Float& V)
{
	const VectorRegister4Float Len = VectorDot4(V, V);
	const float rlen = 1.0f / FMath::Sqrt(VectorGetComponent(Len, 0));
	return VectorMultiply(V, VectorLoadFloat1(&rlen));
}

MATHTEST_INLINE VectorRegister4Float TestVectorNormalize_InvSqrt(const VectorRegister4Float& V)
{
	const VectorRegister4Float Len = VectorDot4(V, V);
	const float rlen = FMath::InvSqrt(VectorGetComponent(Len, 0));
	return VectorMultiply(V, VectorLoadFloat1(&rlen));
}

MATHTEST_INLINE VectorRegister4Float TestVectorNormalize_InvSqrtEst(const VectorRegister4Float& V)
{
	const VectorRegister4Float Len = VectorDot4(V, V);
	const float rlen = FMath::InvSqrtEst(VectorGetComponent(Len, 0));
	return VectorMultiply(V, VectorLoadFloat1(&rlen));
}


// A Mod M
MATHTEST_INLINE VectorRegister4Float TestReferenceMod(const VectorRegister4Float& A, const VectorRegister4Float& M)
{
	float AF[4], MF[4];
	VectorStore(A, AF);
	VectorStore(M, MF);

	return MakeVectorRegister(
		(float)fmodf(AF[0], MF[0]),
		(float)fmodf(AF[1], MF[1]),
		(float)fmodf(AF[2], MF[2]),
		(float)fmodf(AF[3], MF[3]));
}

MATHTEST_INLINE VectorRegister4Double TestReferenceMod(const VectorRegister4Double& A, const VectorRegister4Double& M)
{
	double AF[4], MF[4];
	VectorStore(A, AF);
	VectorStore(M, MF);

	return MakeVectorRegister(
		(double)fmod(AF[0], MF[0]),
		(double)fmod(AF[1], MF[1]),
		(double)fmod(AF[2], MF[2]),
		(double)fmod(AF[3], MF[3]));
}

// SinCos
template<typename FloatType, typename VectorRegisterType>
MATHTEST_INLINE void TestReferenceSinCos(VectorRegisterType& S, VectorRegisterType& C, const VectorRegisterType& VAngles)
{
	FloatType FloatAngles[4];
	VectorStore(VAngles, FloatAngles);

	S = MakeVectorRegister(
		FMath::Sin(FloatAngles[0]),
		FMath::Sin(FloatAngles[1]),
		FMath::Sin(FloatAngles[2]),
		FMath::Sin(FloatAngles[3])
	);

	C = MakeVectorRegister(
		FMath::Cos(FloatAngles[0]),
		FMath::Cos(FloatAngles[1]),
		FMath::Cos(FloatAngles[2]),
		FMath::Cos(FloatAngles[3])
	);
}

template<typename FloatType, typename VectorRegisterType>
MATHTEST_INLINE void TestFastSinCos(VectorRegisterType& S, VectorRegisterType& C, const VectorRegisterType& VAngles)
{
	FloatType FloatAngles[4];
	VectorStore(VAngles, FloatAngles);

	FloatType SFloat[4], CFloat[4];
	FMath::SinCos(&SFloat[0], &CFloat[0], FloatAngles[0]);
	FMath::SinCos(&SFloat[1], &CFloat[1], FloatAngles[1]);
	FMath::SinCos(&SFloat[2], &CFloat[2], FloatAngles[2]);
	FMath::SinCos(&SFloat[3], &CFloat[3], FloatAngles[3]);

	S = VectorLoad(SFloat);
	C = VectorLoad(CFloat);
}

template<typename FloatType, typename VectorRegisterType>
MATHTEST_INLINE void TestVectorSinCos(VectorRegisterType& S, VectorRegisterType& C, const VectorRegisterType& VAngles)
{
	VectorSinCos(&S, &C, &VAngles);
}

// Generic way to test functions taking a single FloatType param and returning a FloatType value, and comparing to a vectorized version of the same function.
template<typename FloatType, typename VectorRegisterType, FloatType(*Func)(FloatType), VectorRegisterType(*VectorFunc)(VectorRegisterType)>
FORCENOINLINE bool TestVectorFunction1Param(VectorRegisterType& ReferenceResult, VectorRegisterType& VectorResult, VectorRegisterType VParams, FloatType ErrorTolerance)
{
	// Load params
	FloatType FloatParams[4];
	VectorStore(VParams, FloatParams);

	// Reference function
	FloatType Results[4];
	Results[0] = Func(FloatParams[0]);
	Results[1] = Func(FloatParams[1]);
	Results[2] = Func(FloatParams[2]);
	Results[3] = Func(FloatParams[3]);
	ReferenceResult = VectorLoad(Results);

	// Vector function
	VectorResult = VectorFunc(VParams);

	// Check for errors
	return TestVectorsEqual_ComponentWiseError(ReferenceResult, VectorResult, ErrorTolerance);
}

// Generic way to test functions taking two FloatType params and returning a FloatType value, and comparing to a vectorized version of the same function.
template<typename FloatType, typename VectorRegisterType, FloatType(*Func)(FloatType, FloatType), VectorRegisterType(*VectorFunc)(VectorRegisterType, VectorRegisterType)>
FORCENOINLINE bool TestVectorFunction2Params(VectorRegisterType& ReferenceResult, VectorRegisterType& VectorResult, VectorRegisterType VParams0, VectorRegisterType VParams1, FloatType ErrorTolerance)
{
	// Load params
	FloatType FloatParams0[4];
	VectorStore(VParams0, FloatParams0);
	FloatType FloatParams1[4];
	VectorStore(VParams1, FloatParams1);

	// Reference function
	FloatType Results[4];
	Results[0] = Func(FloatParams0[0], FloatParams1[0]);
	Results[1] = Func(FloatParams0[1], FloatParams1[1]);
	Results[2] = Func(FloatParams0[2], FloatParams1[2]);
	Results[3] = Func(FloatParams0[3], FloatParams1[3]);
	ReferenceResult = VectorLoad(Results);

	// Vector function
	VectorResult = VectorFunc(VParams0, VParams1);

	// Check for errors
	return TestVectorsEqual_ComponentWiseError(ReferenceResult, VectorResult, ErrorTolerance);
}

/**
 * Helper debugf function to print out success or failure information for a test
 *
 * @param TestName Name of the current test
 * @param bHasPassed true if the test has passed
 */

template<typename FloatType>
FORCENOINLINE void LogTest(const TCHAR* TestName, bool bHasPassed)
{
	if (bHasPassed == false)
	{
		UE_LOGF(LogUnrealMathTest, Error, "Unimplemented type for LogTest()");
		CheckPassing(false);
	}
}

// Specialization for 'int32' so it checks the correct global state filled by the various comparision validation functions
template<>
FORCENOINLINE void LogTest<int32>(const TCHAR* TestName, bool bHasPassed)
{
	if (bHasPassed == false)
	{
		int32* GScratchI = static_cast<int32*>(static_cast<void*>(GScratch));
		int32& GSumI = *static_cast<int32*>(static_cast<void*>(&GSum));
		UE_LOGF(LogUnrealMathTest, Warning, "%ls <int32>: %ls", bHasPassed ? TEXT("PASSED") : TEXT("FAILED"), TestName);
		UE_LOGF(LogUnrealMathTest, Warning, "Bad(%d): (%d %d %d %d) (%d %d %d %d)", GSumI, GScratchI[0], GScratchI[1], GScratchI[2], GScratchI[3], GScratchI[4], GScratchI[5], GScratchI[6], GScratchI[7]);
		CheckPassing(false);
	}
}

// Specialization for 'float' so it checks the correct global state filled by the various comparision validation functions
template<>
FORCENOINLINE void LogTest<float>(const TCHAR* TestName, bool bHasPassed)
{
	if (bHasPassed == false)
	{
		UE_LOGF(LogUnrealMathTest, Warning, "%ls <float>: %ls", bHasPassed ? TEXT("PASSED") : TEXT("FAILED"), TestName);
		UE_LOGF(LogUnrealMathTest, Warning, "Bad(%.8f): (%.8f %.8f %.8f %.8f) (%.8f %.8f %.8f %.8f)", GSum, GScratch[0], GScratch[1], GScratch[2], GScratch[3], GScratch[4], GScratch[5], GScratch[6], GScratch[7]);
		CheckPassing(false);
	}
}

// Specialization for 'double' so it checks the correct global state filled by the various comparision validation functions
template<>
FORCENOINLINE void LogTest<double>(const TCHAR* TestName, bool bHasPassed)
{
	if (bHasPassed == false)
	{
		UE_LOGF(LogUnrealMathTest, Warning, "%ls <double>: %ls", bHasPassed ? TEXT("PASSED") : TEXT("FAILED"), TestName);
		UE_LOGF(LogUnrealMathTest, Warning, "Bad(%.8f): (%.8f %.8f %.8f %.8f) (%.8f %.8f %.8f %.8f)", GSumDouble, GScratchDouble[0], GScratchDouble[1], GScratchDouble[2], GScratchDouble[3], GScratchDouble[4], GScratchDouble[5], GScratchDouble[6], GScratchDouble[7]);
		CheckPassing(false);
	}
}

// Report an error if bComparison is not equal to bExpected.
FORCENOINLINE void LogBoundsTest(bool bExpected, const TCHAR* TestName, const FBounds3f& A, const FBounds3f& B, bool bComparison)
{
	const bool bHasPassed = (bComparison == bExpected);
	if (bHasPassed == false)
	{
		UE_LOGF(LogUnrealMathTest, Log, "%ls: %ls", bHasPassed ? TEXT("PASSED") : TEXT("FAILED"), TestName);
		UE_LOGF(LogUnrealMathTest, Log, "A=(Min:%ls, Max:%ls) B=(Min:%ls, Max:%ls)", *A.Min.ToString(), *A.Max.ToString(), *B.Min.ToString(), *B.Max.ToString());
		CheckPassing(false);
	}
}

/**
 * Set the contents of the scratch memory
 *
 * @param X,Y,Z,W,U values to push into GScratch
 */
void SetScratch(float X, float Y, float Z, float W, float U = 0.0f)
{
	GScratch[0] = X;
	GScratch[1] = Y;
	GScratch[2] = Z;
	GScratch[3] = W;
	GScratch[4] = U;
}

void SetScratchDouble(double X, double Y, double Z, double W, double U = 0.0)
{
	GScratchDouble[0] = X;
	GScratchDouble[1] = Y;
	GScratchDouble[2] = Z;
	GScratchDouble[3] = W;
	GScratchDouble[4] = U;
}

void SetScratchDouble(float X, float Y, float Z, float W, float U = 0.0f)
{
	SetScratchDouble((double)X, (double)Y, (double)Z, (double)W, (double)U);
}



template<typename RealType, typename VectorRegisterType>
FORCENOINLINE void TestVectorReplicate()
{
	const RealType ArrayV0[4] = { (RealType)0.0, (RealType)1.0, (RealType)2.0, (RealType)3.0 };
	VectorRegisterType V0 = VectorLoad(ArrayV0);
	VectorRegisterType V1, V2;

#define ReplicateTest(A, X) \
		V1 = VectorReplicate(A, X); \
		V2 = MakeVectorRegister(ArrayV0[X], ArrayV0[X], ArrayV0[X], ArrayV0[X]); \
		LogTest<RealType>(*FString::Printf(TEXT("VectorReplicate<%d>"), X), TestVectorsEqual(V1, V2));

	ReplicateTest(V0, 0);
	ReplicateTest(V0, 1);
	ReplicateTest(V0, 2);
	ReplicateTest(V0, 3);

#undef ReplicateTest
}


template<typename RealType, typename VectorRegisterType>
FORCENOINLINE void TestVectorSwizzle()
{
	const RealType ArrayV0[4] = { (RealType)0.0, (RealType)1.0, (RealType)2.0, (RealType)3.0 };
	VectorRegisterType V0 = VectorLoad(ArrayV0);
	VectorRegisterType V1, V2;

#define SwizzleTest(A, X, Y, Z, W) \
		V1 = VectorSwizzle(A, X, Y, Z, W); \
		V2 = MakeVectorRegister(ArrayV0[X], ArrayV0[Y], ArrayV0[Z], ArrayV0[W]); \
		LogTest<RealType>(*FString::Printf(TEXT("VectorSwizzle<%d,%d,%d,%d>"), X, Y, Z, W), TestVectorsEqual(V1, V2));

	// This is not an exhaustive list because it would be 4*4*4*4 = 256 entries, but it tries to test a lot of common permutations.
	// Unfortunately it can't be done in a loop because it uses a #define and compile-time constants for the VectorSwizzle() 'function'.
	// Many of these were selected to also stress the specializations in certain implementations.

	SwizzleTest(V0, 0, 1, 2, 3); // Identity

	SwizzleTest(V0, 0, 0, 0, 0); // Replicate 0
	SwizzleTest(V0, 1, 1, 1, 1); // Replicate 1
	SwizzleTest(V0, 2, 2, 2, 2); // Replicate 2
	SwizzleTest(V0, 3, 3, 3, 3); // Replicate 3

	SwizzleTest(V0, 1, 2, 3, 0); // Rotate << 1
	SwizzleTest(V0, 2, 3, 0, 1); // Rotate << 2
	SwizzleTest(V0, 3, 0, 1, 2); // Rotate << 3

	// Lane swaps
	SwizzleTest(V0, 0, 1, 2, 3); // lane 0, lane 1
	SwizzleTest(V0, 2, 3, 0, 1); // lane 1, lane 0
	SwizzleTest(V0, 0, 1, 0, 1); // lane 0, lane 0
	SwizzleTest(V0, 2, 3, 2, 3); // lane 1, lane 1

	// Lane swaps with two in-lane permutes
	SwizzleTest(V0, 1, 0, 3, 2); // lane 0, lane 1
	SwizzleTest(V0, 3, 2, 1, 0); // lane 1, lane 0
	SwizzleTest(V0, 1, 0, 1, 0); // lane 0, lane 0
	SwizzleTest(V0, 3, 2, 3, 2); // lane 1, lane 1

	// Lane swaps with one in-lane permute (first lane)
	SwizzleTest(V0, 1, 0, 2, 3); // lane 0, lane 1
	SwizzleTest(V0, 3, 2, 0, 1); // lane 1, lane 0
	SwizzleTest(V0, 1, 0, 0, 1); // lane 0, lane 0
	SwizzleTest(V0, 3, 2, 2, 3); // lane 1, lane 1

	// Lane swaps with one in-lane permute (second lane)
	SwizzleTest(V0, 0, 1, 3, 2); // lane 0, lane 1
	SwizzleTest(V0, 2, 3, 1, 0); // lane 1, lane 0
	SwizzleTest(V0, 0, 1, 1, 0); // lane 0, lane 0
	SwizzleTest(V0, 2, 3, 3, 2); // lane 1, lane 1

	// Lane swaps with a cross-lane permute
	SwizzleTest(V0, 0, 1, 0, 3); // lane 0, lane X
	SwizzleTest(V0, 2, 3, 0, 3); // lane 1, lane X
	SwizzleTest(V0, 0, 1, 2, 1); // lane 0, lane X
	SwizzleTest(V0, 2, 3, 1, 2); // lane 1, lane X

	SwizzleTest(V0, 3, 0, 3, 2); // lane X, lane 1
	SwizzleTest(V0, 2, 0, 1, 0); // lane X, lane 0
	SwizzleTest(V0, 1, 2, 1, 0); // lane X, lane 0
	SwizzleTest(V0, 2, 1, 3, 2); // lane X, lane 1

	// In-lane permute with a cross-lane permute
	SwizzleTest(V0, 1, 0, 3, 0); // lane 0, lane X
	SwizzleTest(V0, 3, 2, 0, 3); // lane 1, lane X
	SwizzleTest(V0, 1, 0, 2, 1); // lane 0, lane X
	SwizzleTest(V0, 3, 2, 1, 2); // lane 1, lane X

	SwizzleTest(V0, 3, 0, 3, 2); // lane X, lane 1
	SwizzleTest(V0, 0, 3, 1, 0); // lane X, lane 0
	SwizzleTest(V0, 1, 2, 1, 0); // lane X, lane 0
	SwizzleTest(V0, 2, 1, 3, 2); // lane X, lane 1

	// Two cross-lane permutes
	SwizzleTest(V0, 3, 0, 0, 3); // lane X, lane X
	SwizzleTest(V0, 1, 2, 2, 1); // lane X, lane X
	SwizzleTest(V0, 3, 0, 2, 1); // lane X, lane X
	SwizzleTest(V0, 1, 2, 0, 3); // lane X, lane X
	SwizzleTest(V0, 0, 2, 0, 2); // lane X, lane X
	SwizzleTest(V0, 2, 0, 0, 2); // lane X, lane X

	// Common specializations or uses
	SwizzleTest(V0, 0, 0, 1, 1);
	SwizzleTest(V0, 0, 0, 2, 2);
	SwizzleTest(V0, 0, 1, 2, 2);
	SwizzleTest(V0, 0, 2, 2, 3);
	SwizzleTest(V0, 1, 1, 3, 3);
	SwizzleTest(V0, 1, 3, 1, 3);
	SwizzleTest(V0, 1, 0, 3, 0);
	SwizzleTest(V0, 2, 2, 3, 3);
	SwizzleTest(V0, 2, 0, 3, 0);

#undef SwizzleTest
}

template<typename RealType, typename VectorRegisterType>
FORCENOINLINE void TestVectorShuffle()
{
	RealType ArrayV0[4] = { (RealType)0.0, (RealType)0.1, (RealType)0.2, (RealType)0.3 };
	RealType ArrayV1[4] = { (RealType)1.0, (RealType)1.1, (RealType)1.2, (RealType)1.3 };
	VectorRegisterType V0 = VectorLoad(ArrayV0);
	VectorRegisterType V1 = VectorLoad(ArrayV1);
	VectorRegisterType V2, V3;

#define ShuffleTest(A, B, X, Y, Z, W) \
		V2 = VectorShuffle(A, B, X, Y, Z, W); \
		V3 = MakeVectorRegister(ArrayV0[X], ArrayV0[Y], ArrayV1[Z], ArrayV1[W]); \
		LogTest<RealType>(*FString::Printf(TEXT("VectorShuffle<%d,%d,%d,%d>"), X, Y, Z, W), TestVectorsEqual(V2, V3));

	// This is not an exhaustive list because it would be 4*4*4*4 = 256 entries, but it tries to test a lot of common permutations.
	// Unfortunately it can't be done in a loop because it uses a #define and compile-time constants for the VectorShuffle() 'function'.
	// Many of these were selected to also stress the specializations in certain implementations.

	ShuffleTest(V0, V1, 0, 1, 2, 3); // Identity

	ShuffleTest(V0, V1, 0, 0, 0, 0); // Replicate 0
	ShuffleTest(V0, V1, 1, 1, 1, 1); // Replicate 1
	ShuffleTest(V0, V1, 2, 2, 2, 2); // Replicate 2
	ShuffleTest(V0, V1, 3, 3, 3, 3); // Replicate 3

	ShuffleTest(V0, V1, 1, 2, 3, 0); // Rotate << 1
	ShuffleTest(V0, V1, 2, 3, 0, 1); // Rotate << 2
	ShuffleTest(V0, V1, 3, 0, 1, 2); // Rotate << 3

	// Lane swaps
	ShuffleTest(V0, V1, 0, 1, 2, 3); // lane 0, lane 1
	ShuffleTest(V0, V1, 2, 3, 0, 1); // lane 1, lane 0
	ShuffleTest(V0, V1, 0, 1, 0, 1); // lane 0, lane 0
	ShuffleTest(V0, V1, 2, 3, 2, 3); // lane 1, lane 1

	// Lane swaps with two in-lane permutes
	ShuffleTest(V0, V1, 1, 0, 3, 2); // lane 0, lane 1
	ShuffleTest(V0, V1, 3, 2, 1, 0); // lane 1, lane 0
	ShuffleTest(V0, V1, 1, 0, 1, 0); // lane 0, lane 0
	ShuffleTest(V0, V1, 3, 2, 3, 2); // lane 1, lane 1

	// Lane swaps with one in-lane permute (first lane)
	ShuffleTest(V0, V1, 1, 0, 2, 3); // lane 0, lane 1
	ShuffleTest(V0, V1, 3, 2, 0, 1); // lane 1, lane 0
	ShuffleTest(V0, V1, 1, 0, 0, 1); // lane 0, lane 0
	ShuffleTest(V0, V1, 3, 2, 2, 3); // lane 1, lane 1

	// Lane swaps with one in-lane permute (second lane)
	ShuffleTest(V0, V1, 0, 1, 3, 2); // lane 0, lane 1
	ShuffleTest(V0, V1, 2, 3, 1, 0); // lane 1, lane 0
	ShuffleTest(V0, V1, 0, 1, 1, 0); // lane 0, lane 0
	ShuffleTest(V0, V1, 2, 3, 3, 2); // lane 1, lane 1

	// Lane swaps with a cross-lane permute
	ShuffleTest(V0, V1, 0, 1, 0, 3); // lane 0, lane X
	ShuffleTest(V0, V1, 2, 3, 0, 3); // lane 1, lane X
	ShuffleTest(V0, V1, 0, 1, 2, 1); // lane 0, lane X
	ShuffleTest(V0, V1, 2, 3, 1, 2); // lane 1, lane X

	ShuffleTest(V0, V1, 3, 0, 3, 2); // lane X, lane 1
	ShuffleTest(V0, V1, 2, 0, 1, 0); // lane X, lane 0
	ShuffleTest(V0, V1, 1, 2, 1, 0); // lane X, lane 0
	ShuffleTest(V0, V1, 2, 1, 3, 2); // lane X, lane 1

	// In-lane permute with a cross-lane permute
	ShuffleTest(V0, V1, 1, 0, 3, 0); // lane 0, lane X
	ShuffleTest(V0, V1, 3, 2, 0, 3); // lane 1, lane X
	ShuffleTest(V0, V1, 1, 0, 2, 1); // lane 0, lane X
	ShuffleTest(V0, V1, 3, 2, 1, 2); // lane 1, lane X

	ShuffleTest(V0, V1, 3, 0, 3, 2); // lane X, lane 1
	ShuffleTest(V0, V1, 0, 3, 1, 0); // lane X, lane 0
	ShuffleTest(V0, V1, 1, 2, 1, 0); // lane X, lane 0
	ShuffleTest(V0, V1, 2, 1, 3, 2); // lane X, lane 1

	// Two cross-lane permutes
	ShuffleTest(V0, V1, 3, 0, 0, 3); // lane X, lane X
	ShuffleTest(V0, V1, 1, 2, 2, 1); // lane X, lane X
	ShuffleTest(V0, V1, 3, 0, 2, 1); // lane X, lane X
	ShuffleTest(V0, V1, 1, 2, 0, 3); // lane X, lane X
	ShuffleTest(V0, V1, 0, 2, 0, 2); // lane X, lane X
	ShuffleTest(V0, V1, 2, 0, 0, 2); // lane X, lane X

	// Common specializations or uses
	ShuffleTest(V0, V1, 0, 0, 1, 1);
	ShuffleTest(V0, V1, 0, 0, 2, 2);
	ShuffleTest(V0, V1, 0, 1, 2, 2);
	ShuffleTest(V0, V1, 0, 2, 2, 3);
	ShuffleTest(V0, V1, 1, 1, 3, 3);
	ShuffleTest(V0, V1, 1, 3, 1, 3);
	ShuffleTest(V0, V1, 1, 0, 3, 0);
	ShuffleTest(V0, V1, 2, 2, 3, 3);
	ShuffleTest(V0, V1, 2, 0, 3, 0);

	ShuffleTest(V0, V1, 3, 1, 3, 0);
	ShuffleTest(V0, V1, 2, 2, 0, 1);
	ShuffleTest(V0, V1, 0, 1, 3, 2);
	ShuffleTest(V0, V1, 1, 3, 0, 3);

#undef ShuffleTest
}


template<typename FloatType, typename VectorRegisterType>
FORCENOINLINE void TestVectorTrigFunctions()
{
	const VectorRegisterType Signs[] = {
			MakeVectorRegister(+1.0f, +1.0f, +1.0f, +1.0f),
			MakeVectorRegister(-1.0f, -1.0f, -1.0f, -1.0f),
	};

	// Sin, Cos
	{
		const VectorRegisterType QuadrantDegreesArray[] = {
			MakeVectorRegister(0.0f, 10.0f, 20.0f, 30.0f),
			MakeVectorRegister(45.0f, 60.0f, 70.0f, 80.0f),
		};

		const int32 Cycles = 3; // Go through a full circle this many times (negative and positive)
		for (int32 OffsetQuadrant = -4 * Cycles; OffsetQuadrant <= 4 * Cycles; ++OffsetQuadrant)
		{
			const FloatType OffsetFloat = (FloatType)OffsetQuadrant * FloatType(90.0); // Add 90 degrees repeatedly to cover all quadrants and wrap a few times
			const VectorRegisterType VOffset = VectorLoadFloat1(&OffsetFloat);
			for (VectorRegisterType const& VDegrees : QuadrantDegreesArray)
			{
				const VectorRegisterType VAnglesDegrees = VectorAdd(VOffset, VDegrees);
				const VectorRegisterType VAngles = VectorMultiply(VAnglesDegrees, GlobalVectorConstants::DEG_TO_RAD);
				VectorRegisterType S[3], C[3];
				TestReferenceSinCos<FloatType, VectorRegisterType>(S[0], C[0], VAngles);
				TestFastSinCos<FloatType, VectorRegisterType>(S[1], C[1], VAngles);
				TestVectorSinCos<FloatType, VectorRegisterType>(S[2], C[2], VAngles);

				const FloatType SinCosTolerance = FloatType(1e-5);
				LogTest<FloatType>(TEXT("SinCos (Sin): Ref vs Fast"), TestVectorsEqual_ComponentWiseError(S[0], S[1], SinCosTolerance));
				LogTest<FloatType>(TEXT("SinCos (Sin): Ref vs Vec"), TestVectorsEqual_ComponentWiseError(S[0], S[2], SinCosTolerance));
				LogTest<FloatType>(TEXT("SinCos (Cos): Ref vs Fast"), TestVectorsEqual_ComponentWiseError(C[0], C[1], SinCosTolerance));
				LogTest<FloatType>(TEXT("SinCos (Cos): Ref vs Vec"), TestVectorsEqual_ComponentWiseError(C[0], C[2], SinCosTolerance));

				VectorRegisterType ReferenceResult, VectorResult;

				const FloatType FastSinTolerance = FloatType(0.001091);
				LogTest<FloatType>(TEXT("Sin: Ref vs Vec"), TestVectorFunction1Param<FloatType, VectorRegisterType, FMath::Sin, VectorSin>(ReferenceResult, VectorResult, VAngles, FastSinTolerance));
				LogTest<FloatType>(TEXT("Cos: Ref vs Vec"), TestVectorFunction1Param<FloatType, VectorRegisterType, FMath::Cos, VectorCos>(ReferenceResult, VectorResult, VAngles, FastSinTolerance));
			}
		}
	}

	// Tan
	{
		const VectorRegisterType QuadrantDegreesArray[] = {
			MakeVectorRegister(0.0f, 0.1f, 45.0f, 60.0f),
			MakeVectorRegister(80.0f, 89.f, 91.0f, 120.0f),
			MakeVectorRegister(179.9f, 180.0f, 245.0f, 269.f),
			MakeVectorRegister(271.f, 320.f, 359.f, 359.9f),
		};

		VectorRegisterType ReferenceResult, VectorResult;
		for (VectorRegisterType const& Sign : Signs)
		{
			for (VectorRegisterType const& VDegrees : QuadrantDegreesArray)
			{
				VectorRegisterType TestValue = VectorMultiply(Sign, VDegrees);
				TestValue = VectorMultiply(TestValue, GlobalVectorConstants::DEG_TO_RAD);

				const FloatType TanTolerance = FloatType(1e-5);
				LogTest<FloatType>(TEXT("Tan: Ref vs Vec"), TestVectorFunction1Param<FloatType, VectorRegisterType, FMath::Tan, VectorTan>(ReferenceResult, VectorResult, TestValue, TanTolerance));
			}
		}
	}

	// ASin, ACos, ATan tests
	{
		const VectorRegisterType ValuesArray[] = {
			MakeVectorRegister(0.0f,  0.1f,  0.2f,  0.3f),
			MakeVectorRegister(0.4f,  0.5f,  0.6f,  0.7f),
			MakeVectorRegister(0.8f,  0.9f,  1.0f,  0.995f),
		};
		const VectorRegisterType Multipliers[] = {
			MakeVectorRegister(+1.0f, +1.0f, +1.0f, +1.0f),
			MakeVectorRegister(-1.0f, -1.0f, -1.0f, -1.0f),
		};

		VectorRegisterType ReferenceResult, VectorResult;
		for (VectorRegisterType const& Sign : Signs)
		{
			for (VectorRegisterType const& Value : ValuesArray)
			{
				const VectorRegisterType TestValue = VectorMultiply(Sign, Value);
				const FloatType GenericTrigTolerance = FloatType(1e-6);
				LogTest<FloatType>(TEXT("ASin: Ref vs Vec"), TestVectorFunction1Param<FloatType, VectorRegisterType, FMath::Asin, VectorASin>(ReferenceResult, VectorResult, TestValue, GenericTrigTolerance));
				LogTest<FloatType>(TEXT("ACos: Ref vs Vec"), TestVectorFunction1Param<FloatType, VectorRegisterType, FMath::Acos, VectorACos>(ReferenceResult, VectorResult, TestValue, GenericTrigTolerance));
				LogTest<FloatType>(TEXT("ATan: Ref vs Vec"), TestVectorFunction1Param<FloatType, VectorRegisterType, FMath::Atan, VectorATan>(ReferenceResult, VectorResult, TestValue, GenericTrigTolerance));
			}
		}
	}

	// ATan2
	{
		const VectorRegisterType ArrayY[] = {
			MakeVectorRegister(0.0f, -0.1f,  0.2f, -0.3f),
			MakeVectorRegister(0.4f,  0.5f, -0.6f,  0.7f),
			MakeVectorRegister(1.0f, -2.1f,  3.2f, -4.3f),
			MakeVectorRegister(-5.4f,  6.5f, -7.6f,  8.7f),
		};
		const VectorRegisterType ArrayX[] = {
			MakeVectorRegister(-5.4f, -6.5f, -7.6f, -8.7f),
			MakeVectorRegister(1.0f, -2.1f,  3.2f,  4.3f),
			MakeVectorRegister(0.4f, -0.5f, -0.6f,  0.7f),
			MakeVectorRegister(0.0f,  0.1f,  0.2f,  0.3f),
		};
		static_assert(UE_ARRAY_COUNT(ArrayY) == UE_ARRAY_COUNT(ArrayX));

		VectorRegisterType ReferenceResult, VectorResult;
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(ArrayY); Index++)
		{
			const FloatType GenericTrigTolerance = FloatType(1e-6);
			LogTest<FloatType>(TEXT("ATan2: Ref vs Vec"), TestVectorFunction2Params<FloatType, VectorRegisterType, FMath::Atan2, VectorATan2>(ReferenceResult, VectorResult, ArrayY[Index], ArrayX[Index], GenericTrigTolerance));
		}
	}
}


// Exp, Log tests
template<typename FloatType, typename VectorRegisterType>
FORCENOINLINE void TestVectorExpLogFunctions()
{
	const VectorRegisterType Signs[] = {
			MakeVectorRegister(+1.0f, +1.0f, +1.0f, +1.0f),
			MakeVectorRegister(-1.0f, -1.0f, -1.0f, -1.0f),
	};

	// Exp tests
	{
		const VectorRegisterType ValuesArray[] = {
			MakeVectorRegister(0.0f, 0.1f, 0.2f, 0.3f),
			MakeVectorRegister(1.4f, 1.5f, 2.0f, 2.7f),
			MakeVectorRegister(5.1f, 6.8f, 7.7f, 8.0f),
		};


		VectorRegisterType ReferenceResult, VectorResult;
		for (VectorRegisterType const& Sign : Signs)
		{
			for (VectorRegisterType const& Value : ValuesArray)
			{
				const VectorRegisterType TestValue = VectorMultiply(Sign, Value);
				const FloatType GenericTestTolerance = FloatType(1e-3);
				LogTest<FloatType>(TEXT("Exp: Ref vs Vec"), TestVectorFunction1Param<FloatType, VectorRegisterType, FMath::Exp, VectorExp>(ReferenceResult, VectorResult, TestValue, GenericTestTolerance));
				LogTest<FloatType>(TEXT("Exp2: Ref vs Vec"), TestVectorFunction1Param<FloatType, VectorRegisterType, FMath::Exp2, VectorExp2>(ReferenceResult, VectorResult, TestValue, GenericTestTolerance));
			}
		}
	}

	// Log tests
	{
		const VectorRegisterType ValuesArray[] = {
			MakeVectorRegister(0.01f, 0.1f,  0.2f,  0.3f),
			MakeVectorRegister(1.4f,  1.5f,  2.0f,  2.7f),
			MakeVectorRegister(9.1f,  8.8f, 13.7f, 17.0f),
		};
		VectorRegisterType ReferenceResult, VectorResult;

		for (VectorRegisterType const& Value : ValuesArray)
		{
			const FloatType GenericTestTolerance = FloatType(1e-6);
			LogTest<FloatType>(TEXT("Loge: Ref vs Vec"), TestVectorFunction1Param<FloatType, VectorRegisterType, FMath::Loge, VectorLog>(ReferenceResult, VectorResult, Value, GenericTestTolerance));
			LogTest<FloatType>(TEXT("Log2: Ref vs Vec"), TestVectorFunction1Param<FloatType, VectorRegisterType, FMath::Log2, VectorLog2>(ReferenceResult, VectorResult, Value, GenericTestTolerance));
		}
	}
}

template<typename TransformType>
FORCENOINLINE void TestTransformFunctions()
{
	using FloatType = typename TransformType::FReal;
	using VectorType = UE::Math::TVector<FloatType>;
	using RotatorType = UE::Math::TRotator<FloatType>;
	using Matrix44Type = UE::Math::TMatrix<FloatType>;

	// Negative scale isn't behaving nicely with FTransform, stick to positive non-uniform for now
	VectorType ScaleA(2.5f, 0.5f, 1.2f);
	VectorType ScaleB(2.215f, 0.75f, 1.215f);

	VectorType TranslationA(10.34f, -188.1f, 0.017f);
	VectorType TranslationB(-110.354f, 18.16f, 0.17017f);

	RotatorType RotationA(19.5f, 71.81f, 0.18f);
	RotatorType RotationB(-13.5f, 1.781f, 7.18f);

	TransformType TransformA(RotationA, TranslationA, ScaleA);
	TransformType TransformB(RotationB, TranslationB, ScaleB);

	TransformType TransformAB = TransformA * TransformB;

	TransformType ExpectedTransformB = TransformA.GetRelativeTransformReverse(TransformAB);

	LogTest<FloatType>(TEXT("FTransform::GetRelativeTransformReverse"), ExpectedTransformB.Equals(TransformB));
}


bool RunDoubleVectorTest()
{
	ResetPassing();

	double F1 = 1.0;
	uint64 U1 = *(uint64*)&F1;
	VectorRegister4Double V0, V1;
	VectorRegister4Double V2, V3;
	double Double0, Double1, Double2, Double3;

	// Using a union as we need to do a bitwise cast of 0xFFFFFFFFFFFFFFFF into a double for NaN.
	typedef union
	{
		uint64 IntNaN;
		double DoubleNaN;
	} Int64DoubleUnion;
	Int64DoubleUnion NaNU;
	NaNU.IntNaN = 0xFFFFFFFFFFFFFFFF;
	const double NaN = NaNU.DoubleNaN;
	const VectorRegister4Double VectorNaN = MakeVectorRegisterDouble(NaN, NaN, NaN, NaN);

	V0 = MakeVectorRegisterDouble(U1, U1, U1, U1);
	V1 = MakeVectorRegisterDouble(F1, F1, F1, F1);
	LogTest<double>(TEXT("MakeVectorRegister"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(0., 0., 0., 0.);
	V1 = VectorZeroDouble();
	LogTest<double>(TEXT("VectorZero"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(1., 1., 1., 1.);
	V1 = VectorOneDouble();
	LogTest<double>(TEXT("VectorOne"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(0.f, 1.f, 2.f, 3.f); // floats
	V1 = MakeVectorRegisterDouble(0.0, 1.0, 2.0, 3.0); // doubles
	LogTest<double>(TEXT("MakeVectorRegister4Double<float>, MakeVectorRegister4Double"), TestVectorsEqual(V0, V1));

	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, -0.25f, -0.5f);
	V1 = VectorLoad(GScratchDouble);
	LogTest<double>(TEXT("VectorLoad"), TestVectorsEqual(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = VectorLoad(GScratch);		// float*
	V1 = VectorLoad(GScratchDouble);// double*
	LogTest<double>(TEXT("VectorLoad<float>, VectorLoad"), TestVectorsEqual(V0, V1));

	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, -0.25f, -0.5f);
	V1 = VectorLoadAligned(GScratchDouble);
	LogTest<double>(TEXT("VectorLoadAligned"), TestVectorsEqual(V0, V1));

	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = VectorLoad(GScratchDouble + 1);
	V1 = VectorLoadDouble3(GScratchDouble + 1);
	LogTest<double>(TEXT("VectorLoadFloat3"), TestVectorsEqual3(V0, V1));

	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, -0.25f, 0.0f);
	V1 = VectorLoadDouble3_W0(GScratchDouble);
	LogTest<double>(TEXT("VectorLoadFloat3_W0"), TestVectorsEqual(V0, V1));

	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, -0.25f, 1.0f);
	V1 = VectorLoadDouble3_W1(GScratchDouble);
	LogTest<double>(TEXT("VectorLoadFloat3_W1"), TestVectorsEqual(V0, V1));

	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegisterDouble(-0.5f, -0.5f, -0.5f, -0.5f);
	V1 = VectorLoadDouble1(GScratchDouble + 3);
	LogTest<double>(TEXT("VectorLoadFloat1"), TestVectorsEqual(V0, V1));

	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = VectorSetDouble3(GScratchDouble[1], GScratchDouble[2], GScratchDouble[3]);
	GScratchDouble[4] = 127.;
	V1 = VectorLoadDouble3(GScratchDouble + 1);
	LogTest<double>(TEXT("VectorSet3"), TestVectorsEqual3(V0, V1));

	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = VectorSet(GScratchDouble[1], GScratchDouble[2], GScratchDouble[3], GScratchDouble[4]);
	V1 = VectorLoad(GScratchDouble + 1);
	LogTest<double>(TEXT("VectorSet"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, -0.25f, 1.0f);
	VectorStoreAligned(V0, GScratchDouble + 8);
	V1 = VectorLoad(GScratchDouble + 8);
	LogTest<double>(TEXT("VectorStoreAligned"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, -0.55f, 1.0f);
	VectorStore(V0, GScratchDouble + 7);
	V1 = VectorLoad(GScratchDouble + 7);
	LogTest<double>(TEXT("VectorStore"), TestVectorsEqual(V0, V1));

	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegisterDouble(5.0f, 3.0f, 1.0f, -1.0f);
	VectorStoreDouble3(V0, GScratchDouble);
	V1 = VectorLoad(GScratchDouble);
	V0 = MakeVectorRegisterDouble(5.0f, 3.0f, 1.0f, -0.5f);
	LogTest<double>(TEXT("VectorStoreFloat3"), TestVectorsEqual(V0, V1));

	SetScratchDouble(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegisterDouble(5.0f, 3.0f, 1.0f, -1.0f);
	VectorStoreDouble1(V0, GScratchDouble + 1);
	V1 = VectorLoad(GScratchDouble);
	V0 = MakeVectorRegisterDouble(1.0f, 5.0f, -0.25f, -0.5f);
	LogTest<double>(TEXT("VectorStoreFloat1"), TestVectorsEqual(V0, V1));

	// Replicate
	TestVectorReplicate<double, VectorRegister4Double>();

	// Swizzle
	TestVectorSwizzle<double, VectorRegister4Double>();

	// Shuffle
	TestVectorShuffle<double, VectorRegister4Double>();

	// VectorGetComponent
	V0 = MakeVectorRegisterDouble(0., 0., 2., 0.);
	V1 = MakeVectorRegisterDouble(0., 1., 0., 3.);
	Double0 = VectorGetComponent(V0, 0);
	Double1 = VectorGetComponent(V1, 1);
	Double2 = VectorGetComponent(V0, 2);
	Double3 = VectorGetComponent(V1, 3);
	V0 = MakeVectorRegisterDouble(0., 1., 2., 3.);
	V1 = MakeVectorRegisterDouble(Double0, Double1, Double2, Double3);
	LogTest<double>(TEXT("VectorGetComponent"), TestVectorsEqual(V0, V1));

	// Abs
	V0 = MakeVectorRegisterDouble(1.0f, -2.0f, 3.0f, -4.0f);
	V1 = VectorAbs(V0);
	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 4.0f);
	LogTest<double>(TEXT("VectorAbs"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(1.0f, -2.0f, 3.0f, -4.0f);
	V1 = VectorNegate(V0);
	V0 = MakeVectorRegisterDouble(-1.0f, 2.0f, -3.0f, 4.0f);
	LogTest<double>(TEXT("VectorNegate"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 4.0f);
	V1 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = VectorAdd(V0, V1);
	V0 = MakeVectorRegisterDouble(3.0f, 6.0f, 9.0f, 12.0f);
	LogTest<double>(TEXT("VectorAdd"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 4.0f);
	V1 = VectorSubtract(V0, V1);
	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 4.0f);
	LogTest<double>(TEXT("VectorSubtract"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 4.0f);
	V2 = VectorMultiply(V0, V1);
	V3 = MakeVectorRegisterDouble(2.0f, 8.0f, 18.0f, 32.0f);
	LogTest<double>(TEXT("VectorMultiply"), TestVectorsEqual(V3, V2));

	V0 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 4.0f);
	V2 = VectorMultiplyAdd(V0, V1, VectorOneDouble());
	V3 = MakeVectorRegisterDouble(3.0f, 9.0f, 19.0f, 33.0f);
	LogTest<double>(TEXT("VectorMultiplyAdd"), TestVectorsEqual(V3, V2));

	V0 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 4.0f);
	V1 = VectorDot3(V0, V1);
	V0 = MakeVectorRegisterDouble(28.0f, 28.0f, 28.0f, 28.0f);
	LogTest<double>(TEXT("VectorDot3"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 5.0f);
	V1 = MakeVectorRegisterDouble(7.0f, 11.0f, 13.0f, 17.0f);
	V1 = VectorDot3(V0, V1);
	V0 = MakeVectorRegisterDouble(68.0f, 68.0f, 68.0f, 68.0f);
	LogTest<double>(TEXT("VectorDot3"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 4.0f);
	Double0 = VectorDot3Scalar(V0, V1);
	V1 = VectorSetDouble1(Double0);
	V0 = MakeVectorRegisterDouble(28.0f, 28.0f, 28.0f, 28.0f);
	LogTest<double>(TEXT("VectorDot3Scalar"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 5.0f);
	V1 = MakeVectorRegisterDouble(7.0f, 11.0f, 13.0f, 17.0f);
	Double0 = VectorDot3Scalar(V0, V1);
	V1 = VectorSetDouble1(Double0);
	V0 = MakeVectorRegisterDouble(68.0f, 68.0f, 68.0f, 68.0f);
	LogTest<double>(TEXT("VectorDot3Scalar"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 4.0f);
	V1 = VectorDot4(V0, V1);
	V0 = MakeVectorRegisterDouble(60.0f, 60.0f, 60.0f, 60.0f);
	LogTest<double>(TEXT("VectorDot4"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(1.0f, 2.0f, 3.0f, 5.0f);
	V1 = MakeVectorRegisterDouble(7.0f, 11.0f, 13.0f, 17.0f);
	V1 = VectorDot4(V0, V1);
	V0 = MakeVectorRegisterDouble(153.0f, 153.0f, 153.0f, 153.0f);
	LogTest<double>(TEXT("VectorDot4"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(1.0f, 0.0f, 0.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(0.0f, 2.0f, 0.0f, 4.0f);
	V1 = VectorCross(V0, V1);
	V0 = MakeVectorRegisterDouble(0.f, 0.0f, 2.0f, 0.0f);
	LogTest<double>(TEXT("VectorCross"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(2.0, 3.0, 4.0, 8.0);
	V1 = MakeVectorRegisterDouble(5.0, 6.0, 7.0, 4.0);
	V1 = VectorCross(V0, V1);
	V0 = MakeVectorRegisterDouble(-3.0, 6.0, -3.0, 0.0);
	LogTest<double>(TEXT("VectorCross"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(2.0, 4.0, 6.0, 8.0);
	V1 = MakeVectorRegisterDouble(4.0, 3.0, 2.0, 1.0);
	V1 = VectorPow(V0, V1);
	V0 = MakeVectorRegister(16.0, 64.0, 36.0, 8.0);
	LogTest<double>(TEXT("VectorPow"), TestVectorsEqual(V0, V1, 0.001));

	// Vector component comparisons

	// VectorCompareGT
	const VectorRegister4Double VectorZeroMaskDouble = MakeVectorRegisterDoubleMask((uint64)0, (uint64)0, (uint64)0, (uint64)0);
	V0 = MakeVectorRegisterDouble(1.0f, 3.0f, 2.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(2.0f, 4.0f, 2.0f, 1.0f);
	V2 = VectorCompareGT(V0, V1);
	V3 = MakeVectorRegisterDouble((uint64)0, (uint64)0, (uint64)0, (uint64)-1);
	LogTest<double>(TEXT("VectorCompareGT"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareGT(V0, VectorNaN);
	V3 = VectorZeroMaskDouble;
	LogTest<double>(TEXT("VectorCompareGT (NaN)"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareGT(VectorNaN, V0);
	LogTest<double>(TEXT("VectorCompareGT (NaN)"), TestVectorsEqualBitwise(V2, V3));

	// VectorCompareGE
	V0 = MakeVectorRegisterDouble(1.0f, 3.0f, 2.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(2.0f, 4.0f, 2.0f, 1.0f);
	V2 = VectorCompareGE(V0, V1);
	V3 = MakeVectorRegisterDouble((uint64)0, (uint64)0, (uint64)-1, (uint64)-1);
	LogTest<double>(TEXT("VectorCompareGE"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareGE(V0, VectorNaN);
	V3 = VectorZeroMaskDouble;
	LogTest<double>(TEXT("VectorCompareGE (NaN)"), TestVectorsEqualBitwise(V2, V3));

	// VectorCompareLT
	V0 = MakeVectorRegisterDouble(2.0f, 4.0f, 2.0f, 1.0f);
	V1 = MakeVectorRegisterDouble(1.0f, 3.0f, 2.0f, 8.0f);
	V2 = VectorCompareLT(V0, V1);
	V3 = MakeVectorRegisterDouble((uint64)0, (uint64)0, (uint64)0, (uint64)-1);
	LogTest<double>(TEXT("VectorCompareLT"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareLT(V0, VectorNaN);
	V3 = VectorZeroMaskDouble;
	LogTest<double>(TEXT("VectorCompareLT (NaN)"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareLT(VectorNaN, V0);
	LogTest<double>(TEXT("VectorCompareLT (NaN)"), TestVectorsEqualBitwise(V2, V3));

	// VectorCompareLE
	V0 = MakeVectorRegisterDouble(2.0f, 4.0f, 2.0f, 1.0f);
	V1 = MakeVectorRegisterDouble(1.0f, 3.0f, 2.0f, 8.0f);
	V2 = VectorCompareLE(V0, V1);
	V3 = MakeVectorRegisterDouble((uint64)0, (uint64)0, (uint64)-1, (uint64)-1);
	LogTest<double>(TEXT("VectorCompareLE"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareLE(V0, VectorNaN);
	V3 = VectorZeroMaskDouble;
	LogTest<double>(TEXT("VectorCompareLE (NaN)"), TestVectorsEqualBitwise(V2, V3));

	// VectorCompareEQ
	V0 = MakeVectorRegisterDouble(1.0f, 3.0f, 2.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(2.0f, 4.0f, 2.0f, 1.0f);
	V2 = VectorCompareEQ(V0, V1);
	V3 = MakeVectorRegisterDouble((uint64)0, (uint64)0, (uint64)-1, (uint64)0);
	LogTest<double>(TEXT("VectorCompareEQ"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareEQ(V0, VectorNaN);
	V3 = VectorZeroMaskDouble;
	LogTest<double>(TEXT("VectorCompareEQ (NaN)"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareEQ(VectorNaN, V0);
	LogTest<double>(TEXT("VectorCompareEQ (NaN)"), TestVectorsEqualBitwise(V2, V3));
	// NaN:NaN comparisons are undefined according to the Intel instruction manual, and will vary in optimized versus debug builds.
	/*
	V2 = VectorCompareEQ(VectorNaN, VectorNaN);
	V3 = VectorZeroMaskDouble;
	LogTest<double>(TEXT("VectorCompareEQ (NaN:NaN)"), TestVectorsEqualBitwise(V2, V3));
	*/

	// VectorCompareNE
	V0 = MakeVectorRegisterDouble(1.0f, 3.0f, 2.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(2.0f, 4.0f, 2.0f, 1.0f);
	V2 = VectorCompareNE(V0, V1);
	V3 = MakeVectorRegisterDouble((uint64)(0xFFFFFFFFFFFFFFFFU), (uint64)(0xFFFFFFFFFFFFFFFFU), (uint64)(0), (uint64)(0xFFFFFFFFFFFFFFFFU));
	LogTest<double>(TEXT("VectorCompareNE"), TestVectorsEqualBitwise(V2, V3));
	// VectorCompareNE should return true if either argument is NaN
	V2 = VectorCompareNE(V0, VectorNaN);
	V3 = GlobalVectorConstants::DoubleAllMask();
	LogTest<double>(TEXT("VectorCompareNE (NaN)"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareNE(VectorNaN, V0);
	LogTest<double>(TEXT("VectorCompareNE (NaN)"), TestVectorsEqualBitwise(V2, V3));
	// NaN:NaN comparisons are undefined according to the Intel instruction manual, and will vary in optimized versus debug builds.
	/*
	V2 = VectorCompareNE(VectorNaN, VectorNaN);
	V3 = GlobalVectorConstants::DoubleAllMask;
	LogTest<double>(TEXT("VectorCompareNE (NaN:NaN)"), TestVectorsEqualBitwise(V2, V3));
	*/

	// VectorSelect
	V0 = MakeVectorRegisterDouble(1.0f, 3.0f, 5.0f, 7.0f);
	V1 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V2 = MakeVectorRegisterDouble((uint64)-1, (uint64)0, (uint64)0, (uint64)-1);
	V2 = VectorSelect(V2, V0, V1);
	V3 = MakeVectorRegisterDouble(1.0f, 4.0f, 6.0f, 7.0f);
	LogTest<double>(TEXT("VectorSelect"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegisterDouble(1.0f, 3.0f, 5.0f, 7.0f);
	V1 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V2 = MakeVectorRegisterDouble((uint64)0, (uint64)-1, (uint64)-1, (uint64)0);
	V2 = VectorSelect(V2, V0, V1);
	V3 = MakeVectorRegisterDouble(2.0f, 3.0f, 5.0f, 8.0f);
	LogTest<double>(TEXT("VectorSelect"), TestVectorsEqual(V2, V3));

	// Vector bitwise operations
	V0 = MakeVectorRegisterDouble(1.0f, 3.0f, 0.0f, 0.0f);
	V1 = MakeVectorRegisterDouble(0.0f, 0.0f, 2.0f, 1.0f);
	V2 = VectorBitwiseOr(V0, V1);
	V3 = MakeVectorRegisterDouble(1.0f, 3.0f, 2.0f, 1.0f);
	LogTest<double>(TEXT("VectorBitwiseOr-Double1"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegisterDouble(1.0f, 3.0f, 24.0f, 36.0f);
	V1 = MakeVectorRegisterDouble((uint64)(0x8000000000000000U), (uint64)(0x8000000000000000U), (uint64)(0x8000000000000000U), (uint64)(0x8000000000000000U));
	V2 = VectorBitwiseOr(V0, V1);
	V3 = MakeVectorRegisterDouble(-1.0f, -3.0f, -24.0f, -36.0f);
	LogTest<double>(TEXT("VectorBitwiseOr-Double2"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegisterDouble(-1.0f, -3.0f, -24.0f, 36.0f);
	V1 = MakeVectorRegisterDouble((uint64)(0xFFFFFFFFFFFFFFFFU), (uint64)(0x7FFFFFFFFFFFFFFFU), (uint64)(0x7FFFFFFFFFFFFFFFU), (uint64)(0xFFFFFFFFFFFFFFFFU));
	V2 = VectorBitwiseAnd(V0, V1);
	V3 = MakeVectorRegisterDouble(-1.0f, 3.0f, 24.0f, 36.0f);
	LogTest<double>(TEXT("VectorBitwiseAnd-Double"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegisterDouble(-1.0f, -3.0f, -24.0f, 36.0f);
	V1 = MakeVectorRegisterDouble((uint64)(0x8000000000000000U), (uint64)(0x0000000000000000U), (uint64)(0x8000000000000000U), (uint64)(0x8000000000000000U));
	V2 = VectorBitwiseXor(V0, V1);
	V3 = MakeVectorRegisterDouble(1.0f, -3.0f, 24.0f, -36.0f);
	LogTest<double>(TEXT("VectorBitwiseXor-Double"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegisterDouble(2.0f, -2.0f, 3.0f, -4.0f);
	V1 = VectorSet_W0(V0);
	V0 = MakeVectorRegisterDouble(2.0f, -2.0f, 3.0f, 0.0f);
	LogTest<double>(TEXT("VectorSet_W0"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(2.0f, -2.0f, 3.0f, -4.0f);
	V1 = VectorSet_W1(V0);
	V0 = MakeVectorRegisterDouble(2.0f, -2.0f, 3.0f, 1.0f);
	LogTest<double>(TEXT("VectorSet_W1"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(4.0f, 3.0f, 2.0f, 1.0f);
	V1 = VectorMin(V0, V1);
	V0 = MakeVectorRegisterDouble(2.0f, 3.0f, 2.0f, 1.0f);
	LogTest<double>(TEXT("VectorMin"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegisterDouble(4.0f, 3.0f, 2.0f, 1.0f);
	V1 = VectorMax(V0, V1);
	V0 = MakeVectorRegisterDouble(4.0f, 4.0f, 6.0f, 8.0f);
	LogTest<double>(TEXT("VectorMax"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(-2.0f, 3.0f, -4.0f, 5.0f);
	V1 = MakeVectorRegisterDouble(4.0f, 4.0f, 6.0f, 8.0f);
	V2 = MakeVectorRegisterDouble(-8.0f, 5.0f, -1.0f, 1.0f);
	V2 = VectorClamp(V2, V0, V1);
	V3 = MakeVectorRegisterDouble(-2.0f, 4.0f, -1.0f, 5.0f);
	LogTest<double>(TEXT("VectorClamp"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegisterDouble(1.0, 3.0, 5.0, 7.0);
	V1 = MakeVectorRegisterDouble(2.0, 4.0, 6.0, 8.0);
	V1 = VectorCombineHigh(V0, V1);
	V0 = MakeVectorRegisterDouble(5.0, 7.0, 6.0, 8.0);
	LogTest<double>(TEXT("VectorCombineHigh"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegisterDouble(1.0, 3.0, 5.0, 7.0);
	V1 = MakeVectorRegisterDouble(2.0, 4.0, 6.0, 8.0);
	V1 = VectorCombineLow(V0, V1);
	V0 = MakeVectorRegisterDouble(1.0, 3.0, 2.0, 4.0);
	LogTest<double>(TEXT("VectorCombineLow"), TestVectorsEqual(V0, V1));

	//TODO: Rename functions to explicitly mention Double?
	V0 = MakeVectorRegister(2.0, 4.0, 6.0, 8.0);
	V1 = MakeVectorRegister(4.0, 3.0, 2.0, 1.0);
	bool bIsVAGT_TRUE = VectorAnyGreaterThan(V0, V1) != 0;
	LogTest<double>(TEXT("VectorAnyGreaterThan-true"), bIsVAGT_TRUE);

	V0 = MakeVectorRegister(0.0, -0.0, 1.0, 0.8);
	V1 = MakeVectorRegister(0.0, 0.0, 0.0, 1.0);
	bIsVAGT_TRUE = VectorAnyGreaterThan(V0, V1) != 0;
	LogTest<double>(TEXT("VectorAnyGreaterThan-true"), bIsVAGT_TRUE);

	V0 = MakeVectorRegister(1.0, 3.0, 2.0, 1.0);
	V1 = MakeVectorRegister(2.0, 4.0, 6.0, 8.0);
	bool bIsVAGT_FALSE = VectorAnyGreaterThan(V0, V1) == 0;
	LogTest<double>(TEXT("VectorAnyGreaterThan-false"), bIsVAGT_FALSE);

	V0 = MakeVectorRegister(1.0, 3.0, 2.0, 1.0);
	V1 = MakeVectorRegister(2.0, 4.0, 6.0, 8.0);
	LogTest<double>(TEXT("VectorAnyLesserThan-true"), VectorAnyLesserThan(V0, V1) != 0);

	V0 = MakeVectorRegister(3.0, 5.0, 7.0, 9.0);
	V1 = MakeVectorRegister(2.0, 4.0, 6.0, 8.0);
	LogTest<double>(TEXT("VectorAnyLesserThan-false"), VectorAnyLesserThan(V0, V1) == 0);

	V0 = MakeVectorRegister(3.0, 5.0, 7.0, 9.0);
	V1 = MakeVectorRegister(2.0, 4.0, 6.0, 8.0);
	LogTest<double>(TEXT("VectorAllGreaterThan-true"), VectorAllGreaterThan(V0, V1) != 0);

	V0 = MakeVectorRegister(3.0, 1.0, 7.0, 9.0);
	V1 = MakeVectorRegister(2.0, 4.0, 6.0, 8.0);
	LogTest<double>(TEXT("VectorAllGreaterThan-false"), VectorAllGreaterThan(V0, V1) == 0);

	V0 = MakeVectorRegister(1.0, 3.0, 2.0, 1.0);
	V1 = MakeVectorRegister(2.0, 4.0, 6.0, 8.0);
	LogTest<double>(TEXT("VectorAllLesserThan-true"), VectorAllLesserThan(V0, V1) != 0);

	V0 = MakeVectorRegister(3.0, 3.0, 2.0, 1.0);
	V1 = MakeVectorRegister(2.0, 4.0, 6.0, 8.0);
	LogTest<double>(TEXT("VectorAllLesserThan-false"), VectorAllLesserThan(V0, V1) == 0);

	V0 = MakeVectorRegister(1.0, 3.0, 2.0, 8.0);
	V1 = MakeVectorRegister(2.0, 4.0, 2.0, 1.0);
	V2 = VectorCompareGT(V0, V1);
	V3 = MakeVectorRegisterDouble((uint64)0, (uint64)0, (uint64)0, (uint64)-1);
	LogTest<double>(TEXT("VectorCompareGT"), TestVectorsEqualBitwise(V2, V3));

	V0 = MakeVectorRegister(1.0, 3.0, 2.0, 8.0);
	V1 = MakeVectorRegister(2.0, 4.0, 2.0, 1.0);
	V2 = VectorCompareGE(V0, V1);
	V3 = MakeVectorRegisterDouble((uint64)0, (uint64)0, (uint64)-1, (uint64)-1);
	LogTest<double>(TEXT("VectorCompareGE"), TestVectorsEqualBitwise(V2, V3));

	V0 = MakeVectorRegister(1.0, 3.0, 2.0, 8.0);
	V1 = MakeVectorRegister(2.0, 4.0, 2.0, 1.0);
	V2 = VectorCompareEQ(V0, V1);
	V3 = MakeVectorRegisterDouble((uint64)0, (uint64)0, (uint64)-1, (uint64)0);
	LogTest<double>(TEXT("VectorCompareEQ"), TestVectorsEqualBitwise(V2, V3));

	V0 = MakeVectorRegister(1.0, 3.0, 2.0, 8.0);
	V1 = MakeVectorRegister(2.0, 4.0, 2.0, 1.0);
	V2 = VectorCompareNE(V0, V1);
	V3 = MakeVectorRegisterDouble((uint64)(0xFFFFFFFFFFFFFFFF), (uint64)(0xFFFFFFFFFFFFFFFF), (uint64)(0), (uint64)(0xFFFFFFFFFFFFFFFF));
	LogTest<double>(TEXT("VectorCompareNE"), TestVectorsEqualBitwise(V2, V3));

	V0 = MakeVectorRegister(-1.0, -3.0, -24.0, 36.0);
	V1 = MakeVectorRegister(5.0, 35.0, 23.0, 48.0);
	V2 = VectorMergeVecXYZ_VecW(V0, V1);
	V3 = MakeVectorRegister(-1.0, -3.0, -24.0, 48.0);
	LogTest<double>(TEXT("VectorMergeXYZ_VecW-1"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(-1.0, -3.0, -24.0, 36.0);
	V1 = MakeVectorRegister(5.0, 35.0, 23.0, 48.0);
	V2 = VectorMergeVecXYZ_VecW(V1, V0);
	V3 = MakeVectorRegister(5.0, 35.0, 23.0, 36.0);
	LogTest<double>(TEXT("VectorMergeXYZ_VecW-2"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(1.0, 1.0e6, 1.3e-8, 35.0);
	V1 = VectorReciprocalEstimate(V0);
	V3 = VectorMultiply(V1, V0);
	LogTest<double>(TEXT("VectorReciprocalEstimate"), TestVectorsEqual(VectorOneDouble(), V3, 0.008));

	V0 = MakeVectorRegister(1.0, 1.0e6, 1.3e-8, 35.0);
	V1 = VectorReciprocal(V0);
	V3 = VectorMultiply(V1, V0);
	LogTest<double>(TEXT("VectorReciprocalAccurate"), TestVectorsEqual(VectorOneDouble(), V3, 1e-7));

	V0 = MakeVectorRegister(1.0, 1.0e6, 1.3e-8, 35.0);
	V1 = VectorReciprocalSqrtEstimate(V0);
	V3 = VectorMultiply(VectorMultiply(V1, V1), V0);
	LogTest<double>(TEXT("VectorReciprocalSqrtEstimate"), TestVectorsEqual(VectorOneDouble(), V3, 0.007));

	V0 = MakeVectorRegister(1.0, 1.0e6, 1.3e-8, 35.0);
	V1 = VectorReciprocalSqrt(V0);
	V3 = VectorMultiply(VectorMultiply(V1, V1), V0);
	LogTest<double>(TEXT("VectorReciprocalSqrtAccurate"), TestVectorsEqual(VectorOneDouble(), V3, 1e-6));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = VectorReciprocalLenEstimate(V0);
	V0 = MakeVectorRegister(0.25f, 0.25f, 0.25f, 0.25f);
	LogTest<double>(TEXT("VectorReciprocalLenEstimate"), TestVectorsEqual(V0, V1, 0.004));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = VectorReciprocalLen(V0);
	V0 = MakeVectorRegister(0.25f, 0.25f, 0.25f, 0.25f);
	LogTest<double>(TEXT("VectorReciprocalLenAccurate"), TestVectorsEqual(V0, V1, 0.0001));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = VectorNormalizeEstimate(V0);
	V0 = MakeVectorRegister(0.5f, -0.5f, 0.5f, -0.5f);
	LogTest<double>(TEXT("VectorNormalizeEstimate"), TestVectorsEqual(V0, V1, 0.004));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = VectorNormalize(V0);
	V0 = MakeVectorRegister(0.5f, -0.5f, 0.5f, -0.5f);
	LogTest<double>(TEXT("VectorNormalizeAccurate"), TestVectorsEqual(V0, V1, 1e-8));

	// VectorMod
	V0 = MakeVectorRegister(0.0, 3.2, 2.8, 1.5);
	V1 = MakeVectorRegister(2.0, 1.2, 2.0, 3.0);
	V2 = TestReferenceMod(V0, V1);
	V3 = VectorMod(V0, V1);
	LogTest<double>(TEXT("VectorMod positive"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(-2.0, 3.2, -2.8, -1.5);
	V1 = MakeVectorRegister(-1.5, -1.2, 2.0, 3.0);
	V2 = TestReferenceMod(V0, V1);
	V3 = VectorMod(V0, V1);
	LogTest<double>(TEXT("VectorMod negative"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(89.9, 180.0, -256.0, -270.1);
	V1 = MakeVectorRegister(360.0, 0.1, 360.0, 180.0);
	V2 = TestReferenceMod(V0, V1);
	V3 = VectorMod(V0, V1);
	LogTest<double>(TEXT("VectorMod common"), TestVectorsEqual(V2, V3));

	// VectorMod360
	V0 = MakeVectorRegister(89.9, 180.0, -256.0, -270.1);
	V1 = GlobalVectorConstants::Double360;
	V2 = TestReferenceMod(V0, V1);
	V3 = VectorMod360(V0);
	LogTest<double>(TEXT("VectorMod360"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(0.0, 1079.9, -1720.1, 12345.12345);
	V1 = GlobalVectorConstants::Double360;
	V2 = TestReferenceMod(V0, V1);
	V3 = VectorMod360(V0);
	LogTest<double>(TEXT("VectorMod360"), TestVectorsEqual(V2, V3));

#if UE_BUILD_DEBUG
	V1 = GlobalVectorConstants::Double360;
	double RotStep = 1.01;
	for (double F = -720.0; F <= 720.0; F += RotStep)
	{
		V0 = MakeVectorRegister(F, F + 0.001, F + 0.025, F + 0.6125);
		V2 = TestReferenceMod(V0, V1);
		V3 = VectorMod360(V0);
		LogTest<double>(TEXT("VectorMod360"), TestVectorsEqual(V2, V3));
	}
#endif // UE_BUILD_DEBUG

	// VectorSign
	V0 = MakeVectorRegister(2.0, -2.0, 0.0, -3.0);
	V2 = MakeVectorRegister(1.0, -1.0, 1.0, -1.0);
	V3 = VectorSign(V0);
	LogTest<double>(TEXT("VectorSign"), TestVectorsEqual(V2, V3));

	// VectorStep
	V0 = MakeVectorRegister(2.0, -2.0, 0.0, -3.0);
	V2 = MakeVectorRegister(1.0, 0.0, 1.0, 0.0);
	V3 = VectorStep(V0);
	LogTest<double>(TEXT("VectorStep"), TestVectorsEqual(V2, V3));

	// VectorTruncate
	V0 = MakeVectorRegister(-1.8, -1.0, -0.8, 0.0);
	V2 = MakeVectorRegister(-1.0, -1.0, 0.0, 0.0);
	V3 = VectorTruncate(V0);
	LogTest<double>(TEXT("VectorTruncate"), TestVectorsEqual(V2, V3, UE_DOUBLE_KINDA_SMALL_NUMBER));

	V0 = MakeVectorRegister(0.0, 0.8, 1.0, 1.8);
	V2 = MakeVectorRegister(0.0, 0.0, 1.0, 1.0);
	V3 = VectorTruncate(V0);
	LogTest<double>(TEXT("VectorTruncate"), TestVectorsEqual(V2, V3, UE_DOUBLE_KINDA_SMALL_NUMBER));

	V0 = MakeVectorRegister(0.1, 1.8, 2.4, -2.4);
	V2 = MakeVectorRegister(0.0, 1.0, 2.0, -2.0);
	V3 = VectorTruncate(V0);
	LogTest<double>(TEXT("VectorTruncate"), TestVectorsEqual(V2, V3, UE_DOUBLE_KINDA_SMALL_NUMBER));

	// VectorFractional
	V0 = MakeVectorRegister(-1.8, -1.0, -0.8, 0.0);
	V2 = MakeVectorRegister(-0.8, 0.0, -0.8, 0.0);
	V3 = VectorFractional(V0);
	LogTest<double>(TEXT("VectorFractional"), TestVectorsEqual(V2, V3, UE_DOUBLE_KINDA_SMALL_NUMBER));

	V0 = MakeVectorRegister(0.0, 0.8, 1.0, 1.8);
	V2 = MakeVectorRegister(0.0, 0.8, 0.0, 0.8);
	V3 = VectorFractional(V0);
	LogTest<double>(TEXT("VectorFractional"), TestVectorsEqual(V2, V3, UE_DOUBLE_KINDA_SMALL_NUMBER));

	// VectorCeil
	V0 = MakeVectorRegister(-1.8, -1.0, -0.8, 0.0);
	V2 = MakeVectorRegister(-1.0, -1.0, -0.0, 0.0);
	V3 = VectorCeil(V0);
	LogTest<double>(TEXT("VectorCeil"), TestVectorsEqual(V2, V3, UE_DOUBLE_KINDA_SMALL_NUMBER));

	V0 = MakeVectorRegister(0.0, 0.8, 1.0, 1.8);
	V2 = MakeVectorRegister(0.0, 1.0, 1.0, 2.0);
	V3 = VectorCeil(V0);
	LogTest<double>(TEXT("VectorCeil"), TestVectorsEqual(V2, V3, UE_DOUBLE_KINDA_SMALL_NUMBER));

	// VectorFloor
	V0 = MakeVectorRegister(-1.8, -1.0, -0.8, 0.0);
	V2 = MakeVectorRegister(-2.0, -1.0, -1.0, 0.0);
	V3 = VectorFloor(V0);
	LogTest<double>(TEXT("VectorFloor"), TestVectorsEqual(V2, V3, UE_DOUBLE_KINDA_SMALL_NUMBER));

	V0 = MakeVectorRegister(0.0, 0.8, 1.0, 1.8);
	V2 = MakeVectorRegister(0.0, 0.0, 1.0, 1.0);
	V3 = VectorFloor(V0);
	LogTest<double>(TEXT("VectorFloor"), TestVectorsEqual(V2, V3));

	// VectorDeinterleave
	V0 = MakeVectorRegister(0.0, 1.0, 2.0, 3.0);
	V1 = MakeVectorRegister(4.0, 5.0, 6.0, 7.0);
	VectorDeinterleave(V2, V3, V0, V1);
	V0 = MakeVectorRegister(0.0, 2.0, 4.0, 6.0);
	V1 = MakeVectorRegister(1.0, 3.0, 5.0, 7.0);
	LogTest<double>(TEXT("VectorDeinterleave"), TestVectorsEqual(V2, V0) && TestVectorsEqual(V3, V1));

	// VectorMaskBits
	V0 = MakeVectorRegister(1.0, 2.0, 3.0, 4.0);
	uint32 MaskBits = VectorMaskBits(V0);
	LogTest<double>(TEXT("VectorMaskBits"), MaskBits == 0);
	V0 = MakeVectorRegister(-1.0, -2.0, -3.0, -4.0);
	MaskBits = VectorMaskBits(V0);
	LogTest<double>(TEXT("VectorMaskBits"), MaskBits == 0xf);
	V0 = MakeVectorRegister(-1.0, 2.0, -3.0, 4.0);
	MaskBits = VectorMaskBits(V0);
	LogTest<double>(TEXT("VectorMaskBits"), MaskBits == 5);

	// Matrix multiplications and transformations
	{
		FMatrix44d	M0, M1, M2, M3;
		FVector3d Eye, LookAt, Up;
		// Create Look at Matrix
		Eye = FVector3d(1024.0f, -512.0f, -2048.0f);
		LookAt = FVector3d(0.0f, 0.0f, 0.0f);
		Up = FVector3d(0.0f, 1.0f, 0.0f);
		M0 = FLookAtMatrix(Eye, LookAt, Up);

		// Create GL ortho projection matrix
		const double Width = 1920.0f;
		const double Height = 1080.0f;
		const double Left = 0.0f;
		const double Right = Left + Width;
		const double Top = 0.0f;
		const double Bottom = Top + Height;
		const double ZNear = -100.0f;
		const double ZFar = 100.0f;

		M1 = FMatrix44d(FPlane4d(2.0f / (Right - Left), 0, 0, 0),
			FPlane4d(0, 2.0f / (Top - Bottom), 0, 0),
			FPlane4d(0, 0, 1 / (ZNear - ZFar), 0),
			FPlane4d((Left + Right) / (Left - Right), (Top + Bottom) / (Bottom - Top), ZNear / (ZNear - ZFar), 1));

		VectorMatrixMultiply(&M2, &M0, &M1);
		TestVectorMatrixMultiply(&M3, &M0, &M1);
		LogTest<double>(TEXT("VectorMatrixMultiply"), TestMatricesEqual(M2, M3, 0.000001));

		VectorMatrixInverse(&M2, &M1);
		TestVectorMatrixInverse(&M3, &M1);
		LogTest<double>(TEXT("VectorMatrixInverse"), TestMatricesEqual(M2, M3, 0.000001));

		// 	FTransform Transform;
		// 	Transform.SetFromMatrix(M1);
		// 	FTransform InvTransform = Transform.Inverse();
		// 	FTransform InvTransform2 = FTransform(Transform.ToMatrixWithScale().Inverse());
		// 	LogTest<double>( TEXT("FTransform Inverse"), InvTransform.Equals(InvTransform2, 1e-3f ) );

		V0 = MakeVectorRegister(100.0f, -100.0f, 200.0f, 1.0f);
		V1 = VectorTransformVector(V0, &M0);
		V2 = TestVectorTransformVector(V0, &M0);
		LogTest<double>(TEXT("VectorTransformVector"), TestVectorsEqual(V1, V2, 1e-8));

		V0 = MakeVectorRegister(32768.0f, 131072.0f, -8096.0f, 1.0f);
		V1 = VectorTransformVector(V0, &M1);
		V2 = TestVectorTransformVector(V0, &M1);
		LogTest<double>(TEXT("VectorTransformVector"), TestVectorsEqual(V1, V2, 1e-8));
	}

	// FTransform
	TestTransformFunctions<FTransform3d>();

	// Quat / Rotator conversion to vectors, matrices
	{
		FQuat4d Q0, Q1;

		FRotator3d Rotator0;
		Rotator0 = FRotator3d(30.0f, -45.0f, 90.0f);
		Q0 = FQuat4d(Rotator0);
		Q1 = TestRotatorToQuaternion(Rotator0);
		LogTest<double>(TEXT("TestRotatorToQuaternion"), TestQuatsEqual(Q0, Q1, 1e-6));

		using namespace UE::Math;

		FVector3d FV0, FV1;
		FV0 = Rotator0.Vector();
		FV1 = TRotationMatrix<double>(Rotator0).GetScaledAxis(EAxis::X);
		LogTest<double>(TEXT("Test0 Rotator::Vector()"), TestFVector3Equal(FV0, FV1, 1e-6));

		FV0 = TRotationMatrix<double>(Rotator0).GetScaledAxis(EAxis::X);
		FV1 = TQuatRotationMatrix<double>(FQuat4d(Q0.X, Q0.Y, Q0.Z, Q0.W)).GetScaledAxis(EAxis::X);
		LogTest<double>(TEXT("Test0 FQuatRotationMatrix"), TestFVector3Equal(FV0, FV1, 1e-5));

		Rotator0 = FRotator3d(45.0f, 60.0f, 120.0f);
		Q0 = FQuat4d(Rotator0);
		Q1 = TestRotatorToQuaternion(Rotator0);
		LogTest<double>(TEXT("TestRotatorToQuaternion"), TestQuatsEqual(Q0, Q1, 1e-6f));

		FV0 = Rotator0.Vector();
		FV1 = TRotationMatrix<double>(Rotator0).GetScaledAxis(EAxis::X);
		LogTest<double>(TEXT("Test1 Rotator::Vector()"), TestFVector3Equal(FV0, FV1, 1e-6));

		FV0 = TRotationMatrix<double>(Rotator0).GetScaledAxis(EAxis::X);
		FV1 = TQuatRotationMatrix<double>(FQuat4d(Q0.X, Q0.Y, Q0.Z, Q0.W)).GetScaledAxis(EAxis::X);
		LogTest<double>(TEXT("Test1 FQuatRotationMatrix"), TestFVector3Equal(FV0, FV1, 1e-5));

		FV0 = TRotationMatrix<double>(FRotator3d::ZeroRotator).GetScaledAxis(EAxis::X);
		FV1 = TQuatRotationMatrix<double>(FQuat4d::Identity).GetScaledAxis(EAxis::X);
		LogTest<double>(TEXT("Test2 FQuatRotationMatrix"), TestFVector3Equal(FV0, FV1, 1e-6));
	}

	// NaN / Inf tests
	SetScratch(0.0, 0.0, 0.0, 0.0);
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(MakeVectorRegisterDouble(0.0, 1.0, -1.0, 0.0)));
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(MakeVectorRegisterDouble(NaN, NaN, NaN, NaN)));
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(MakeVectorRegisterDouble(NaN, 0.0, 0.0, 0.0)));
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(MakeVectorRegisterDouble(0.0, 0.0, 0.0, NaN)));
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(GlobalVectorConstants::DoubleInfinity()));
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(MakeVectorRegisterDouble((uint64)0xFFF0000000000000ULL, (uint64)0xFFF0000000000000ULL,
		(uint64)0xFFF0000000000000ULL, (uint64)0xFFF0000000000000ULL))); // negative infinity
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(GlobalVectorConstants::DoubleAllMask()));

	// Not Nan/Inf
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(GlobalVectorConstants::DoubleZero));
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(GlobalVectorConstants::DoubleOne));
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(GlobalVectorConstants::DoubleMinusOneHalf));
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(GlobalVectorConstants::DoubleSmallNumber));
	LogTest<double>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(GlobalVectorConstants::DoubleBigNumber));

	// Sin, Cos, Tan tests
	TestVectorTrigFunctions<double, VectorRegister4Double>();

	// Exp, Log tests
	TestVectorExpLogFunctions<double, VectorRegister4Double>();

	// Quat multiplication, Matrix conversion
	{
		FQuat4d Q0, Q1, Q2, Q3;
		FMatrix44d M0, M1;
		FMatrix44d IdentityInverse = FMatrix44d::Identity.Inverse();

		Q0 = FQuat4d(FRotator3d(30.0f, -45.0f, 90.0f));
		Q1 = FQuat4d(FRotator3d(45.0f, 60.0f, 120.0f));
		VectorQuaternionMultiply(&Q2, &Q0, &Q1);
		TestVectorQuaternionMultiply(&Q3, &Q0, &Q1);
		LogTest<double>(TEXT("VectorQuaternionMultiply"), TestQuatsEqual(Q2, Q3, 1e-6));
		V0 = VectorLoadAligned(&Q0);
		V1 = VectorLoadAligned(&Q1);
		V2 = VectorQuaternionMultiply2(V0, V1);
		V3 = VectorLoadAligned(&Q3);
		LogTest<double>(TEXT("VectorQuaternionMultiply2"), TestVectorsEqual(V2, V3, 1e-6));

		M0 = Q0 * FMatrix44d::Identity;
		M1 = UE::Math::TQuatRotationMatrix<double>(Q0);
		LogTest<double>(TEXT("QuaterionMatrixConversion Q0"), TestMatricesEqual(M0, M1, 0.000001));
		M0 = Q1.ToMatrix();
		M1 = UE::Math::TQuatRotationMatrix<double>(Q1);
		LogTest<double>(TEXT("QuaterionMatrixConversion Q1"), TestMatricesEqual(M0, M1, 0.000001));

		Q0 = FQuat4d(FRotator3d(0.0f, 180.0f, 45.0f));
		Q1 = FQuat4d(FRotator3d(-120.0f, -90.0f, 0.0f));
		VectorQuaternionMultiply(&Q2, &Q0, &Q1);
		TestVectorQuaternionMultiply(&Q3, &Q0, &Q1);
		LogTest<double>(TEXT("VectorQuaternionMultiply"), TestQuatsEqual(Q2, Q3, 1e-6));
		V0 = VectorLoadAligned(&Q0);
		V1 = VectorLoadAligned(&Q1);
		V2 = VectorQuaternionMultiply2(V0, V1);
		V3 = VectorLoadAligned(&Q3);
		LogTest<double>(TEXT("VectorQuaternionMultiply2"), TestVectorsEqual(V2, V3, 1e-6));

		M0 = (-Q0) * FMatrix44d::Identity;
		M1 = (-Q0).ToMatrix();
		LogTest<double>(TEXT("QuaterionMatrixConversion Q0"), TestMatricesEqual(M0, M1, 0.000001));
		M0 = (Q1.Inverse().Inverse()) * IdentityInverse;
		M1 = Q1.ToMatrix();
		LogTest<double>(TEXT("QuaterionMatrixConversion Q1"), TestMatricesEqual(M0, M1, 0.000001));
	}

	return GPassing;
}

/**
 * Run a suite of vector operations to validate vector intrinsics are working on the platform
 */
TEST_CASE_NAMED(FVectorRegisterAbstractionTest, "System::Core::Math::Vector Register Abstraction Test", "[ApplicationContextMask][SmokeFilter]")
{
	float F1 = 1.f;
	uint32 U1 = *(uint32*)&F1;
	VectorRegister4Float V0, V1, V2, V3;
	VectorRegister4Int VI0, VI1, VI2, VI3;
	float Float0, Float1, Float2, Float3;

	// Using a union as we need to do a bitwise cast of 0xFFFFFFFF into a float for NaN.
	typedef union
	{
		unsigned int IntNaN;
		float FloatNaN;
	} IntFloatUnion;
	IntFloatUnion NaNU;
	NaNU.IntNaN = 0xFFFFFFFF;
	const float NaN = NaNU.FloatNaN;
	const VectorRegister4Float VectorNaN = MakeVectorRegisterFloat(NaN, NaN, NaN, NaN);

	ResetPassing();

	V0 = MakeVectorRegister(U1, U1, U1, U1);
	V1 = MakeVectorRegister(F1, F1, F1, F1);
	LogTest<float>(TEXT("MakeVectorRegister"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(0.f, 0.f, 0.f, 0.f);
	V1 = VectorZero();
	LogTest<float>(TEXT("VectorZero"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(1.f, 1.f, 1.f, 1.f);
	V1 = VectorOne();
	LogTest<float>(TEXT("VectorOne"), TestVectorsEqual(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegister(1.0f, 2.0f, -0.25f, -0.5f);
	V1 = VectorLoad(GScratch);
	LogTest<float>(TEXT("VectorLoad"), TestVectorsEqual(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegister(1.0f, 2.0f, -0.25f, -0.5f);
	V1 = VectorLoad(GScratch);
	LogTest<float>(TEXT("VectorLoad"), TestVectorsEqual(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegister(1.0f, 2.0f, -0.25f, -0.5f);
	V1 = VectorLoadAligned(GScratch);
	LogTest<float>(TEXT("VectorLoadAligned"), TestVectorsEqual(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = VectorLoad(GScratch + 1);
	V1 = VectorLoadFloat3(GScratch + 1);
	LogTest<float>(TEXT("VectorLoadFloat3"), TestVectorsEqual3(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegister(1.0f, 2.0f, -0.25f, 0.0f);
	V1 = VectorLoadFloat3_W0(GScratch);
	LogTest<float>(TEXT("VectorLoadFloat3_W0"), TestVectorsEqual(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegister(1.0f, 2.0f, -0.25f, 1.0f);
	V1 = VectorLoadFloat3_W1(GScratch);
	LogTest<float>(TEXT("VectorLoadFloat3_W1"), TestVectorsEqual(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegister(-0.5f, -0.5f, -0.5f, -0.5f);
	V1 = VectorLoadFloat1(GScratch + 3);
	LogTest<float>(TEXT("VectorLoadFloat1"), TestVectorsEqual(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = VectorSetFloat3(GScratch[1], GScratch[2], GScratch[3]);
	V1 = VectorLoadFloat3(GScratch + 1);
	LogTest<float>(TEXT("VectorSetFloat3"), TestVectorsEqual3(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = VectorSet(GScratch[1], GScratch[2], GScratch[3], GScratch[4]);
	V1 = VectorLoad(GScratch + 1);
	LogTest<float>(TEXT("VectorSet"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(1.0f, 2.0f, -0.25f, 1.0f);
	VectorStoreAligned(V0, GScratch + 8);
	V1 = VectorLoad(GScratch + 8);
	LogTest<float>(TEXT("VectorStoreAligned"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(1.0f, 2.0f, -0.55f, 1.0f);
	VectorStore(V0, GScratch + 7);
	V1 = VectorLoad(GScratch + 7);
	LogTest<float>(TEXT("VectorStore"), TestVectorsEqual(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegister(5.0f, 3.0f, 1.0f, -1.0f);
	VectorStoreFloat3(V0, GScratch);
	V1 = VectorLoad(GScratch);
	V0 = MakeVectorRegister(5.0f, 3.0f, 1.0f, -0.5f);
	LogTest<float>(TEXT("VectorStoreFloat3"), TestVectorsEqual(V0, V1));

	SetScratch(1.0f, 2.0f, -0.25f, -0.5f, 5.f);
	V0 = MakeVectorRegister(5.0f, 3.0f, 1.0f, -1.0f);
	VectorStoreFloat1(V0, GScratch + 1);
	V1 = VectorLoad(GScratch);
	V0 = MakeVectorRegister(1.0f, 5.0f, -0.25f, -0.5f);
	LogTest<float>(TEXT("VectorStoreFloat1"), TestVectorsEqual(V0, V1));

	// Replicate
	TestVectorReplicate<float, VectorRegister4Float>();

	// Swizzle
	TestVectorSwizzle<float, VectorRegister4Float>();

	// Shuffle
	TestVectorShuffle<float, VectorRegister4Float>();

	V0 = MakeVectorRegisterFloat(0.f, 0.f, 2.f, 0.f);
	V1 = MakeVectorRegisterFloat(0.f, 1.f, 0.f, 3.f);
	Float0 = VectorGetComponent(V0, 0);
	Float1 = VectorGetComponent(V1, 1);
	Float2 = VectorGetComponent(V0, 2);
	Float3 = VectorGetComponent(V1, 3);
	V0 = MakeVectorRegisterFloat(0.f, 1.f, 2.f, 3.f);
	V1 = MakeVectorRegisterFloat(Float0, Float1, Float2, Float3);
	LogTest<float>(TEXT("VectorGetComponent<float>"), TestVectorsEqual(V0, V1));

	//VectorIntMultiply
	VI0 = MakeVectorRegisterInt(1, 2, -3, 4);
	VI1 = MakeVectorRegisterInt(2, -4, -6, -8);
	VI1 = VectorIntMultiply(VI0, VI1);
	VI0 = MakeVectorRegisterInt(2, -8, 18, -32);
	LogTest<int32>(TEXT("VectorIntMultiply"), TestVectorsEqualBitwise(VectorCastIntToFloat(VI0), VectorCastIntToFloat(VI1)));

	V0 = MakeVectorRegister(1.0f, -2.0f, 3.0f, -4.0f);
	V1 = VectorAbs(V0);
	V0 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 4.0f);
	LogTest<float>(TEXT("VectorAbs"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(1.0f, -2.0f, 3.0f, -4.0f);
	V1 = VectorNegate(V0);
	V0 = MakeVectorRegister(-1.0f, 2.0f, -3.0f, 4.0f);
	LogTest<float>(TEXT("VectorNegate"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 4.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = VectorAdd(V0, V1);
	V0 = MakeVectorRegister(3.0f, 6.0f, 9.0f, 12.0f);
	LogTest<float>(TEXT("VectorAdd"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 4.0f);
	V1 = VectorSubtract(V0, V1);
	V0 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 4.0f);
	LogTest<float>(TEXT("VectorSubtract"), TestVectorsEqual(V0, V1));

	// Note: Older versions of MSVC had codegen issues with optimization on that caused this to fail. If this passes in Debug builds but not optimized builds, try updating your MSVC compiler.
	V0 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 4.0f);
	V2 = VectorMultiply(V0, V1);
	V3 = MakeVectorRegister(2.0f, 8.0f, 18.0f, 32.0f);
	LogTest<float>(TEXT("VectorMultiply"), TestVectorsEqual(V3, V2));

	// Note: Older versions of MSVC had codegen issues with optimization on that caused this to fail. If this passes in Debug builds but not optimized builds, try updating your MSVC compiler.
	V0 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 4.0f);
	V2 = VectorMultiplyAdd(V0, V1, VectorOne());
	V3 = MakeVectorRegister(3.0f, 9.0f, 19.0f, 33.0f);
	LogTest<float>(TEXT("VectorMultiplyAdd"), TestVectorsEqual(V3, V2));

	V0 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 4.0f);
	V1 = VectorDot3(V0, V1);
	V0 = MakeVectorRegister(28.0f, 28.0f, 28.0f, 28.0f);
	LogTest<float>(TEXT("VectorDot3"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 5.0f);
	V1 = MakeVectorRegister(7.0f, 11.0f, 13.0f, 17.0f);
	V1 = VectorDot3(V0, V1);
	V0 = MakeVectorRegister(68.0f, 68.0f, 68.0f, 68.0f);
	LogTest<float>(TEXT("VectorDot3"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 4.0f);
	Float0 = VectorDot3Scalar(V0, V1);
	V1 = VectorSetFloat1(Float0);
	V0 = MakeVectorRegister(28.0f, 28.0f, 28.0f, 28.0f);
	LogTest<float>(TEXT("VectorDot3Scalar"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 5.0f);
	V1 = MakeVectorRegister(7.0f, 11.0f, 13.0f, 17.0f);
	Float0 = VectorDot3Scalar(V0, V1);
	V1 = VectorSetFloat1(Float0);
	V0 = MakeVectorRegister(68.0f, 68.0f, 68.0f, 68.0f);
	LogTest<float>(TEXT("VectorDot3Scalar"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 4.0f);
	V1 = VectorDot4(V0, V1);
	V0 = MakeVectorRegister(60.0f, 60.0f, 60.0f, 60.0f);
	LogTest<float>(TEXT("VectorDot4"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 5.0f);
	V1 = MakeVectorRegister(7.0f, 11.0f, 13.0f, 17.0f);
	V1 = VectorDot4(V0, V1);
	V0 = MakeVectorRegister(153.0f, 153.0f, 153.0f, 153.0f);
	LogTest<float>(TEXT("VectorDot4"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(1.0f, 0.0f, 0.0f, 8.0f);
	V1 = MakeVectorRegister(0.0f, 2.0f, 0.0f, 4.0f);
	V1 = VectorCross(V0, V1);
	V0 = MakeVectorRegister(0.f, 0.0f, 2.0f, 0.0f);
	LogTest<float>(TEXT("VectorCross"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(2.0f, 3.0f, 4.0f, 8.0f);
	V1 = MakeVectorRegister(5.0f, 6.0f, 7.0f, 4.0f);
	V1 = VectorCross(V0, V1);
	V0 = MakeVectorRegister(-3.0f, 6.0f, -3.0f, 0.0f);
	LogTest<float>(TEXT("VectorCross"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegister(4.0f, 3.0f, 2.0f, 1.0f);
	V1 = VectorPow(V0, V1);
	V0 = MakeVectorRegister(16.0f, 64.0f, 36.0f, 8.0f);
	LogTest<float>(TEXT("VectorPow"), TestVectorsEqual(V0, V1, 0.001f));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = VectorReciprocalLenEstimate(V0);
	V0 = MakeVectorRegister(0.25f, 0.25f, 0.25f, 0.25f);
	LogTest<float>(TEXT("VectorReciprocalLenEstimate"), TestVectorsEqual(V0, V1, 0.004f));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = VectorReciprocalLen(V0);
	V0 = MakeVectorRegister(0.25f, 0.25f, 0.25f, 0.25f);
	LogTest<float>(TEXT("VectorReciprocalLenAccurate"), TestVectorsEqual(V0, V1, 0.0001f));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = VectorNormalizeEstimate(V0);
	V0 = MakeVectorRegister(0.5f, -0.5f, 0.5f, -0.5f);
	LogTest<float>(TEXT("VectorNormalizeEstimate"), TestVectorsEqual(V0, V1, 0.004f));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = VectorNormalize(V0);
	V0 = MakeVectorRegister(0.5f, -0.5f, 0.5f, -0.5f);
	LogTest<float>(TEXT("VectorNormalizeAccurate"), TestVectorsEqual(V0, V1, 1e-8f));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = TestVectorNormalize_Sqrt(V0);
	V0 = MakeVectorRegister(0.5f, -0.5f, 0.5f, -0.5f);
	LogTest<float>(TEXT("TestVectorNormalize_Sqrt"), TestVectorsEqual(V0, V1, 1e-8f));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = TestVectorNormalize_InvSqrt(V0);
	V0 = MakeVectorRegister(0.5f, -0.5f, 0.5f, -0.5f);
	LogTest<float>(TEXT("TestVectorNormalize_InvSqrt"), TestVectorsEqual(V0, V1, 1e-8f));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = TestVectorNormalize_InvSqrtEst(V0);
	V0 = MakeVectorRegister(0.5f, -0.5f, 0.5f, -0.5f);
	LogTest<float>(TEXT("TestVectorNormalize_InvSqrtEst"), TestVectorsEqual(V0, V1, 1e-6f));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = VectorSet_W0(V0);
	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, 0.0f);
	LogTest<float>(TEXT("VectorSet_W0"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, -2.0f);
	V1 = VectorSet_W1(V0);
	V0 = MakeVectorRegister(2.0f, -2.0f, 2.0f, 1.0f);
	LogTest<float>(TEXT("VectorSet_W1"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegister(4.0f, 3.0f, 2.0f, 1.0f);
	V1 = VectorMin(V0, V1);
	V0 = MakeVectorRegister(2.0f, 3.0f, 2.0f, 1.0f);
	LogTest<float>(TEXT("VectorMin"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegister(4.0f, 3.0f, 2.0f, 1.0f);
	V1 = VectorMax(V0, V1);
	V0 = MakeVectorRegister(4.0f, 4.0f, 6.0f, 8.0f);
	LogTest<float>(TEXT("VectorMax"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(-2.0f, 3.0f, -4.0f, 5.0f);
	V1 = MakeVectorRegister(4.0f, 4.0f, 6.0f, 8.0f);
	V2 = MakeVectorRegister(-8.0f, 5.0f, -1.0f, 1.0f);
	V2 = VectorClamp(V2, V0, V1);
	V3 = MakeVectorRegister(-2.0f, 4.0f, -1.0f, 5.0f);
	LogTest<float>(TEXT("VectorClamp"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(1.0f, 3.0f, 5.0f, 7.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = VectorCombineHigh(V0, V1);
	V0 = MakeVectorRegister(5.0f, 7.0f, 6.0f, 8.0f);
	LogTest<float>(TEXT("VectorCombineHigh"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(1.0f, 3.0f, 5.0f, 7.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = VectorCombineLow(V0, V1);
	V0 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 4.0f);
	LogTest<float>(TEXT("VectorCombineLow"), TestVectorsEqual(V0, V1));

	// LoadByte4 and StoreByte4
	uint8 Bytes[4] = { 25, 75, 125, 200 };
	V0 = VectorLoadByte4(Bytes);
	V1 = MakeVectorRegister(25.f, 75.f, 125.f, 200.f);
	LogTest<float>(TEXT("VectorLoadByte4"), TestVectorsEqual(V0, V1));

	V0 = VectorLoadByte4Reverse(Bytes);
	V1 = MakeVectorRegister(25.f, 75.f, 125.f, 200.f);
	V1 = VectorSwizzle(V1, 3, 2, 1, 0);
	LogTest<float>(TEXT("VectorLoadByte4Reverse"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(4.0f, 3.0f, 2.0f, 1.0f);
	VectorStoreByte4(V0, Bytes);
	V1 = VectorLoadByte4(Bytes);
	LogTest<float>(TEXT("VectorStoreByte4"), TestVectorsEqual(V0, V1));

	// LoadSignedByte4 and StoreSignedByte4
	int8 SignedBytes[4] = { 25, -75, 125, -50 };
	V0 = VectorLoadSignedByte4(SignedBytes);
	V1 = MakeVectorRegister(25.f, -75.f, 125.f, -50.f);
	LogTest<float>(TEXT("VectorLoadSignedByte4"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(-4.0f, 3.0f, -2.0f, 1.0f);
	VectorStoreSignedByte4(V0, SignedBytes);
	V1 = VectorLoadSignedByte4(SignedBytes);
	LogTest<float>(TEXT("VectorStoreSignedByte4"), TestVectorsEqual(V0, V1));

	// LoadURGBA16N and StoreURGBA16N
	// NOTE: StoreURGBA16N is _not_ symmetric with Load. Load just gives you the
	// values as floats in [0,65535], but Load expects the values as floats in
	// [0,1], clamps to that range first, then scales to write ints.
	uint16 UInt16s[4] = { 1234, 2345, 3456, 64000 };
	V0 = VectorLoadURGBA16N(UInt16s);
	V1 = MakeVectorRegister(1234.f, 2345.f, 3456.f, 64000.f);
	LogTest<float>(TEXT("VectorLoadURGBA16N"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(-200000.f, 123.f / 65535.f, 234.f / 65535.f, 1.5f);
	VectorStoreURGBA16N(V0, UInt16s);
	V0 = MakeVectorRegister(0.f, 123.f, 234.f, 65535.f);
	V1 = VectorLoadURGBA16N(UInt16s);
	LogTest<float>(TEXT("VectorStoreURGBA16N"), TestVectorsEqual(V0, V1));

	// LoadSRGBA16N
	int16 Int16s[4] = { 1234, -2345, 3456, -32500 };
	V0 = VectorLoadSRGBA16N(Int16s);
	V1 = MakeVectorRegister(1234.f, -2345.f, 3456.f, -32500.f);
	LogTest<float>(TEXT("VectorLoadSRGBA16N"), TestVectorsEqual(V0, V1));

	// Vector Any/All comparisons
	V0 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V1 = MakeVectorRegister(4.0f, 3.0f, 2.0f, 1.0f);
	bool bIsVAGT_TRUE = VectorAnyGreaterThan(V0, V1) != 0;
	LogTest<float>(TEXT("VectorAnyGreaterThan-true"), bIsVAGT_TRUE);

	V0 = MakeVectorRegister(0.0f, -0.0f, 1.0f, 0.8f);
	V1 = MakeVectorRegister(0.0f, 0.0f, 0.0f, 1.0f);
	bIsVAGT_TRUE = VectorAnyGreaterThan(V0, V1) != 0;
	LogTest<float>(TEXT("VectorAnyGreaterThan-true"), bIsVAGT_TRUE);

	V0 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 1.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	bool bIsVAGT_FALSE = VectorAnyGreaterThan(V0, V1) == 0;
	LogTest<float>(TEXT("VectorAnyGreaterThan-false"), bIsVAGT_FALSE);

	V0 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 1.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	LogTest<float>(TEXT("VectorAnyLesserThan-true"), VectorAnyLesserThan(V0, V1) != 0);

	V0 = MakeVectorRegister(3.0f, 5.0f, 7.0f, 9.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	LogTest<float>(TEXT("VectorAnyLesserThan-false"), VectorAnyLesserThan(V0, V1) == 0);

	V0 = MakeVectorRegister(3.0f, 5.0f, 7.0f, 9.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	LogTest<float>(TEXT("VectorAllGreaterThan-true"), VectorAllGreaterThan(V0, V1) != 0);

	V0 = MakeVectorRegister(3.0f, 1.0f, 7.0f, 9.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	LogTest<float>(TEXT("VectorAllGreaterThan-false"), VectorAllGreaterThan(V0, V1) == 0);

	V0 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 1.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	LogTest<float>(TEXT("VectorAllLesserThan-true"), VectorAllLesserThan(V0, V1) != 0);

	V0 = MakeVectorRegister(3.0f, 3.0f, 2.0f, 1.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	LogTest<float>(TEXT("VectorAllLesserThan-false"), VectorAllLesserThan(V0, V1) == 0);

	// Vector component comparisons

	// VectorCompareGT
	const VectorRegister4Float VectorZeroMaskFloat = MakeVectorRegisterFloatMask((uint32)0, (uint32)0, (uint32)0, (uint32)0);
	V0 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 8.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 2.0f, 1.0f);
	V2 = VectorCompareGT(V0, V1);
	V3 = MakeVectorRegister((uint32)0, (uint32)0, (uint32)0, (uint32)-1);
	LogTest<float>(TEXT("VectorCompareGT"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareGT(V0, VectorNaN);
	V3 = VectorZeroMaskFloat;
	LogTest<float>(TEXT("VectorCompareGT (NaN)"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareGT(VectorNaN, V0);
	LogTest<float>(TEXT("VectorCompareGT (NaN)"), TestVectorsEqualBitwise(V2, V3));

	// VectorCompareGE
	V0 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 8.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 2.0f, 1.0f);
	V2 = VectorCompareGE(V0, V1);
	V3 = MakeVectorRegister((uint32)0, (uint32)0, (uint32)-1, (uint32)-1);
	LogTest<float>(TEXT("VectorCompareGE"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareGE(V0, VectorNaN);
	V3 = VectorZeroMaskFloat;
	LogTest<float>(TEXT("VectorCompareGE (NaN)"), TestVectorsEqualBitwise(V2, V3));

	// VectorCompareLT
	V0 = MakeVectorRegister(2.0f, 4.0f, 2.0f, 1.0f);
	V1 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 8.0f);
	V2 = VectorCompareLT(V0, V1);
	V3 = MakeVectorRegister((uint32)0, (uint32)0, (uint32)0, (uint32)-1);
	LogTest<float>(TEXT("VectorCompareLT"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareLT(V0, VectorNaN);
	V3 = VectorZeroMaskFloat;
	LogTest<float>(TEXT("VectorCompareLT (NaN)"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareLT(VectorNaN, V0);
	LogTest<float>(TEXT("VectorCompareLT (NaN)"), TestVectorsEqualBitwise(V2, V3));

	// VectorCompareLE
	V0 = MakeVectorRegister(2.0f, 4.0f, 2.0f, 1.0f);
	V1 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 8.0f);
	V2 = VectorCompareLE(V0, V1);
	V3 = MakeVectorRegister((uint32)0, (uint32)0, (uint32)-1, (uint32)-1);
	LogTest<float>(TEXT("VectorCompareLE"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareLE(V0, VectorNaN);
	V3 = VectorZeroMaskFloat;
	LogTest<float>(TEXT("VectorCompareLE (NaN)"), TestVectorsEqualBitwise(V2, V3));

	// VectorCompareEQ
	V0 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 8.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 2.0f, 1.0f);
	V2 = VectorCompareEQ(V0, V1);
	V3 = MakeVectorRegister((uint32)0, (uint32)0, (uint32)-1, (uint32)0);
	LogTest<float>(TEXT("VectorCompareEQ"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareEQ(V0, VectorNaN);
	V3 = VectorZeroMaskFloat;
	LogTest<float>(TEXT("VectorCompareEQ (NaN)"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareEQ(VectorNaN, V0);
	LogTest<float>(TEXT("VectorCompareEQ (NaN)"), TestVectorsEqualBitwise(V2, V3));
	// NaN:NaN comparisons are undefined according to the Intel instruction manual, and will vary in optimized versus debug builds.
	/*
	V2 = VectorCompareEQ(VectorNaN, VectorNaN);
	V3 = VectorZeroMaskFloat;
	LogTest<float>(TEXT("VectorCompareEQ (NaN:NaN)"), TestVectorsEqualBitwise(V2, V3));
	*/

	// VectorCompareNE
	V0 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 8.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 2.0f, 1.0f);
	V2 = VectorCompareNE(V0, V1);
	V3 = MakeVectorRegister((uint32)(0xFFFFFFFFU), (uint32)(0xFFFFFFFFU), (uint32)(0), (uint32)(0xFFFFFFFFU));
	LogTest<float>(TEXT("VectorCompareNE"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareNE(V0, VectorNaN);
	V3 = GlobalVectorConstants::AllMask();
	LogTest<float>(TEXT("VectorCompareNE (NaN)"), TestVectorsEqualBitwise(V2, V3));
	V2 = VectorCompareNE(VectorNaN, V0);
	LogTest<float>(TEXT("VectorCompareNE (NaN)"), TestVectorsEqualBitwise(V2, V3));
	// NaN:NaN comparisons are undefined according to the Intel instruction manual, and will vary in optimized versus debug builds.
	/*
	V2 = VectorCompareNE(VectorNaN, VectorNaN);
	V3 = GlobalVectorConstants::AllMask;
	LogTest<float>(TEXT("VectorCompareNE (NaN:NaN)"), TestVectorsEqualBitwise(V2, V3));
	*/

	// VectorSelect
	V0 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 8.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 2.0f, 1.0f);
	V2 = MakeVectorRegister((uint32)-1, (uint32)0, (uint32)0, (uint32)-1);
	V2 = VectorSelect(V2, V0, V1);
	V3 = MakeVectorRegister(1.0f, 4.0f, 2.0f, 8.0f);
	LogTest<float>(TEXT("VectorSelect"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(1.0f, 3.0f, 5.0f, 7.0f);
	V1 = MakeVectorRegister(2.0f, 4.0f, 6.0f, 8.0f);
	V2 = MakeVectorRegister((uint32)0, (uint32)-1, (uint32)-1, (uint32)0);
	V2 = VectorSelect(V2, V0, V1);
	V3 = MakeVectorRegister(2.0f, 3.0f, 5.0f, 8.0f);
	LogTest<float>(TEXT("VectorSelect"), TestVectorsEqual(V2, V3));

	// Vector bitwise operations
	V0 = MakeVectorRegister(1.0f, 3.0f, 0.0f, 0.0f);
	V1 = MakeVectorRegister(0.0f, 0.0f, 2.0f, 1.0f);
	V2 = VectorBitwiseOr(V0, V1);
	V3 = MakeVectorRegister(1.0f, 3.0f, 2.0f, 1.0f);
	LogTest<float>(TEXT("VectorBitwiseOr-Float1"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(1.0f, 3.0f, 24.0f, 36.0f);
	V1 = MakeVectorRegister((uint32)(0x80000000U), (uint32)(0x80000000U), (uint32)(0x80000000U), (uint32)(0x80000000U));
	V2 = VectorBitwiseOr(V0, V1);
	V3 = MakeVectorRegister(-1.0f, -3.0f, -24.0f, -36.0f);
	LogTest<float>(TEXT("VectorBitwiseOr-Float2"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(-1.0f, -3.0f, -24.0f, 36.0f);
	V1 = MakeVectorRegister((uint32)(0xFFFFFFFFU), (uint32)(0x7FFFFFFFU), (uint32)(0x7FFFFFFFU), (uint32)(0xFFFFFFFFU));
	V2 = VectorBitwiseAnd(V0, V1);
	V3 = MakeVectorRegister(-1.0f, 3.0f, 24.0f, 36.0f);
	LogTest<float>(TEXT("VectorBitwiseAnd-Float"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(-1.0f, -3.0f, -24.0f, 36.0f);
	V1 = MakeVectorRegister((uint32)(0x80000000U), (uint32)(0x00000000U), (uint32)(0x80000000U), (uint32)(0x80000000U));
	V2 = VectorBitwiseXor(V0, V1);
	V3 = MakeVectorRegister(1.0f, -3.0f, 24.0f, -36.0f);
	LogTest<float>(TEXT("VectorBitwiseXor-Float"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(-1.0f, -3.0f, -24.0f, 36.0f);
	V1 = MakeVectorRegister(5.0f, 35.0f, 23.0f, 48.0f);
	V2 = VectorMergeVecXYZ_VecW(V0, V1);
	V3 = MakeVectorRegister(-1.0f, -3.0f, -24.0f, 48.0f);
	LogTest<float>(TEXT("VectorMergeXYZ_VecW-1"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(-1.0f, -3.0f, -24.0f, 36.0f);
	V1 = MakeVectorRegister(5.0f, 35.0f, 23.0f, 48.0f);
	V2 = VectorMergeVecXYZ_VecW(V1, V0);
	V3 = MakeVectorRegister(5.0f, 35.0f, 23.0f, 36.0f);
	LogTest<float>(TEXT("VectorMergeXYZ_VecW-2"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(1.0f, 1.0e6f, 1.3e-8f, 35.0f);
	V1 = VectorReciprocalEstimate(V0);
	V3 = VectorMultiply(V1, V0);
	LogTest<float>(TEXT("VectorReciprocalEstimate"), TestVectorsEqual(VectorOne(), V3, 0.008f));

	V0 = MakeVectorRegister(1.0f, 1.0e6f, 1.3e-8f, 35.0f);
	V1 = VectorReciprocal(V0);
	V3 = VectorMultiply(V1, V0);
	LogTest<float>(TEXT("VectorReciprocalAccurate"), TestVectorsEqual(VectorOne(), V3, 1e-7f));

	V0 = MakeVectorRegister(1.0f, 1.0e6f, 1.3e-8f, 35.0f);
	V1 = VectorReciprocalSqrtEstimate(V0);
	V3 = VectorMultiply(VectorMultiply(V1, V1), V0);
	LogTest<float>(TEXT("VectorReciprocalSqrtEstimate"), TestVectorsEqual(VectorOne(), V3, 0.007f));

	V0 = MakeVectorRegister(1.0f, 1.0e6f, 1.3e-8f, 35.0f);
	V1 = VectorReciprocalSqrt(V0);
	V3 = VectorMultiply(VectorMultiply(V1, V1), V0);
	LogTest<float>(TEXT("VectorReciprocalSqrtAccurate"), TestVectorsEqual(VectorOne(), V3, 1e-6f));

	SetScratch(1.0f, 2.0f, 3.0f, 4.0f);
	V0 = VectorLoadTwoPairsFloat(GScratch + 0, GScratch + 1);
	V1 = MakeVectorRegister(1.0f, 2.0f, 2.0f, 3.0f);
	LogTest<float>(TEXT("VectorLoadTwoPairsFloat"), TestVectorsEqual(V0, V1));

	V0 = MakeVectorRegister(0.0f, 1.0f, 2.0f, 3.0f);
	V1 = MakeVectorRegister(4.0f, 5.0f, 6.0f, 7.0f);
	VectorDeinterleave(V2, V3, V0, V1);
	V0 = MakeVectorRegister(0.0f, 2.0f, 4.0f, 6.0f);
	V1 = MakeVectorRegister(1.0f, 3.0f, 5.0f, 7.0f);
	LogTest<float>(TEXT("VectorDeinterleave"), TestVectorsEqual(V2, V0) && TestVectorsEqual(V3, V1));

	// VectorMod
	V0 = MakeVectorRegister(0.0f, 3.2f, 2.8f, 1.5f);
	V1 = MakeVectorRegister(2.0f, 1.2f, 2.0f, 3.0f);
	V2 = TestReferenceMod(V0, V1);
	V3 = VectorMod(V0, V1);
	LogTest<float>(TEXT("VectorMod positive"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(-2.0f, 3.2f, -2.8f, -1.5f);
	V1 = MakeVectorRegister(-1.5f, -1.2f, 2.0f, 3.0f);
	V2 = TestReferenceMod(V0, V1);
	V3 = VectorMod(V0, V1);
	LogTest<float>(TEXT("VectorMod negative"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(89.9f, 180.0f, -256.0f, -270.1f);
	V1 = MakeVectorRegister(360.0f, 0.1f, 360.0f, 180.0f);
	V2 = TestReferenceMod(V0, V1);
	V3 = VectorMod(V0, V1);
	LogTest<float>(TEXT("VectorMod common"), TestVectorsEqual(V2, V3));

	// VectorMod360
	V0 = MakeVectorRegister(89.9f, 180.0f, -256.0f, -270.1f);
	V1 = GlobalVectorConstants::Float360;
	V2 = TestReferenceMod(V0, V1);
	V3 = VectorMod360(V0);
	LogTest<float>(TEXT("VectorMod360"), TestVectorsEqual(V2, V3));

	V0 = MakeVectorRegister(0.0f, -1079.9f, -1720.1f, 12345.12345f);
	V1 = GlobalVectorConstants::Float360;
	V2 = TestReferenceMod(V0, V1);
	V3 = VectorMod360(V0);
	LogTest<float>(TEXT("VectorMod360"), TestVectorsEqual(V2, V3));

#if UE_BUILD_DEBUG
	V1 = GlobalVectorConstants::Float360;
	float RotStep = 1.01f;
	for (float F = -720.f; F <= 720.f; F += RotStep)
	{
		V0 = MakeVectorRegister(F, F + 0.001f, F + 0.025f, F + 0.6125f);
		V2 = TestReferenceMod(V0, V1);
		V3 = VectorMod360(V0);
		LogTest<float>(TEXT("VectorMod360"), TestVectorsEqual(V2, V3));
	}
#endif // UE_BUILD_DEBUG

	// VectorSign
	V0 = MakeVectorRegister(2.0f, -2.0f, 0.0f, -3.0f);
	V2 = MakeVectorRegister(1.0f, -1.0f, 1.0f, -1.0f);
	V3 = VectorSign(V0);
	LogTest<float>(TEXT("VectorSign"), TestVectorsEqual(V2, V3));

	// VectorStep
	V0 = MakeVectorRegister(2.0f, -2.0f, 0.0f, -3.0f);
	V2 = MakeVectorRegister(1.0f, 0.0f, 1.0f, 0.0f);
	V3 = VectorStep(V0);
	LogTest<float>(TEXT("VectorStep"), TestVectorsEqual(V2, V3));

	// VectorTruncate
	V0 = MakeVectorRegister(-1.8f, -1.0f, -0.8f, 0.0f);
	V2 = MakeVectorRegister(-1.0f, -1.0f, 0.0f, 0.0f);
	V3 = VectorTruncate(V0);
	LogTest<float>(TEXT("VectorTruncate"), TestVectorsEqual(V2, V3, UE_KINDA_SMALL_NUMBER));

	V0 = MakeVectorRegister(0.0f, 0.8f, 1.0f, 1.8f);
	V2 = MakeVectorRegister(0.0f, 0.0f, 1.0f, 1.0f);
	V3 = VectorTruncate(V0);
	LogTest<float>(TEXT("VectorTruncate"), TestVectorsEqual(V2, V3, UE_KINDA_SMALL_NUMBER));

	// VectorFractional
	V0 = MakeVectorRegister(-1.8f, -1.0f, -0.8f, 0.0f);
	V2 = MakeVectorRegister(-0.8f, 0.0f, -0.8f, 0.0f);
	V3 = VectorFractional(V0);
	LogTest<float>(TEXT("VectorFractional"), TestVectorsEqual(V2, V3, UE_KINDA_SMALL_NUMBER));

	V0 = MakeVectorRegister(0.0f, 0.8f, 1.0f, 1.8f);
	V2 = MakeVectorRegister(0.0f, 0.8f, 0.0f, 0.8f);
	V3 = VectorFractional(V0);
	LogTest<float>(TEXT("VectorFractional"), TestVectorsEqual(V2, V3, UE_KINDA_SMALL_NUMBER));

	// VectorCeil
	V0 = MakeVectorRegister(-1.8f, -1.0f, -0.8f, 0.0f);
	V2 = MakeVectorRegister(-1.0f, -1.0f, -0.0f, 0.0f);
	V3 = VectorCeil(V0);
	LogTest<float>(TEXT("VectorCeil"), TestVectorsEqual(V2, V3, UE_KINDA_SMALL_NUMBER));

	V0 = MakeVectorRegister(0.0f, 0.8f, 1.0f, 1.8f);
	V2 = MakeVectorRegister(0.0f, 1.0f, 1.0f, 2.0f);
	V3 = VectorCeil(V0);
	LogTest<float>(TEXT("VectorCeil"), TestVectorsEqual(V2, V3, UE_KINDA_SMALL_NUMBER));

	// VectorFloor
	V0 = MakeVectorRegister(-1.8f, -1.0f, -0.8f, 0.0f);
	V2 = MakeVectorRegister(-2.0f, -1.0f, -1.0f, 0.0f);
	V3 = VectorFloor(V0);
	LogTest<float>(TEXT("VectorFloor"), TestVectorsEqual(V2, V3, UE_KINDA_SMALL_NUMBER));

	V0 = MakeVectorRegister(0.0f, 0.8f, 1.0f, 1.8f);
	V2 = MakeVectorRegister(0.0f, 0.0f, 1.0f, 1.0f);
	V3 = VectorFloor(V0);
	LogTest<float>(TEXT("VectorFloor"), TestVectorsEqual(V2, V3));

	// VectorMaskBits
	V0 = MakeVectorRegister(1.0f, 2.0f, 3.0f, 4.0f);
	uint32 MaskBits = VectorMaskBits(V0);
	LogTest<float>(TEXT("VectorMaskBits"), MaskBits == 0);
	V0 = MakeVectorRegister(-1.0f, -2.0f, -3.0f, -4.0f);
	MaskBits = VectorMaskBits(V0);
	LogTest<float>(TEXT("VectorMaskBits"), MaskBits == 0xf);
	V0 = MakeVectorRegister(-1.0f, 2.0f, -3.0f, 4.0f);
	MaskBits = VectorMaskBits(V0);
	LogTest<float>(TEXT("VectorMaskBits"), MaskBits == 5);

	// VectorIntSelect
	VI0 = MakeVectorRegisterInt(1, 3, 2, 8);
	VI1 = MakeVectorRegisterInt(2, 4, 2, 1);
	VI2 = MakeVectorRegisterInt(-1, 0, 0, -1);
	VI2 = VectorIntSelect(VI2, VI0, VI1);
	VI3 = MakeVectorRegisterInt(1, 4, 2, 8);
	LogTest<int32>(TEXT("VectorIntSelect"), TestVectorsEqualBitwise(VectorCastIntToFloat(VI2), VectorCastIntToFloat(VI3)));

	VI0 = MakeVectorRegisterInt(1, 3, 5, 7);
	VI1 = MakeVectorRegisterInt(2, 4, 6, 8);
	VI2 = MakeVectorRegisterInt(0, -1, -1, 0);
	VI2 = VectorIntSelect(VI2, VI0, VI1);
	VI3 = MakeVectorRegisterInt(2, 3, 5, 8);
	LogTest<int32>(TEXT("VectorIntSelect"), TestVectorsEqualBitwise(VectorCastIntToFloat(VI2), VectorCastIntToFloat(VI3)));

	// VectorIntMin/VectorIntMax
	VI0 = MakeVectorRegisterInt(10, 12, 30, 22);
	VI1 = MakeVectorRegisterInt(5, 14, 32, -100);
	VI2 = VectorIntMin(VI0, VI1);
	VI3 = MakeVectorRegisterInt(5, 12, 30, -100);
	LogTest<int32>(TEXT("VectorIntMin"), TestVectorsEqualBitwise(VectorCastIntToFloat(VI2), VectorCastIntToFloat(VI3)));

	VI2 = VectorIntMax(VI0, VI1);
	VI3 = MakeVectorRegisterInt(10, 14, 32, 22);
	LogTest<int32>(TEXT("VectorIntMax"), TestVectorsEqualBitwise(VectorCastIntToFloat(VI2), VectorCastIntToFloat(VI3)));

	// VectorIntAbs
	VI0 = MakeVectorRegisterInt(1, -2, 3, -4);
	VI2 = VectorIntAbs(VI0);
	VI3 = MakeVectorRegisterInt(1, 2, 3, 4);
	LogTest<int32>(TEXT("VectorIntAbs"), TestVectorsEqualBitwise(VectorCastIntToFloat(VI2), VectorCastIntToFloat(VI3)));

	// Matrix multiplications and transformations
	{
		FMatrix44f	M0, M1, M2, M3;
		FVector3f Eye, LookAt, Up;
		// Create Look at Matrix
		Eye = FVector3f(1024.0f, -512.0f, -2048.0f);
		LookAt = FVector3f(0.0f, 0.0f, 0.0f);
		Up = FVector3f(0.0f, 1.0f, 0.0f);
		M0 = FLookAtMatrix44f(Eye, LookAt, Up);

		// Create GL ortho projection matrix
		const float Width = 1920.0f;
		const float Height = 1080.0f;
		const float Left = 0.0f;
		const float Right = Left + Width;
		const float Top = 0.0f;
		const float Bottom = Top + Height;
		const float ZNear = -100.0f;
		const float ZFar = 100.0f;

		M1 = FMatrix44f(FPlane4f(2.0f / (Right - Left), 0, 0, 0),
			FPlane4f(0, 2.0f / (Top - Bottom), 0, 0),
			FPlane4f(0, 0, 1 / (ZNear - ZFar), 0),
			FPlane4f((Left + Right) / (Left - Right), (Top + Bottom) / (Bottom - Top), ZNear / (ZNear - ZFar), 1));

		VectorMatrixMultiply(&M2, &M0, &M1);
		TestVectorMatrixMultiply(&M3, &M0, &M1);
		LogTest<float>(TEXT("VectorMatrixMultiply"), TestMatricesEqual(M2, M3, 0.000001f));

		VectorMatrixInverse(&M2, &M1);
		TestVectorMatrixInverse(&M3, &M1);
		LogTest<float>(TEXT("VectorMatrixInverse"), TestMatricesEqual(M2, M3, 0.000001f));

		// 	FTransform Transform;
		// 	Transform.SetFromMatrix(M1);
		// 	FTransform InvTransform = Transform.Inverse();
		// 	FTransform InvTransform2 = FTransform(Transform.ToMatrixWithScale().Inverse());
		// 	LogTest<float>( TEXT("FTransform Inverse"), InvTransform.Equals(InvTransform2, 1e-3f ) );

		V0 = MakeVectorRegister(100.0f, -100.0f, 200.0f, 1.0f);
		V1 = VectorTransformVector(V0, &M0);
		V2 = TestVectorTransformVector(V0, &M0);
		LogTest<float>(TEXT("VectorTransformVector"), TestVectorsEqual(V1, V2, 1e-8f));

		V0 = MakeVectorRegister(32768.0f, 131072.0f, -8096.0f, 1.0f);
		V1 = VectorTransformVector(V0, &M1);
		V2 = TestVectorTransformVector(V0, &M1);
		LogTest<float>(TEXT("VectorTransformVector"), TestVectorsEqual(V1, V2, 1e-8f));
	}

	// FTransform
	TestTransformFunctions<FTransform3f>();

	// NaN / Inf tests
	SetScratch(0.0f, 0.0f, 0.0f, 0.0f);
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(MakeVectorRegister(NaN, NaN, NaN, NaN)));
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(MakeVectorRegister(NaN, 0.f, 0.f, 0.f)));
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(MakeVectorRegister(0.f, 0.f, 0.f, NaN)));
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(GlobalVectorConstants::FloatInfinity()));
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(MakeVectorRegister((uint32)0xFF800000, (uint32)0xFF800000, (uint32)0xFF800000, (uint32)0xFF800000))); // negative infinity
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite true"), VectorContainsNaNOrInfinite(GlobalVectorConstants::AllMask()));

	// Not Nan/Inf
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(GlobalVectorConstants::FloatZero));
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(GlobalVectorConstants::FloatOne));
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(GlobalVectorConstants::FloatMinusOneHalf));
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(GlobalVectorConstants::SmallNumber));
	LogTest<float>(TEXT("VectorContainsNaNOrInfinite false"), !VectorContainsNaNOrInfinite(GlobalVectorConstants::BigNumber));

	// FMath::FMod (floats)
	{
		struct XYPair
		{
			float X, Y;
		};

		static XYPair XYArray[] =
		{
			// Test normal ranges
			{ 0.0f,	 1.0f},
			{ 1.5f,	 1.0f},
			{ 2.8f,	 0.3f},
			{-2.8f,	 0.3f},
			{ 2.8f,	-0.3f},
			{-2.8f,	-0.3f},
			{-0.4f,	 5.5f},
			{ 0.4f,	-5.5f},
			{ 2.8f,	 2.0f + UE_KINDA_SMALL_NUMBER},
			{-2.8f,	 2.0f - UE_KINDA_SMALL_NUMBER},

			// Analytically should be zero but floating point precision can cause results close to Y (or erroneously negative) depending on the method used.
			{55.8f,	 9.3f},
			{1234.1234f, 0.1234f},

			// Commonly used for FRotators and angles
			{725.2f,	360.0f},
			{179.9f,	90.0f},
			{ 5.3f * UE_PI,	2.f * UE_PI},
			{-5.3f * UE_PI,	2.f * UE_PI},

			// Test extreme ranges
			{ 1.0f,				 UE_KINDA_SMALL_NUMBER},
			{ 1.0f,				-UE_KINDA_SMALL_NUMBER},
			{-UE_SMALL_NUMBER,   UE_SMALL_NUMBER},
			{ UE_SMALL_NUMBER,  -UE_SMALL_NUMBER},
			{ 1.0f,				 MIN_flt},
			{ 1.0f,				-MIN_flt},
			{ MAX_flt,			 MIN_flt},
			{ MAX_flt,			-MIN_flt},

			// Test some tricky cases
			{8388808.00f, 178.222f},  // FAILS old version
			{360.0f * 1e10f,		360.0f},
			{-720.0f * 1000000.0f,	360.0f},
			{1080.0f * 12345.f,		360.0f},
			{360.0f,				360.0f - UE_KINDA_SMALL_NUMBER},
			{1080.0f,				360.0f - UE_KINDA_SMALL_NUMBER},
			{719.0f,				360.0f - UE_KINDA_SMALL_NUMBER},


			// We define this to be zero and not NaN.
			// Disabled since we don't want to trigger an ensure, but left here for testing that logic.
			//{ 1.0,	 0.0}, 
			//{ 1.0,	-0.0},
		};

		for (auto XY : XYArray)
		{
			const float X = XY.X;
			const float Y = XY.Y;
			const float Ours = FMath::Fmod(X, Y);
			const float Theirs = fmodf(X, Y);
			const float Delta = FMath::Abs(Ours - Theirs);

			//UE_LOGF(LogUnrealMathTest, Warning, "fmodf(%.12f, %.12f) Ref: %.12f Ours: %.12f (delta %.12f)", X, Y, Theirs, Ours, Delta);

			// A compiler bug causes stock fmodf() to rarely return NaN for valid input, we don't want to report this as a fatal error.
			if (Y != 0.0f && FMath::IsNaN(Theirs))
			{
				UE_LOGF(LogUnrealMathTest, Warning, "fmodf(%f, %f) with valid input resulted in NaN!", X, Y);
				continue;
			}

			if (Delta > 1e-8f)
			{
				UE_LOGF(LogUnrealMathTest, Log, "FMath::Fmod(%f, %f)=%f <-> fmodf(%f, %f)=%f: FAILED", X, Y, Ours, X, Y, Theirs);
				CheckPassing(false);
			}
		}
	}

	// FMath::FMod (doubles)
	{
		struct XYPair
		{
			double X, Y;
		};

		static XYPair XYArray[] =
		{
			// Test normal ranges
			{ 0.0,	 1.0},
			{ 1.5,	 1.0},
			{ 2.8,	 0.3},
			{-2.8,	 0.3},
			{ 2.8,	-0.3},
			{-2.8,	-0.3},
			{-0.4,	 5.5},
			{ 0.4,	-5.5},
			{ 2.8,	 2.0 + UE_DOUBLE_KINDA_SMALL_NUMBER},
			{-2.8,	 2.0 - UE_DOUBLE_KINDA_SMALL_NUMBER},

			// Analytically should be zero but floating point precision can cause results close to Y (or erroneously negative) depending on the method used.
			{55.8,	 9.3},
			{1234.1234, 0.1234},
			{12341234.12341234, 0.12341234},

			// Commonly used for FRotators and angles
			{725.2,	360.0},
			{179.9,	90.0},
			{ 5.3 * UE_DOUBLE_PI,	2. * UE_DOUBLE_PI},
			{-5.3 * UE_DOUBLE_PI,	2. * UE_DOUBLE_PI},

			// Test extreme ranges
			{ 1.0,			 UE_DOUBLE_KINDA_SMALL_NUMBER},
			{ 1.0,			-UE_DOUBLE_KINDA_SMALL_NUMBER},
			{-UE_DOUBLE_SMALL_NUMBER,  UE_DOUBLE_SMALL_NUMBER},
			{ UE_DOUBLE_SMALL_NUMBER, -UE_DOUBLE_SMALL_NUMBER},
			{ 1.0,			 MIN_dbl},
			{ 1.0,			-MIN_dbl},
			{ MAX_flt,		 MIN_dbl},
			{ MAX_flt,		-MIN_dbl},

			// Test some tricky cases
			{8388808.00,			178.222},  // FAILS old version
			{360.0 * 1e10,			360.0},
			{-720.0 * 1000000.0,	360.0},
			{1080.0 * 12345.,		360.0},
			{360.0,					360.0 - UE_DOUBLE_KINDA_SMALL_NUMBER},
			{1080.0,				360.0 - UE_DOUBLE_KINDA_SMALL_NUMBER},
			{719.0,					360.0 - UE_DOUBLE_KINDA_SMALL_NUMBER},

			// We define this to be zero and not NaN.
			// Disabled since we don't want to trigger an ensure, but left here for testing that logic.
			//{ 1.0,	 0.0}, 
			//{ 1.0,	-0.0},
		};

		for (auto XY : XYArray)
		{
			const double X = XY.X;
			const double Y = XY.Y;
			const double Ours = FMath::Fmod(X, Y);
			const double Theirs = fmod(X, Y);
			const double Delta = FMath::Abs(Ours - Theirs);

			//UE_LOGF(LogUnrealMathTest, Warning, "fmod(%.12f, %.12f) Ref: %.12f Ours: %.12f (delta %.12f)", X, Y, Theirs, Ours, Delta);

			// A compiler bug causes stock fmodf() to rarely return NaN for valid input, we don't want to report this as a fatal error.
			if (Y != 0.0 && FMath::IsNaN(Theirs))
			{
				UE_LOGF(LogUnrealMathTest, Warning, "fmod(%f, %f) with valid input resulted in NaN!", X, Y);
				continue;
			}

			if (Delta > 1e-8)
			{
				// If we differ significantly, that is likely due to rounding and the difference should be nearly equal to Y.
				const double FractionalDelta = FMath::Abs(Delta - FMath::Abs(Y));
				if (FractionalDelta > 1e-8)
				{
					UE_LOGF(LogUnrealMathTest, Log, "FMath::Fmod(%.12f, %.12f)=%.12f <-> fmod(%.12f, %.12f)=%.12f: FAILED", X, Y, Ours, X, Y, Theirs);
					CheckPassing(false);
				}
			}
		}
	}


	// Float function compilation with various types. Set this define to 1 to verify that warnings are generated for all code within MATHTEST_CHECK_INVALID_OVERLOAD_VARIANTS blocks.
#define MATHTEST_CHECK_INVALID_OVERLOAD_VARIANTS 0
	{
		float F = 1.f, TestFloat;
		double D = 1.0, TestDouble;
		int32 I = 1;
		uint32 U = 0;

		// Expected to compile
		TestFloat = FMath::Fmod(F, F);
		TestFloat = FMath::Fmod(F, I);
		TestFloat = FMath::Fmod(I, F);

#if MATHTEST_CHECK_INVALID_OVERLOAD_VARIANTS
		// Expected to generate warnings
		TestFloat = FMath::Fmod(F, D);
		TestFloat = FMath::Fmod(D, F);
		TestFloat = FMath::Fmod(D, D);
		TestFloat = FMath::Fmod(D, I);
		TestFloat = FMath::Fmod(I, D);
		TestFloat = FMath::Fmod(I, I); // Should be warned to be deprecated
#endif

		// Expected to compile
		TestDouble = FMath::Fmod(F, F);
		TestDouble = FMath::Fmod(F, D);
		TestDouble = FMath::Fmod(F, I);
		TestDouble = FMath::Fmod(D, F);
		TestDouble = FMath::Fmod(D, D);
		TestDouble = FMath::Fmod(D, I);
		TestDouble = FMath::Fmod(I, F);
		TestDouble = FMath::Fmod(I, D);

#if MATHTEST_CHECK_INVALID_OVERLOAD_VARIANTS
		// Expected to generate warnings
		TestDouble = FMath::Fmod(I, I); // Should be warned to be deprecated
#endif

#if MATHTEST_CHECK_INVALID_OVERLOAD_VARIANTS
		// Expected to generate warnings
		int TestInt;
		TestInt = FMath::Fmod(F, F);
		TestInt = FMath::Fmod(F, D);
		TestInt = FMath::Fmod(F, I);
		TestInt = FMath::Fmod(D, F);
		TestInt = FMath::Fmod(D, D);
		TestInt = FMath::Fmod(D, I);
		TestInt = FMath::Fmod(I, F);
		TestInt = FMath::Fmod(I, D);
		TestInt = FMath::Fmod(I, I); // Should be warned to be deprecated
#endif

		TestFloat = FMath::TruncToFloat(F);

		TestDouble = FMath::TruncToDouble(D);
		TestDouble = FMath::TruncToDouble(F);

		TestDouble = FMath::TruncToFloat(F);

		TestDouble = FMath::TruncToDouble(I); // not currently a warning

#if MATHTEST_CHECK_INVALID_OVERLOAD_VARIANTS
		// Expected to generate errors
		TestFloat = FMath::TruncToFloat(I);
		TestFloat = FMath::TruncToFloat(D);
		TestFloat = FMath::TruncToDouble(D);
		// Generates a warning on *some* platforms
		TestDouble = FMath::TruncToFloat(D);
#endif

		bool B;
		float F2 = 0, F3 = 0;
		double D1 = 0, D2 = 0, D3 = 0;

		// Check for ambiguity for bool predicates handling multiple float arguments and
		// where there are overloads with different numbers of arguments or with default values.
		B = FMath::IsNearlyEqual(F1, F2, F3);
		B = FMath::IsNearlyEqual(F1, F2);
		B = FMath::IsNearlyEqual(D1, D2, D3);
		B = FMath::IsNearlyEqual(D1, D2);

		B = FMath::IsNearlyEqual(F1, D2);
		B = FMath::IsNearlyEqual(D1, F2);

		B = FMath::IsNearlyEqual(F1, F2, 0.1);
		B = FMath::IsNearlyEqual(F1, F2, 0.1f);

		B = FMath::IsNearlyEqual(D1, F2, F3);
		B = FMath::IsNearlyEqual(F1, D2, F3);
		B = FMath::IsNearlyEqual(F1, F2, D3);
		B = FMath::IsNearlyEqual(D1, F2, D3);
		B = FMath::IsNearlyEqual(D1, D2, F3);
		B = FMath::IsNearlyEqual(F1, D2, D3);

		int32 I1 = 0, I2 = 0, I3 = 0;
		uint32 U2 = 0, U3 = 0;

		F = FMath::Clamp(F1, F2, F3);
		D = FMath::Clamp(D1, D2, D3);
		D = FMath::Clamp(D1, F2, F3);
		D = FMath::Clamp(F1, D2, D3);
		D = FMath::Clamp(D1, F2, D3);

		I = FMath::Clamp(I1, I2, I3);
		U = FMath::Clamp(U1, U2, U3);
		F = FMath::Clamp(F1, I1, F2);
		D = FMath::Clamp(F1, D2, I3);

		D = FMath::Max(F1, D1);
		D = FMath::Min(D1, F1);

		CheckPassing(FMath::Clamp(1, 0, 2) == 1);
		CheckPassing(FMath::Clamp(-1, 0, 2) == 0);
		CheckPassing(FMath::Clamp(3, 0, 2) == 2);

		U1 = 0; U2 = 2;
		CheckPassing(FMath::Clamp<int32>(-1, U1, U2) == 0);

#if MATHTEST_CHECK_INVALID_OVERLOAD_VARIANTS
		// Expected to generate errors (ambiguous arguments)
		U = FMath::Clamp(U1, I1, U3);
		// Expected to generate errors (double->float truncation)
		F = FMath::Clamp(F1, F2, D3);
		F = FMath::Clamp(F1, D2, F3);
		F = FMath::Clamp(D1, D2, F3);
		F = FMath::Max(F1, D1);
		F = FMath::Min(D1, F1);
		// Expected to generate errors (float->int truncation)
		I = FMath::Clamp(I1, F2, I3);
#endif

		int64 Big1 = 0, Big2 = 0, Big3 = 0;
		Big3 = FMath::Max(Big1, Big2);

		U1 = uint32(-1);
		CheckPassing(FMath::Max<int32>(0, U1) == 0);

		U1 = uint32(-1);
		CheckPassing(FMath::Min<int32>(0, U1) == -1);

		// Mix float/int types
		F = FMath::Max(Big1, F1);
		F = FMath::Max(F1, I1);
		D = FMath::Min(Big1, D1);
		D = FMath::Min(D1, I1);

		// Test compilation mixing int32/int64 types
		static_assert(std::is_same_v< decltype(FMath::Max(Big1, I1)), decltype(Big1) >);
		static_assert(std::is_same_v< decltype(FMath::Min(I1, Big1)), decltype(Big1) >);
		Big3 = FMath::Max(Big1, I1);
		Big3 = FMath::Min(Big1, I1);

		static_assert(std::is_same_v< decltype(FMath::Max(I1, Big2)), decltype(Big1) >);
		static_assert(std::is_same_v< decltype(FMath::Min(Big2, I1)), decltype(Big1) >);
		Big3 = FMath::Max(I1, Big2);
		Big3 = FMath::Min(I1, Big2);

		Big3 = FMath::Clamp(Big1, I1, I2);

#if MATHTEST_CHECK_INVALID_OVERLOAD_VARIANTS
		// Expected to generate errors (no overload for mixing signed/unsigned types).
		Big3 = FMath::Max(U1, I1);
		Big3 = FMath::Min(I1, U1);
		// Expected to generate errors or warnings (truncation / loss of data)
		Big3 = FMath::Clamp(I1, Big1, I2);
		I = FMath::Clamp<int32>(Big1, I1, I2);
#endif

		I = UE::LWC::FloatToIntCastChecked<int32>(F1);
		I = UE::LWC::FloatToIntCastChecked<int32>(D1);

		Big1 = UE::LWC::FloatToIntCastChecked<int64>(F1);
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(D1);

		// Valid (implicit int32->int64)
		Big1 = UE::LWC::FloatToIntCastChecked<int32>(F1);
		Big1 = UE::LWC::FloatToIntCastChecked<int32>(D1);

#if MATHTEST_CHECK_INVALID_OVERLOAD_VARIANTS
		// Shouldn't match a function
		I = UE::LWC::FloatToIntCastChecked(F1);
		I = UE::LWC::FloatToIntCastChecked(D1);
		Big1 = UE::LWC::FloatToIntCastChecked(F1);
		Big1 = UE::LWC::FloatToIntCastChecked(D1);
		I = UE::LWC::FloatToIntCastChecked(I);
		Big1 = UE::LWC::FloatToIntCastChecked(Big1);
		// Truncation warnings
		I = UE::LWC::FloatToIntCastChecked<int64>(F1);
		I = UE::LWC::FloatToIntCastChecked<int64>(D1);
		// Possibly invalid, depending on the generic base FloatToIntCastChecked, since this doesn't match a specialization
		Big1 = UE::LWC::FloatToIntCastChecked<int32>(I);
#endif

		// Test FloatToIntCastChecked() range limits

		// float->int32
		I = UE::LWC::FloatToIntCastChecked<int32>(float(TNumericLimits<int32>::Max() - 64));
		I = UE::LWC::FloatToIntCastChecked<int32>(float(TNumericLimits<int32>::Lowest()));

		// float->int64
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(float(TNumericLimits<int32>::Max()));
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(float(TNumericLimits<int32>::Lowest()));
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(float(TNumericLimits<int64>::Max() - (int64)549755813888)); // 2^63 - 1 - 2^39
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(float(TNumericLimits<int64>::Lowest()));

		// double->int32
		I = UE::LWC::FloatToIntCastChecked<int32>(double(TNumericLimits<int32>::Max()));
		I = UE::LWC::FloatToIntCastChecked<int32>(double(TNumericLimits<int32>::Lowest()));

		// double->int64
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(double(TNumericLimits<int64>::Max() - 512));
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(double(TNumericLimits<int64>::Lowest()));

#if MATHTEST_CHECK_INVALID_OVERLOAD_VARIANTS
		// These should all assert
		I = UE::LWC::FloatToIntCastChecked<int32>(float(TNumericLimits<int32>::Max()));
		I = UE::LWC::FloatToIntCastChecked<int32>(TNumericLimits<float>::Max());
		I = UE::LWC::FloatToIntCastChecked<int32>(TNumericLimits<float>::Lowest());

		I = UE::LWC::FloatToIntCastChecked<int32>(TNumericLimits<double>::Max());
		I = UE::LWC::FloatToIntCastChecked<int32>(TNumericLimits<double>::Lowest());

		Big1 = UE::LWC::FloatToIntCastChecked<int64>(float(TNumericLimits<int64>::Max()));
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(TNumericLimits<float>::Max());
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(TNumericLimits<float>::Lowest());

		Big1 = UE::LWC::FloatToIntCastChecked<int64>(double(TNumericLimits<int64>::Max()));
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(TNumericLimits<double>::Max());
		Big1 = UE::LWC::FloatToIntCastChecked<int64>(TNumericLimits<double>::Lowest());
#endif

	}

	// Sin, Cos, Tan tests
	TestVectorTrigFunctions<float, VectorRegister4Float>();

	// Exp, Log tests
	TestVectorExpLogFunctions<float, VectorRegister4Float>();

	FQuat4f Q0, Q1, Q2, Q3;

	// Quat<->Rotator conversions and equality
	{
		// Identity conversion
		{
			const FRotator3f R0 = FRotator3f::ZeroRotator;
			const FRotator3f R1 = FRotator3f(FQuat4f::Identity);
			LogRotatorTest(true, TEXT("FRotator3f::ZeroRotator ~= FQuat4f::Identity : Rotator"), R0, R1, R0.Equals(R1, 0.f));
			LogRotatorTest(true, TEXT("FRotator3f::ZeroRotator == FQuat4f::Identity : Rotator"), R0, R1, R0 == R1);
			LogRotatorTest(true, TEXT("FRotator3f::ZeroRotator not != FQuat4f::Identity : Rotator"), R0, R1, !(R0 != R1));

			Q0 = FQuat4f::Identity;
			Q1 = FQuat4f(FRotator3f::ZeroRotator);
			LogQuaternionTest(TEXT("FRotator3f::ZeroRotator ~= FQuat4f::Identity : Quaternion"), Q0, Q1, Q0.Equals(Q1, 0.f));
			LogQuaternionTest(TEXT("FRotator3f::ZeroRotator == FQuat4f::Identity : Quaternion"), Q0, Q1, Q0 == Q1);
			LogQuaternionTest(TEXT("FRotator3f::ZeroRotator not != FQuat4f::Identity : Quaternion"), Q0, Q1, !(Q0 != Q1));
		}

		const float Nudge = UE_KINDA_SMALL_NUMBER * 0.25f;
		const FRotator3f RotArray[] = {
			FRotator3f(0.f, 0.f, 0.f),
			FRotator3f(Nudge, -Nudge, Nudge),
			FRotator3f(+180.f, -180.f, +180.f),
			FRotator3f(-180.f, +180.f, -180.f),
			FRotator3f(+45.0f - Nudge, -120.f + Nudge, +270.f - Nudge),
			FRotator3f(-45.0f + Nudge, +120.f - Nudge, -270.f + Nudge),
			FRotator3f(+315.f - 360.f, -240.f - 360.f, -90.0f - 360.f),
			FRotator3f(-315.f + 360.f, +240.f + 360.f, +90.0f + 360.f),
			FRotator3f(+360.0f, -720.0f, 1080.0f),
			FRotator3f(+360.0f + 1.0f, -720.0f + 1.0f, 1080.0f + 1.0f),
			FRotator3f(+360.0f + Nudge, -720.0f - Nudge, 1080.0f - Nudge),
			//FRotator3f(+360.0f * 1e10f, -720.0f * 1000000.0f, 1080.0f * 12345.f),	//this breaks when underlying math operations use HW FMA
			FRotator3f(+8388608.f, +8388608.f - 1.1f, -8388608.f - 1.1f),
			FRotator3f(+8388608.f + Nudge, +8388607.9f, -8388607.9f)
		};

		// FRotator3f tests
		{
			// Equality test
			const float RotTolerance = UE_KINDA_SMALL_NUMBER;
			for (auto const& A : RotArray)
			{
				for (auto const& B : RotArray)
				{
					//UE_LOGF(LogUnrealMathTest, Log, "A ?= B : %ls ?= %ls", *A.ToString(), *B.ToString());
					const bool bExpected = TestRotatorEqual0(A, B, RotTolerance);
					LogRotatorTest(bExpected, TEXT("TestRotatorEqual1"), A, B, TestRotatorEqual1(A, B, RotTolerance));
					LogRotatorTest(bExpected, TEXT("TestRotatorEqual2"), A, B, TestRotatorEqual2(A, B, RotTolerance));
					LogRotatorTest(bExpected, TEXT("TestRotatorEqual3"), A, B, TestRotatorEqual3(A, B, RotTolerance));
				}
			}
		}

		// Quaternion conversion test
		const float QuatTolerance = 1e-6f;
		for (auto const& A : RotArray)
		{
			const FQuat4f QA = TestRotatorToQuaternion(A);
			const FQuat4f QB = FQuat4f(A);
			LogQuaternionTest(TEXT("TestRotatorToQuaternion"), QA, QB, TestQuatsEqual(QA, QB, QuatTolerance));
		}
	}

	// Rotator->Quat->Rotator
	{
		const float Nudge = UE_KINDA_SMALL_NUMBER * 0.25f;
		const FRotator3f RotArray[] = {
			FRotator3f(0.0f, 0.0f, 0.0f),
			FRotator3f(30.0f, 30.0f, 30.0f),
			FRotator3f(30.0f, -45.0f, 90.0f),
			FRotator3f(45.0f, 60.0f, -120.0f),
			FRotator3f(0.f, 90.f, 0.f),
			FRotator3f(0.f, -90.f, 0.f),
			FRotator3f(0.f, 180.f, 0.f),
			FRotator3f(0.f, -180.f, 0.f),
			FRotator3f(90.f, 0.f, 0.f),
			FRotator3f(-90.f, 0.f, 0.f),
			FRotator3f(150.f, 0.f, 0.f),
			FRotator3f(0.0f, 0.0f, 45.0f),
			FRotator3f(0.0f, 0.0f, -45.0f),
			FRotator3f(+360.0f, -720.0f, 1080.0f),
			FRotator3f(+360.0f + 1.0f, -720.0f + 1.0f, 1080.0f + 1.0f),
			FRotator3f(+360.0f + Nudge, -720.0f - Nudge, 1080.0f - Nudge),
			FRotator3f(+360.0f * 1e10f, -720.0f * 1000000.0f, 1080.0f * 12345.f),
			FRotator3f(+8388608.f, +8388608.f - 1.1f, -8388608.f - 1.1f),
			FRotator3f(+8388609.1f, +8388607.9f, -8388609.1f)
		};

		for (FRotator3f const& Rotator0 : RotArray)
		{
			Q0 = TestRotatorToQuaternion(Rotator0);
			FRotator3f Rotator1 = Q0.Rotator();
			FRotator3f Rotator2 = TestQuaternionToRotator(Q0);
			LogRotatorTest(TEXT("Rotator->Quat->Rotator"), Rotator1, Rotator2, Rotator1.Equals(Rotator2, 1e-4f));
		}
	}

	// Quat -> Axis and Angle
	{
		FVector3f Axis;
		float Angle;

		// Identity -> X Axis
		Axis = FQuat4f::Identity.GetRotationAxis();
		LogTest<float>(TEXT("FQuat4f::Identity.GetRotationAxis() == FVector3f::XAxisVector"), TestFVector3Equal(Axis, FVector3f::XAxisVector));

		const FQuat4f QuatArray[] = {
			FQuat4f(0.0f, 0.0f, 0.0f, 1.0f),
			FQuat4f(1.0f, 0.0f, 0.0f, 0.0f),
			FQuat4f(0.0f, 1.0f, 0.0f, 0.0f),
			FQuat4f(0.0f, 0.0f, 1.0f, 0.0f),
			FQuat4f(0.000046571717f, -0.000068426132f, 0.000290602446f, 0.999999881000f) // length = 0.99999992665
		};

		for (const FQuat4f& Q : QuatArray)
		{
			Q.ToAxisAndAngle(Axis, Angle);
			LogTest<float>(TEXT("Quat -> Axis and Angle: Q is Normalized"), TestQuatNormalized(Q, 1e-6f));
			LogTest<float>(TEXT("Quat -> Axis and Angle: Axis is Normalized"), TestFVector3Normalized(Axis, 1e-6f));
		}
	}

	// Quat Lerp/Slerp
	{
		// Note: Values chosen to have a positive 'RawCosom' within Slerp_NotNormalized()
		const FRotator3f Rotator0 = FRotator3f(300.0f, -45.0f, 0.0f);
		const FRotator3f Rotator1 = FRotator3f(10.0f, 270.0f, 30.0f);
		const float FloatAlpha = 0.25f;
		const double DoubleAlpha = 0.25;

		// Quat<float>
		Q0 = FQuat4f(Rotator0);
		Q1 = FQuat4f(Rotator1);

		// float alpha
		Q2 = FMath::Lerp(Q0, Q1, FloatAlpha);
		Q3 = FQuat4f::Slerp(Q0, Q1, FloatAlpha);
		LogTest<float>(TEXT("TestQuatLerp"), TestQuatsEqual(Q2, Q3, 1e-6f));
		Q2 = Old_Slerp(Q0, Q1, FloatAlpha);
		LogTest<float>(TEXT("TestQuatLerp"), TestQuatsEqual(Q2, Q3, 1e-6f));

		Q2 = FMath::Lerp<FQuat4f>(Q0, Q1, FloatAlpha);
		Q3 = FQuat4f::Slerp(Q0, Q1, FloatAlpha);
		LogTest<float>(TEXT("TestQuatLerp"), TestQuatsEqual(Q2, Q3, 1e-6f));
		Q2 = Old_Slerp(Q0, Q1, FloatAlpha);
		LogTest<float>(TEXT("TestQuatLerp"), TestQuatsEqual(Q2, Q3, 1e-6f));

		// double alpha
		Q2 = FMath::Lerp(Q0, Q1, -DoubleAlpha);
		Q3 = FQuat4f::Slerp(Q0, Q1, float(-DoubleAlpha));
		LogTest<float>(TEXT("TestQuatLerp"), TestQuatsEqual(Q2, Q3, 1e-6f));
		Q2 = Old_Slerp(Q0, Q1, float(-DoubleAlpha));
		LogTest<float>(TEXT("TestQuatLerp"), TestQuatsEqual(Q2, Q3, 1e-6f));

		Q2 = FMath::Lerp<FQuat4f>(Q0, Q1, -DoubleAlpha);
		Q3 = FQuat4f::Slerp(Q0, Q1, float(-DoubleAlpha));
		LogTest<float>(TEXT("TestQuatLerp"), TestQuatsEqual(Q2, Q3, 1e-6f));
		Q2 = Old_Slerp(Q0, Q1, float(-DoubleAlpha));
		LogTest<float>(TEXT("TestQuatLerp"), TestQuatsEqual(Q2, Q3, 1e-6f));

		// Quat<double>
		FQuat4d QD0, QD1, QD2, QD3;
		// Note: Values chosen to have a negative 'RawCosom' within Slerp_NotNormalized()
		QD0 = FQuat4d(FRotator3d(30.0f, 45.0f, 0.0f));
		QD1 = FQuat4d(FRotator3d(10.0f, -270.0f, -130.0f));

		// double alpha
		QD2 = FMath::Lerp(QD0, QD1, DoubleAlpha);
		QD3 = FQuat4d::Slerp(QD0, QD1, DoubleAlpha);
		LogTest<double>(TEXT("TestQuatLerp"), TestQuatsEqual(QD2, QD3, 1e-6f));
		QD2 = Old_Slerp(QD0, QD1, DoubleAlpha);
		LogTest<double>(TEXT("TestQuatLerp"), TestQuatsEqual(QD2, QD3, 1e-6f));

		QD2 = FMath::Lerp<FQuat4d>(QD0, QD1, DoubleAlpha);
		QD3 = FQuat4d::Slerp(QD0, QD1, DoubleAlpha);
		LogTest<double>(TEXT("TestQuatLerp"), TestQuatsEqual(QD2, QD3, 1e-6f));
		QD2 = Old_Slerp(QD0, QD1, DoubleAlpha);
		LogTest<double>(TEXT("TestQuatLerp"), TestQuatsEqual(QD2, QD3, 1e-6f));

		// float alpha
		QD2 = FMath::Lerp(QD0, QD1, -FloatAlpha);
		QD3 = FQuat4d::Slerp(QD0, QD1, -FloatAlpha);
		LogTest<double>(TEXT("TestQuatLerp"), TestQuatsEqual(QD2, QD3, 1e-6f));
		QD2 = Old_Slerp(QD0, QD1, double(-FloatAlpha));
		LogTest<double>(TEXT("TestQuatLerp"), TestQuatsEqual(QD2, QD3, 1e-6f));

		QD2 = FMath::Lerp<FQuat4d>(QD0, QD1, -FloatAlpha);
		QD3 = FQuat4d::Slerp(QD0, QD1, -FloatAlpha);
		LogTest<double>(TEXT("TestQuatLerp"), TestQuatsEqual(QD2, QD3, 1e-6f));
		QD2 = Old_Slerp(QD0, QD1, double(-FloatAlpha));
		LogTest<double>(TEXT("TestQuatLerp"), TestQuatsEqual(QD2, QD3, 1e-6f));
	}

	// Quat / Rotator conversion to vectors, matrices
	{
		FRotator3f Rotator0;
		Rotator0 = FRotator3f(30.0f, -45.0f, 90.0f);
		Q0 = FQuat4f(Rotator0);
		Q1 = TestRotatorToQuaternion(Rotator0);
		LogTest<float>(TEXT("TestRotatorToQuaternion"), TestQuatsEqual(Q0, Q1, 1e-6f));

		using namespace UE::Math;

		FVector3f FV0, FV1;
		FV0 = Rotator0.Vector();
		FV1 = TRotationMatrix<float>(Rotator0).GetScaledAxis(EAxis::X);
		LogTest<float>(TEXT("Test0 Rotator::Vector()"), TestFVector3Equal(FV0, FV1, 1e-6f));

		FV0 = TRotationMatrix<float>(Rotator0).GetScaledAxis(EAxis::X);
		FV1 = TQuatRotationMatrix<float>(FQuat4f(Q0.X, Q0.Y, Q0.Z, Q0.W)).GetScaledAxis(EAxis::X);
		LogTest<float>(TEXT("Test0 FQuatRotationMatrix"), TestFVector3Equal(FV0, FV1, 1e-5f));

		Rotator0 = FRotator3f(45.0f, 60.0f, 120.0f);
		Q0 = FQuat4f(Rotator0);
		Q1 = TestRotatorToQuaternion(Rotator0);
		LogTest<float>(TEXT("TestRotatorToQuaternion"), TestQuatsEqual(Q0, Q1, 1e-6f));

		FV0 = Rotator0.Vector();
		FV1 = TRotationMatrix<float>(Rotator0).GetScaledAxis(EAxis::X);
		LogTest<float>(TEXT("Test1 Rotator::Vector()"), TestFVector3Equal(FV0, FV1, 1e-6f));

		FV0 = TRotationMatrix<float>(Rotator0).GetScaledAxis(EAxis::X);
		FV1 = TQuatRotationMatrix<float>(FQuat4f(Q0.X, Q0.Y, Q0.Z, Q0.W)).GetScaledAxis(EAxis::X);
		LogTest<float>(TEXT("Test1 FQuatRotationMatrix"), TestFVector3Equal(FV0, FV1, 1e-5f));

		FV0 = TRotationMatrix<float>(FRotator3f::ZeroRotator).GetScaledAxis(EAxis::X);
		FV1 = TQuatRotationMatrix<float>(FQuat4f::Identity).GetScaledAxis(EAxis::X);
		LogTest<float>(TEXT("Test2 FQuatRotationMatrix"), TestFVector3Equal(FV0, FV1, 1e-6f));
	}

	// Quat Rotation tests
	{
		// Use these Quats...
		const FQuat4f TestQuats[] = {
			FQuat4f(FQuat4f::Identity),
			FQuat4f(FRotator3f(30.0f, 30.0f, 30.0f)),
			FQuat4f(FRotator3f(30.0f, -45.0f, 90.0f)),
			FQuat4f(FRotator3f(45.0f,  60.0f, 120.0f)),
			FQuat4f(FRotator3f(0.0f, 180.0f, 45.0f)),
			FQuat4f(FRotator3f(-120.0f, -90.0f, 0.0f)),
			FQuat4f(FRotator3f(-0.01f, 0.02f, -0.03f)),
		};

		// ... to rotate these Vectors...
		const FVector3f TestVectors[] = {
			FVector3f::ZeroVector,
			FVector3f::ForwardVector,
			FVector3f::RightVector,
			FVector3f::UpVector,
			FVector3f(45.0f, -60.0f, 120.0f),
			FVector3f(-45.0f, 60.0f, -120.0f),
			FVector3f(0.57735026918962576451f, 0.57735026918962576451f, 0.57735026918962576451f),
			FVector3f( 0.01f,  0.02f,  0.03f),
			FVector3f(-0.01f,  0.02f, -0.03f),
			FVector3f( 0.01f, -0.02f,  0.03f),
		};

		// ... and test within this tolerance.
		const float Tolerance = 1e-4f;

		// Test Macro. Tests FQuat4f::RotateVector(_Vec) against _Func(_Vec)
#define TEST_QUAT_ROTATE(_QIndex, _VIndex, _Quat, _Vec, _Func, _Tolerance) \
		{ \
			const FString _TestName = FString::Printf(TEXT("Test Quat%d: Vec%d: %s"), _QIndex, _VIndex, TEXT(#_Func)); \
			LogTest<float>( *_TestName, TestFVector3Equal(_Quat.RotateVector(_Vec), _Func(_Quat, _Vec), _Tolerance) ); \
		}

		// Test loop
		for (int32 QIndex = 0; QIndex < UE_ARRAY_COUNT(TestQuats); ++QIndex)
		{
			const FQuat4f& Q = TestQuats[QIndex];
			for (int32 VIndex = 0; VIndex < UE_ARRAY_COUNT(TestVectors); ++VIndex)
			{
				const FVector3f& V = TestVectors[VIndex];
				TEST_QUAT_ROTATE(QIndex, VIndex, Q, V, TestQuaternionRotateVectorScalar, Tolerance);
				TEST_QUAT_ROTATE(QIndex, VIndex, Q, V, TestQuaternionRotateVectorRegister, Tolerance);
				TEST_QUAT_ROTATE(QIndex, VIndex, Q, V, TestQuaternionMultiplyVector, Tolerance);
			}
		}

		
		// FindBetween
		{
			const FVector3f Signs[] = {
				FVector3f(1.0f, 1.0f, 1.0f),
				FVector3f(-1.0f, -1.0f, -1.0f),
			};

			for (FVector3f const& SignVector : Signs)
			{
				for (FVector3f const& A : TestVectors)
				{
					for (FVector3f const& B : TestVectors)
					{
						const FVector3f ANorm = A.GetSafeNormal();
						const FVector3f BNorm = (B * SignVector).GetSafeNormal();

						const FQuat4f Old = FindBetween_Old(ANorm, BNorm);
						const FQuat4f Old_5_2 = FindBetweenNormals_5_2(ANorm, BNorm);
						const FQuat4f NewNormal = FQuat4f::FindBetweenNormals(ANorm, BNorm);
						const FQuat4f NewVector = FQuat4f::FindBetweenVectors(A, B * SignVector);

						const FVector3f RotAOld = Old.RotateVector(ANorm);
						const FVector3f RotAOld_5_2 = Old_5_2.RotateVector(ANorm);
						const FVector3f RotANewNormal = NewNormal.RotateVector(ANorm);
						const FVector3f RotANewVector = NewVector.RotateVector(ANorm);

						if (A.IsZero() || B.IsZero())
						{
							LogTest<float>(TEXT("FindBetween: Old == New (normal)"), TestQuatsEqual(Old, NewNormal, 1e-6f));
							LogTest<float>(TEXT("FindBetween: Old_5_2 == New (normal)"), TestQuatsEqual(Old_5_2, NewNormal, 1e-6f));
							LogTest<float>(TEXT("FindBetween: Old == New (vector)"), TestQuatsEqual(Old, NewVector, 1e-6f));
						}
						else
						{
							LogTest<float>(TEXT("FindBetween: Old_5_2 == New (normal)"), TestQuatsEqual(Old_5_2, NewNormal, 1e-6f));

							LogTest<float>(TEXT("FindBetween: Old A->B"), TestFVector3Equal(RotAOld, BNorm, UE_KINDA_SMALL_NUMBER));
							LogTest<float>(TEXT("FindBetween: Old_5_2 A->B"), TestFVector3Equal(RotAOld_5_2, BNorm, UE_KINDA_SMALL_NUMBER));
							LogTest<float>(TEXT("FindBetween: New A->B (normal)"), TestFVector3Equal(RotANewNormal, BNorm, UE_KINDA_SMALL_NUMBER));
							LogTest<float>(TEXT("FindBetween: New A->B (vector)"), TestFVector3Equal(RotANewVector, BNorm, UE_KINDA_SMALL_NUMBER));
						}
					}
				}
			}
		}


		// FVector3f::ToOrientationRotator(), FVector3f::ToOrientationQuat()
		{
			for (FVector3f const& V : TestVectors)
			{
				const FVector3f VNormal = V.GetSafeNormal();

				Q0 = FQuat4f::FindBetweenNormals(FVector3f::ForwardVector, VNormal);
				Q1 = V.ToOrientationQuat();
				const FRotator3f R0 = V.ToOrientationRotator();

				const FVector3f Rotated0 = Q0.RotateVector(FVector3f::ForwardVector);
				const FVector3f Rotated1 = Q1.RotateVector(FVector3f::ForwardVector);
				const FVector3f Rotated2 = R0.RotateVector(FVector3f::ForwardVector);

				LogTest<float>(TEXT("V.ToOrientationQuat() rotate"), TestFVector3Equal(Rotated0, Rotated1, UE_KINDA_SMALL_NUMBER));
				LogTest<float>(TEXT("V.ToOrientationRotator() rotate"), TestFVector3Equal(Rotated0, Rotated2, UE_KINDA_SMALL_NUMBER));
			}
		}
	}

	// Quat multiplication, Matrix conversion
	{
		FMatrix44f M0, M1;
		FMatrix44f IdentityInverse = FMatrix44f::Identity.Inverse();

		Q0 = FQuat4f(FRotator3f(30.0f, -45.0f, 90.0f));
		Q1 = FQuat4f(FRotator3f(45.0f, 60.0f, 120.0f));
		VectorQuaternionMultiply(&Q2, &Q0, &Q1);
		TestVectorQuaternionMultiply(&Q3, &Q0, &Q1);
		LogTest<float>(TEXT("VectorQuaternionMultiply"), TestQuatsEqual(Q2, Q3, 1e-6f));
		V0 = VectorLoadAligned(&Q0);
		V1 = VectorLoadAligned(&Q1);
		V2 = VectorQuaternionMultiply2(V0, V1);
		V3 = VectorLoadAligned(&Q3);
		LogTest<float>(TEXT("VectorQuaternionMultiply2"), TestVectorsEqual(V2, V3, 1e-6f));

		M0 = Q0 * FMatrix44f::Identity;
		M1 = UE::Math::TQuatRotationMatrix<float>(Q0);
		LogTest<float>(TEXT("QuaterionMatrixConversion Q0"), TestMatricesEqual(M0, M1, 0.000001f));
		M0 = Q1.ToMatrix();
		M1 = UE::Math::TQuatRotationMatrix<float>(Q1);
		LogTest<float>(TEXT("QuaterionMatrixConversion Q1"), TestMatricesEqual(M0, M1, 0.000001f));

		Q0 = FQuat4f(FRotator3f(0.0f, 180.0f, 45.0f));
		Q1 = FQuat4f(FRotator3f(-120.0f, -90.0f, 0.0f));
		VectorQuaternionMultiply(&Q2, &Q0, &Q1);
		TestVectorQuaternionMultiply(&Q3, &Q0, &Q1);
		LogTest<float>(TEXT("VectorQuaternionMultiply"), TestQuatsEqual(Q2, Q3, 1e-6f));
		V0 = VectorLoadAligned(&Q0);
		V1 = VectorLoadAligned(&Q1);
		V2 = VectorQuaternionMultiply2(V0, V1);
		V3 = VectorLoadAligned(&Q3);
		LogTest<float>(TEXT("VectorQuaternionMultiply2"), TestVectorsEqual(V2, V3, 1e-6f));

		M0 = (-Q0) * FMatrix44f::Identity;
		M1 = (-Q0).ToMatrix();
		LogTest<float>(TEXT("QuaterionMatrixConversion Q0"), TestMatricesEqual(M0, M1, 0.000001f));
		M0 = (Q1.Inverse().Inverse()) * IdentityInverse;
		M1 = Q1.ToMatrix();
		LogTest<float>(TEXT("QuaterionMatrixConversion Q1"), TestMatricesEqual(M0, M1, 0.000001f));
	}

	// Bounds
	{
		FBounds3f B0, B1, B2;

		// B0 is the unit cube
		B0 += FVector3f(-1.0f, -1.0f, -1.0f);
		B0 += FVector3f( 1.0f,  1.0f,  1.0f);
		// B1 is fully contained within
		B1 += FVector3f(-0.50f, 0.00f, -0.75f);
		B1 += FVector3f( 0.75f, 0.25f,  1.00f);
		// B2 intersects with B0 but not with B1, with the separation occuring only on the Y axis
		B2 += FVector3f(-0.75f, -1.25f, -0.60f);
		B2 += FVector3f( 0.50f, -0.25f,  1.00f);

		LogBoundsTest(true, TEXT("BoundsIntersect(B0, B1)"), B0, B1, B0.Intersect(B1));
		LogBoundsTest(true, TEXT("BoundsIntersect(B0, B2)"), B0, B2, B0.Intersect(B2));
		LogBoundsTest(false, TEXT("BoundsIntersect(B1, B2)"), B1, B2, B1.Intersect(B2));

		LogBoundsTest(true, TEXT("BoundsContains(B0, B1)"), B0, B1, B0.Contains(B1));
		LogBoundsTest(false, TEXT("BoundsContains(B0, B2)"), B0, B2, B0.Contains(B2));
		LogBoundsTest(false, TEXT("BoundsContains(B1, B0)"), B1, B0, B1.Contains(B0));
		LogBoundsTest(false, TEXT("BoundsContains(B1, B2)"), B1, B2, B1.Contains(B2));
		LogBoundsTest(false, TEXT("BoundsContains(B2, B0)"), B2, B0, B2.Contains(B0));
		LogBoundsTest(false, TEXT("BoundsContains(B2, B1)"), B2, B1, B2.Contains(B1));
	}

	if (!GPassing)
	{
		UE_LOGF(LogUnrealMathTest, Fatal, "VectorIntrinsics <float> Failed.");
	}

	if (!RunDoubleVectorTest())
	{
		UE_LOGF(LogUnrealMathTest, Fatal, "VectorIntrinsics <double> Failed.");
	}
}

TEST_CASE_NAMED(FInterpolationFunctionTests, "System::Core::Math::Interpolation Function Test", "[ApplicationContextMask][SmokeFilter]")
{
	// The purpose of this test is to verify that various combinations of the easing functions are actually equivalent.
	// It currently only tests the InOut versions over different ranges, because the initial implementation was bad.
	// Further improvements (optimizations, new easing functions) to the easing functions should be accompanied by
	// expansions to this test suite.
	typedef float(*EasingFunc)(float Percent);
	auto RunInOutTest = [](const TArray< TPair<EasingFunc, FString> >& Functions)
	{
		for (int32 I = 0; I < 100; ++I)
		{
			float Percent = (float)I / 100.f;
			TArray<float> Values;
			for (const auto& Entry : Functions)
			{
				Values.Push(Entry.Key(Percent));
			}

			bool bSucceeded = true;
			int32 K = 0;
			for (int32 J = 1; J < Functions.Num(); ++J)
			{
				if (!FMath::IsNearlyEqual(Values[K], Values[J], 0.0001f))
				{
					for (int32 L = 0; L < Values.Num(); ++L)
					{
						INFO(FString::Printf(TEXT("%s: %f"), *(Functions[L].Value), Values[L]));
					}
					REQUIRE_MESSAGE(FString::Printf(TEXT("Easing Function tests failed at index %d!"), I), false);
				}
			}
			
		}
	};

#define INTERP_WITH_RANGE( RANGE_MIN, RANGE_MAX, FUNCTION, IDENTIFIER ) \
	auto FUNCTION##IDENTIFIER = [](float Percent ) \
	{ \
		const float Min = RANGE_MIN; \
		const float Max = RANGE_MAX; \
		const float Range = Max - Min; \
		return (FMath::FUNCTION(Min, Max, Percent) - Min) / Range; \
	};

	{
		// Test InterpExpoInOut:
		INTERP_WITH_RANGE(.9f, 1.2f, InterpExpoInOut, A)
			INTERP_WITH_RANGE(0.f, 1.f, InterpExpoInOut, B)
			INTERP_WITH_RANGE(-8.6f; , 2.3f, InterpExpoInOut, C)
			TArray< TPair< EasingFunc, FString > > FunctionsToTest;
		FunctionsToTest.Emplace(InterpExpoInOutA, TEXT("InterpExpoInOutA"));
		FunctionsToTest.Emplace(InterpExpoInOutB, TEXT("InterpExpoInOutB"));
		FunctionsToTest.Emplace(InterpExpoInOutC, TEXT("InterpExpoInOutC"));
		RunInOutTest(FunctionsToTest);
	}

	{
		// Test InterpCircularInOut:
		INTERP_WITH_RANGE(5.f, 9.32f, InterpCircularInOut, A)
			INTERP_WITH_RANGE(0.f, 1.f, InterpCircularInOut, B)
			INTERP_WITH_RANGE(-8.1f; , -.75f, InterpCircularInOut, C)
			TArray< TPair< EasingFunc, FString > > FunctionsToTest;
		FunctionsToTest.Emplace(InterpCircularInOutA, TEXT("InterpCircularInOutA"));
		FunctionsToTest.Emplace(InterpCircularInOutB, TEXT("InterpCircularInOutB"));
		FunctionsToTest.Emplace(InterpCircularInOutC, TEXT("InterpCircularInOutC"));
		RunInOutTest(FunctionsToTest);
	}

	{
		// Test InterpSinInOut:
		INTERP_WITH_RANGE(10.f, 11.2f, InterpSinInOut, A)
			INTERP_WITH_RANGE(0.f, 1.f, InterpSinInOut, B)
			INTERP_WITH_RANGE(-5.6f; , -4.3f, InterpSinInOut, C)
			TArray< TPair< EasingFunc, FString > > FunctionsToTest;
		FunctionsToTest.Emplace(InterpSinInOutA, TEXT("InterpSinInOutA"));
		FunctionsToTest.Emplace(InterpSinInOutB, TEXT("InterpSinInOutB"));
		FunctionsToTest.Emplace(InterpSinInOutC, TEXT("InterpSinInOutC"));
		RunInOutTest(FunctionsToTest);
	}
}

class FMathVectorTestParameter
{
public:
	FMathVectorTestParameter(float F) : Vec(VectorSet1(F))
	{
	}

	FMathVectorTestParameter(int I) : Vec(VectorCastIntToFloat(VectorIntSet1(I)))
	{
	}

	FMathVectorTestParameter(const VectorRegister4Float& F) : Vec(F)
	{
	}

	FMathVectorTestParameter(const VectorRegister4Int& I) : Vec(VectorCastIntToFloat(I))
	{
	}

	friend bool operator ==(const FMathVectorTestParameter& lhs, const FMathVectorTestParameter& rhs)
	{
		return TestVectorsEqualBitwise(lhs.Vec, rhs.Vec);
	}

	friend bool operator !=(const FMathVectorTestParameter& lhs, const FMathVectorTestParameter& rhs)
	{
		return !(lhs == rhs);
	}

private:
	VectorRegister4Float Vec;
};

template<class VectorRoundTests, auto VectorRounder, auto ScalarRounder>
class TTestEqualAfterVectorRounding
{
public:
	explicit TTestEqualAfterVectorRounding(VectorRoundTests& InRoundTests) : RoundTests(InRoundTests)
	{
	}

	template<class ExpectedType>
	bool operator ()(const TCHAR* What, float Actual, ExpectedType Expected, bool bTestScalar = true) const
	{
		// Compare the result of the vector implementation with the equivalent scalar implementation.
		if (bTestScalar)
			CHECK_MESSAGE(What, FMathVectorTestParameter(VectorRounder(VectorSet1(Actual))) == FMathVectorTestParameter(ScalarRounder(Actual)));
		// Compare the result of the vector implementation with the given expected value.
		CHECK_MESSAGE(What, FMathVectorTestParameter(VectorRounder(VectorSet1(Actual))) == FMathVectorTestParameter(Expected));
		return true;
	}

private:
	VectorRoundTests& RoundTests;
};

FORCEINLINE VectorRegister4Int VectorRoundToIntHalfToEvenTest(const VectorRegister4Float& Vec)
{
	return VectorRoundToIntHalfToEven(Vec);
}

class FMathVectorRoundToIntHalfToEvenTestsClass {
public:
	bool MathVectorRoundToIntHalfToEvenTest()
	{
		const bool bSkipScalar = false;

		struct FMathVectorRoundToIntHalfToEvenTestsRoundToInt
		{
			static int RoundToInt(float F)
			{
				return int(FMath::RoundHalfToEven(F));
			}
		};
		TTestEqualAfterVectorRounding<FMathVectorRoundToIntHalfToEvenTestsClass, &VectorRoundToIntHalfToEvenTest, &FMathVectorRoundToIntHalfToEvenTestsRoundToInt::RoundToInt> TestEqualAfterVectorRounding(*this);

		TestEqualAfterVectorRounding(TEXT("VectorRound32-Zero"), 0.0f, 0);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-One"), 1.0f, 1);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-LessHalf"), 1.4f, 1);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegGreaterHalf"), -1.4f, -1);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-LessNearHalf"), 1.4999999f, 1);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegGreaterNearHalf"), -1.4999999f, -1);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-Half"), 1.5f, 2);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegHalf"), -1.5f, -2);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-GreaterNearHalf"), 1.5000001f, 2);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegLesserNearHalf"), -1.5000001f, -2);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-GreaterThanHalf"), 1.6f, 2);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegLesserThanHalf"), -1.6f, -2);

		TestEqualAfterVectorRounding(TEXT("VectorRound32-TwoToOneBitPrecision"), 4194303.25f, 4194303);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-TwoToOneBitPrecision"), 4194303.5f, 4194304);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-TwoToOneBitPrecision"), 4194303.75f, 4194304);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-TwoToOneBitPrecision"), 4194304.0f, 4194304);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-TwoToOneBitPrecision"), 4194304.5f, 4194304);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegTwoToOneBitPrecision"), -4194303.25f, -4194303);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegTwoToOneBitPrecision"), -4194303.5f, -4194304);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegTwoToOneBitPrecision"), -4194303.75f, -4194304);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegTwoToOneBitPrecision"), -4194304.0f, -4194304);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegTwoToOneBitPrecision"), -4194304.5f, -4194304);

		TestEqualAfterVectorRounding(TEXT("VectorRound32-OneToZeroBitPrecision"), 8388607.0f, 8388607);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-OneToZeroBitPrecision"), 8388607.5f, 8388608);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-OneToZeroBitPrecision"), 8388608.0f, 8388608);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-OneToZeroBitPrecision"), 8388608.5f, 8388608);
		//TestEqualAfterVectorRounding(TEXT("VectorRound32-OneToZeroBitPrecision"), 8388609.0f, 8388609, bSkipScalar); // FMath::RoundHalfToEven incorrectly rounds 8388609.0f to 8388610.0f, so skip it.
		TestEqualAfterVectorRounding(TEXT("VectorRound32-OneToZeroBitPrecision"), 8388609.5f, 8388610);

		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegOneToZeroBitPrecision"), -8388607.0f, -8388607);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegOneToZeroBitPrecision"), -8388607.5f, -8388608);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegOneToZeroBitPrecision"), -8388608.0f, -8388608);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegOneToZeroBitPrecision"), -8388608.5f, -8388608);
		//TestEqualAfterVectorRounding(TEXT("VectorRound32-NegOneToZeroBitPrecision"), -8388609.0f, -8388609, bSkipScalar); // FMath::RoundHalfToEven incorrectly rounds -8388609.0f to -8388610.0f, so skip it.
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegOneToZeroBitPrecision"), -8388609.5f, -8388610);

		//TestEqualAfterVectorRounding(TEXT("VectorRound32-ZeroBitPrecision"), 16777215.0f, 16777215, bSkipScalar); // FMath::RoundHalfToEven incorrectly rounds 16777215.0f to 16777216.0f, so skip it.
		TestEqualAfterVectorRounding(TEXT("VectorRound32-ZeroBitPrecision"), 16777215.5f, 16777216);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-ZeroBitPrecision"), 16777216.0f, 16777216);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-ZeroBitPrecision"), 16777216.5f, 16777216);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-ZeroBitPrecision"), 16777217.0f, 16777216);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-ZeroBitPrecision"), 16777217.5f, 16777218);
		//TestEqualAfterVectorRounding(TEXT("VectorRound32-NegZeroBitPrecision"), -16777215.0f, -16777215, bSkipScalar); // FMath::RoundHalfToEven incorrectly rounds -16777215.0f to -16777216.0f, so skip it.
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegZeroBitPrecision"), -16777215.5f, -16777216);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegZeroBitPrecision"), -16777216.0f, -16777216);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegZeroBitPrecision"), -16777216.5f, -16777216);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegZeroBitPrecision"), -16777217.0f, -16777216);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-NegZeroBitPrecision"), -16777217.5f, -16777218);

		TestEqualAfterVectorRounding(TEXT("VectorRound32-FloatMax"), -FLT_MAX, INT32_MIN);

		TestEqualAfterVectorRounding(TEXT("VectorRound32-FloatMin"), FLT_MIN, 0);
		TestEqualAfterVectorRounding(TEXT("VectorRound32-FloatMin"), -FLT_MIN, 0);
		return true;
	}
};

TEST_CASE_NAMED(FMathVectorRoundToIntHalfToEvenTests, "System::Core::Math::RoundToIntHalfToEven Vector", "[ApplicationContextMask][SmokeFilter]")
{
	FMathVectorRoundToIntHalfToEvenTestsClass Instance;
	Instance.MathVectorRoundToIntHalfToEvenTest();
}

TEST_CASE_NAMED(FMathRoundHalfToZeroTests, "System::Core::Math::Round HalfToZero", "[ApplicationContextMask][SmokeFilter]")
{
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-Zero"), FMath::RoundHalfToZero(0.0f) == 0.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-One"), FMath::RoundHalfToZero(1.0f) == 1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-LessHalf"), FMath::RoundHalfToZero(1.4f) == 1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegGreaterHalf"), FMath::RoundHalfToZero(-1.4f) == -1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-LessNearHalf"), FMath::RoundHalfToZero(1.4999999f) == 1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegGreaterNearHalf"), FMath::RoundHalfToZero(-1.4999999f) == -1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-Half"), FMath::RoundHalfToZero(1.5f) == 1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegHalf"), FMath::RoundHalfToZero(-1.5f) == -1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-GreaterNearHalf"), FMath::RoundHalfToZero(1.5000001f) == 2.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegLesserNearHalf"), FMath::RoundHalfToZero(-1.5000001f) == -2.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-GreaterThanHalf"), FMath::RoundHalfToZero(1.6f) == 2.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegLesserThanHalf"), FMath::RoundHalfToZero(-1.6f) == -2.0f);

	CHECK_MESSAGE(TEXT("RoundHalfToZero32-TwoToOneBitPrecision"), FMath::RoundHalfToZero(4194303.25f) == 4194303.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-TwoToOneBitPrecision"), FMath::RoundHalfToZero(4194303.5f) == 4194303.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-TwoToOneBitPrecision"), FMath::RoundHalfToZero(4194303.75f) == 4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-TwoToOneBitPrecision"), FMath::RoundHalfToZero(4194304.0f) == 4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-TwoToOneBitPrecision"), FMath::RoundHalfToZero(4194304.5f) == 4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegTwoToOneBitPrecision"), FMath::RoundHalfToZero(-4194303.25f) == -4194303.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegTwoToOneBitPrecision"), FMath::RoundHalfToZero(-4194303.5f) == -4194303.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegTwoToOneBitPrecision"), FMath::RoundHalfToZero(-4194303.75f) == -4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegTwoToOneBitPrecision"), FMath::RoundHalfToZero(-4194304.0f) == -4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegTwoToOneBitPrecision"), FMath::RoundHalfToZero(-4194304.5f) == -4194304.0f);

	CHECK_MESSAGE(TEXT("RoundHalfToZero32-OneToZeroBitPrecision"), FMath::RoundHalfToZero(8388607.0f) == 8388607.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-OneToZeroBitPrecision"), FMath::RoundHalfToZero(8388607.5f) == 8388607.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-OneToZeroBitPrecision"), FMath::RoundHalfToZero(8388608.0f) == 8388608.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegOneToZeroBitPrecision"), FMath::RoundHalfToZero(-8388607.0f) == -8388607.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegOneToZeroBitPrecision"), FMath::RoundHalfToZero(-8388607.5f) == -8388607.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegOneToZeroBitPrecision"), FMath::RoundHalfToZero(-8388608.0f) == -8388608.0f);

	CHECK_MESSAGE(TEXT("RoundHalfToZero32-ZeroBitPrecision"), FMath::RoundHalfToZero(16777215.0f) == 16777215.0f);
	CHECK_MESSAGE(TEXT("RoundHalfToZero32-NegZeroBitPrecision"), FMath::RoundHalfToZero(-16777215.0f) == -16777215.0f);

	CHECK_MESSAGE(TEXT("RoundHalfToZero64-Zero"), FMath::RoundHalfToZero(0.0) == 0.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-One"), FMath::RoundHalfToZero(1.0) == 1.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-LessHalf"), FMath::RoundHalfToZero(1.4) == 1.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegGreaterHalf"), FMath::RoundHalfToZero(-1.4) == -1.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-LessNearHalf"), FMath::RoundHalfToZero(1.4999999999999997) == 1.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegGreaterNearHalf"), FMath::RoundHalfToZero(-1.4999999999999997) == -1.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-Half"), FMath::RoundHalfToZero(1.5) == 1.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegHalf"), FMath::RoundHalfToZero(-1.5) == -1.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-GreaterNearHalf"), FMath::RoundHalfToZero(1.5000000000000002) == 2.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegLesserNearHalf"), FMath::RoundHalfToZero(-1.5000000000000002) == -2.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-GreaterThanHalf"), FMath::RoundHalfToZero(1.6) == 2.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegLesserThanHalf"), FMath::RoundHalfToZero(-1.6) == -2.0);

	CHECK_MESSAGE(TEXT("RoundHalfToZero64-TwoToOneBitPrecision"), FMath::RoundHalfToZero(2251799813685247.25) == 2251799813685247.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-TwoToOneBitPrecision"), FMath::RoundHalfToZero(2251799813685247.5) == 2251799813685247.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-TwoToOneBitPrecision"), FMath::RoundHalfToZero(2251799813685247.75) == 2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-TwoToOneBitPrecision"), FMath::RoundHalfToZero(2251799813685248.0) == 2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-TwoToOneBitPrecision"), FMath::RoundHalfToZero(2251799813685248.5) == 2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegTwoToOneBitPrecision"), FMath::RoundHalfToZero(-2251799813685247.25) == -2251799813685247.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegTwoToOneBitPrecision"), FMath::RoundHalfToZero(-2251799813685247.5) == -2251799813685247.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegTwoToOneBitPrecision"), FMath::RoundHalfToZero(-2251799813685247.75) == -2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegTwoToOneBitPrecision"), FMath::RoundHalfToZero(-2251799813685248.0) == -2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegTwoToOneBitPrecision"), FMath::RoundHalfToZero(-2251799813685248.5) == -2251799813685248.0);

	CHECK_MESSAGE(TEXT("RoundHalfToZero64-OneToZeroBitPrecision"), FMath::RoundHalfToZero(4503599627370495.0) == 4503599627370495.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-OneToZeroBitPrecision"), FMath::RoundHalfToZero(4503599627370495.5) == 4503599627370495.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-OneToZeroBitPrecision"), FMath::RoundHalfToZero(4503599627370496.0) == 4503599627370496.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegOneToZeroBitPrecision"), FMath::RoundHalfToZero(-4503599627370495.0) == -4503599627370495.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegOneToZeroBitPrecision"), FMath::RoundHalfToZero(-4503599627370495.5) == -4503599627370495.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegOneToZeroBitPrecision"), FMath::RoundHalfToZero(-4503599627370496.0) == -4503599627370496.0);

	CHECK_MESSAGE(TEXT("RoundHalfToZero64-ZeroBitPrecision"), FMath::RoundHalfToZero(9007199254740991.0) == 9007199254740991.0);
	CHECK_MESSAGE(TEXT("RoundHalfToZero64-NegZeroBitPrecision"), FMath::RoundHalfToZero(-9007199254740991.0) == -9007199254740991.0);
}

TEST_CASE_NAMED(FMathRoundHalfFromZeroTests, "System::Core::Math::Round HalfFromZero", "[ApplicationContextMask][SmokeFilter]")
{
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-Zero"), FMath::RoundHalfFromZero(0.0f) == 0.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-One"), FMath::RoundHalfFromZero(1.0f) == 1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-LessHalf"), FMath::RoundHalfFromZero(1.4f) == 1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegGreaterHalf"), FMath::RoundHalfFromZero(-1.4f) == -1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-LessNearHalf"), FMath::RoundHalfFromZero(1.4999999f) == 1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegGreaterNearHalf"), FMath::RoundHalfFromZero(-1.4999999f) == -1.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-Half"), FMath::RoundHalfFromZero(1.5f) == 2.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegHalf"), FMath::RoundHalfFromZero(-1.5f) == -2.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-LessGreaterNearHalf"), FMath::RoundHalfFromZero(1.5000001f) == 2.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegLesserNearHalf"), FMath::RoundHalfFromZero(-1.5000001f) == -2.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-GreaterThanHalf"), FMath::RoundHalfFromZero(1.6f) == 2.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegLesserThanHalf"), FMath::RoundHalfFromZero(-1.6f) == -2.0f);

	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-TwoToOneBitPrecision"), FMath::RoundHalfFromZero(4194303.25f) == 4194303.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-TwoToOneBitPrecision"), FMath::RoundHalfFromZero(4194303.5f) == 4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-TwoToOneBitPrecision"), FMath::RoundHalfFromZero(4194303.75f) == 4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-TwoToOneBitPrecision"), FMath::RoundHalfFromZero(4194304.0f) == 4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-TwoToOneBitPrecision"), FMath::RoundHalfFromZero(4194304.5f) == 4194305.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegTwoToOneBitPrecision"), FMath::RoundHalfFromZero(-4194303.25f) == -4194303.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegTwoToOneBitPrecision"), FMath::RoundHalfFromZero(-4194303.5f) == -4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegTwoToOneBitPrecision"), FMath::RoundHalfFromZero(-4194303.75f) == -4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegTwoToOneBitPrecision"), FMath::RoundHalfFromZero(-4194304.0f) == -4194304.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegTwoToOneBitPrecision"), FMath::RoundHalfFromZero(-4194304.5f) == -4194305.0f);

	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-OneToZeroBitPrecision"), FMath::RoundHalfFromZero(8388607.0f) == 8388607.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-OneToZeroBitPrecision"), FMath::RoundHalfFromZero(8388607.5f) == 8388608.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-OneToZeroBitPrecision"), FMath::RoundHalfFromZero(8388608.0f) == 8388608.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegOneToZeroBitPrecision"), FMath::RoundHalfFromZero(-8388607.0f) == -8388607.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegOneToZeroBitPrecision"), FMath::RoundHalfFromZero(-8388607.5f) == -8388608.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegOneToZeroBitPrecision"), FMath::RoundHalfFromZero(-8388608.0f) == -8388608.0f);

	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-ZeroBitPrecision"), FMath::RoundHalfToZero(16777215.0f) == 16777215.0f);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero32-NegZeroBitPrecision"), FMath::RoundHalfToZero(-16777215.0f) == -16777215.0f);

	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-Zero"), FMath::RoundHalfFromZero(0.0) == 0.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-One"), FMath::RoundHalfFromZero(1.0) == 1.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-LessHalf"), FMath::RoundHalfFromZero(1.4) == 1.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegGreaterHalf"), FMath::RoundHalfFromZero(-1.4) == -1.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-LessNearHalf"), FMath::RoundHalfFromZero(1.4999999999999997) == 1.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegGreaterNearHalf"), FMath::RoundHalfFromZero(-1.4999999999999997) == -1.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-Half"), FMath::RoundHalfFromZero(1.5) == 2.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegHalf"), FMath::RoundHalfFromZero(-1.5) == -2.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-LessGreaterNearHalf"), FMath::RoundHalfFromZero(1.5000000000000002) == 2.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegLesserNearHalf"), FMath::RoundHalfFromZero(-1.5000000000000002) == -2.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-GreaterThanHalf"), FMath::RoundHalfFromZero(1.6) == 2.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegLesserThanHalf"), FMath::RoundHalfFromZero(-1.6) == -2.0);

	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-TwoToOneBitPrecision"), FMath::RoundHalfFromZero(2251799813685247.25) == 2251799813685247.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-TwoToOneBitPrecision"), FMath::RoundHalfFromZero(2251799813685247.5) == 2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-TwoToOneBitPrecision"), FMath::RoundHalfFromZero(2251799813685247.75) == 2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-TwoToOneBitPrecision"), FMath::RoundHalfFromZero(2251799813685248.0) == 2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-TwoToOneBitPrecision"), FMath::RoundHalfFromZero(2251799813685248.5) == 2251799813685249.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegTwoToOneBitPrecision"), FMath::RoundHalfFromZero(-2251799813685247.25) == -2251799813685247.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegTwoToOneBitPrecision"), FMath::RoundHalfFromZero(-2251799813685247.5) == -2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegTwoToOneBitPrecision"), FMath::RoundHalfFromZero(-2251799813685247.75) == -2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegTwoToOneBitPrecision"), FMath::RoundHalfFromZero(-2251799813685248.0) == -2251799813685248.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegTwoToOneBitPrecision"), FMath::RoundHalfFromZero(-2251799813685248.5) == -2251799813685249.0);

	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-OneToZeroBitPrecision"), FMath::RoundHalfFromZero(4503599627370495.0) == 4503599627370495.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-OneToZeroBitPrecision"), FMath::RoundHalfFromZero(4503599627370495.5) == 4503599627370496.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-OneToZeroBitPrecision"), FMath::RoundHalfFromZero(4503599627370496.0) == 4503599627370496.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegOneToZeroBitPrecision"), FMath::RoundHalfFromZero(-4503599627370495.0) == -4503599627370495.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegOneToZeroBitPrecision"), FMath::RoundHalfFromZero(-4503599627370495.5) == -4503599627370496.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegOneToZeroBitPrecision"), FMath::RoundHalfFromZero(-4503599627370496.0) == -4503599627370496.0);

	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-ZeroBitPrecision"), FMath::RoundHalfToZero(9007199254740991.0) == 9007199254740991.0);
	CHECK_MESSAGE(TEXT("RoundHalfFromZero64-NegZeroBitPrecision"), FMath::RoundHalfToZero(-9007199254740991.0) == -9007199254740991.0);
}

class FIsNearlyEqualByULPTestClass {
public:
	void CheckMessage(const FString& What, bool Value)
	{
		CHECK_MESSAGE(*What, Value);
	}

	void CheckFalseMessage(const FString& What, bool Value)
	{
		CHECK_FALSE_MESSAGE(*What, Value);
	}

	bool IsNearlyEqualByULPTest() {
		static const float FloatNan = std::numeric_limits<float>::quiet_NaN();
		static const double DoubleNan = std::numeric_limits<double>::quiet_NaN();

		static const float FloatInf = std::numeric_limits<float>::infinity();
		static const double DoubleInf = std::numeric_limits<double>::infinity();

		float FloatTrueMin;
		double DoubleTrueMin;

		// Construct our own true minimum float constants (aka std::numeric_limits<float>::denorm_min), 
		// to ensure we don't get caught by any application or system-wide flush-to-zero or 
		// denormals-are-zero settings.
		{
			uint32 FloatTrueMinInt = 0x00000001U;
			uint64 DoubleTrueMinInt = 0x0000000000000001ULL;

			::memcpy(&FloatTrueMin, &FloatTrueMinInt, sizeof(FloatTrueMinInt));
			::memcpy(&DoubleTrueMin, &DoubleTrueMinInt, sizeof(DoubleTrueMinInt));
		}


		static struct TestItem
		{
			const FString& Name;
			bool Predicate;
			struct
			{
				float A;
				float B;
			} F;
			struct
			{
				double A;
				double B;
			} D;

			int ULP = 4;
		} TestItems[] = {
			{"ZeroEqual",		true, {0.0f, 0.0f}, {0.0, 0.0}},
			{"OneEqual",		true, {1.0f, 1.0f}, {1.0, 1.0}},
			{"MinusOneEqual",	true, {-1.0f, -1.0f}, {-1.0, -1.0}},
			{"PlusMinusOneNotEqual", false, {-1.0f, 1.0f}, {-1.0, 1.0}},

			{"NanEqualFail",	false, {FloatNan, FloatNan}, {DoubleNan, DoubleNan}},

			// FLT_EPSILON is the smallest quantity that can be added to 1.0 and still be considered a distinct number
			{"OneULPDistUp",	true, {1.0f, 1.0f + FLT_EPSILON}, {1.0, 1.0 + DBL_EPSILON}, 1},

			// Going below one, we need to halve the epsilon, since the exponent has been lowered by one and hence the 
			// numerical density doubles between 0.5 and 1.0.
			{"OneULPDistDown",	true, {1.0f, 1.0f - (FLT_EPSILON / 2.0f)}, {1.0, 1.0 - (DBL_EPSILON / 2.0)}, 1},

			// Make sure the ULP distance is computed correctly for double epsilon.
			{"TwoULPDist",		true, {1.0f, 1.0f + 2 * FLT_EPSILON}, {1.0, 1.0 + 2 * DBL_EPSILON}, 2},
			{"TwoULPDistFail",	false, {1.0f, 1.0f + 2 * FLT_EPSILON}, {1.0, 1.0 + 2 * DBL_EPSILON}, 1},

			// Check if the same test works for higher exponents on both sides.
			{"ONeULPDistEight",	true, {8.0f, 8.0f + 8.0f * FLT_EPSILON}, {8.0, 8.0 + 8.0 * DBL_EPSILON}, 1},
			{"ONeULPDistFailEight",	false, {8.0f, 8.0f + 16.0f * FLT_EPSILON}, {8.0, 8.0 + 16.0 * DBL_EPSILON}, 1},

			// Test for values around the zero point.
			{"AroundZero",		true, {-FloatTrueMin, FloatTrueMin}, {-DoubleTrueMin, DoubleTrueMin}, 2},
			{"AroundZeroFail",	false, {-FloatTrueMin, FloatTrueMin}, {-DoubleTrueMin, DoubleTrueMin}, 1},

			// Test for values close to zero and zero.
			{"PosNextToZero",	true, {0, FloatTrueMin}, {0, DoubleTrueMin}, 1},
			{"NegNextToZero",	true, {-FloatTrueMin, 0}, {-DoubleTrueMin, 0}, 1},

			// Should fail, even for maximum ULP distance.
			{"InfAndMaxFail",	false, {FLT_MAX, FloatInf}, {DBL_MAX, DoubleInf}, INT32_MAX},
			{"InfAndNegInfFail", false, {-FloatInf, FloatInf}, {-DoubleInf, DoubleInf}, INT32_MAX},

			// Two infinities of the same sign should compare the same, regardless of ULP.
			{"InfAndInf",		true, {FloatInf, FloatInf}, {DoubleInf, DoubleInf}, 0},

		};

		void(FIsNearlyEqualByULPTestClass::*FuncTrue)(const FString&, bool) = &FIsNearlyEqualByULPTestClass::CheckMessage;
		void(FIsNearlyEqualByULPTestClass::*FuncFalse)(const FString&, bool) = &FIsNearlyEqualByULPTestClass::CheckFalseMessage;

		for (const TestItem& Item : TestItems)
		{
			auto Func = Item.Predicate ? FuncTrue : FuncFalse;

			(this->*Func)(Item.Name + "-Float", FMath::IsNearlyEqualByULP(Item.F.A, Item.F.B, Item.ULP));
			(this->*Func)(Item.Name + "-Double", FMath::IsNearlyEqualByULP(Item.D.A, Item.D.B, Item.ULP));
		}
		return true;
	}
};

// ConstExprCeilLogTwo
static_assert(FMath::ConstExprCeilLogTwo(0) == 0, "0 -> 0");
static_assert(FMath::ConstExprCeilLogTwo(1) == 0, "1 -> 0");
static_assert(FMath::ConstExprCeilLogTwo(2) == 1, "2 -> 1");
static_assert(FMath::ConstExprCeilLogTwo(3) == 2, "3 -> 2");
static_assert(FMath::ConstExprCeilLogTwo(4) == 2, "4 -> 2");
static_assert(FMath::ConstExprCeilLogTwo(5) == 3, "5 -> 3");
static_assert(FMath::ConstExprCeilLogTwo(6) == 3, "6 -> 3");
static_assert(FMath::ConstExprCeilLogTwo(7) == 3, "7 -> 3");
static_assert(FMath::ConstExprCeilLogTwo(8) == 3, "8 -> 3");
static_assert(FMath::ConstExprCeilLogTwo(16) == 4, "16 -> 4");
static_assert(FMath::ConstExprCeilLogTwo(17) == 5, "17 -> 5");
static_assert(FMath::ConstExprCeilLogTwo(31) == 5, "31 -> 5");
static_assert(FMath::ConstExprCeilLogTwo(32) == 5, "32 -> 5");
static_assert(FMath::ConstExprCeilLogTwo(33) == 6, "33 -> 6");
static_assert(FMath::ConstExprCeilLogTwo(1U << 16) == 16, "2^16 -> 16");
static_assert(FMath::ConstExprCeilLogTwo(uint64(1) << 31) == 31, "2^31 -> 31");
static_assert(FMath::ConstExprCeilLogTwo((uint64(1) << 32) - 1) == 32, "2^32 - 1 -> 32");
static_assert(FMath::ConstExprCeilLogTwo(uint64(1) << 32) == 32, "2^32 -> 32");
static_assert(FMath::ConstExprCeilLogTwo(uint64(0xfffffffffffffffe)) == 64, "2^64 - 2 -> 64");
static_assert(FMath::ConstExprCeilLogTwo(uint64(0xffffffffffffffff)) == 64, "2^64 - 1 -> 64");



TEST_CASE_NAMED(FIsNearlyEqualByULPTest, "System::Core::Math::IsNearlyEqualByULP", "[ApplicationContextMask][SmokeFilter]")
{
	FIsNearlyEqualByULPTestClass Instance;
	Instance.IsNearlyEqualByULPTest();
}

TEST_CASE_NAMED(FMathTruncationTests, "System::Core::Math::TruncationFunctions", "[ApplicationContextMask][EngineFilter]")
{
	// Float: 1-bit Sign, 8-bit exponent, 23-bit mantissa, implicit 1
	float FloatTestCases[][5]{
		//Value				Trunc				Ceil				Floor				Round		
		{-1.5f,				-1.0f,				-1.0f,				-2.0f,				-1.0f,				}, // We do not use round half to even, we always round .5 up (towards +inf)
		{-1.0f,				-1.0f,				-1.0f,				-1.0f,				-1.0f,				},
		{-0.75f,			-0.0f,				-0.0f,				-1.0f,				-1.0f,				},
		{-0.5f,				-0.0f,				-0.0f,				-1.0f,				-0.0f,				}, // We do not use round half to even, we always round .5 up (towards +inf)
		{-0.25f,			-0.0f,				-0.0f,				-1.0f,				-0.0f,				},
		{0.0f,				0.0f,				0.0f,				0.0f,				0.0f,				},
		{0.25f,				0.0f,				1.0f,				0.0f,				0.0f,				},
		{0.5f,				0.0f,				1.0f,				0.0f,				1.0f,				}, // We do not use round half to even, we always round .5 up (towards +inf)
		{0.75f,				0.0f,				1.0f,				0.0f,				1.0f,				},
		{1.0f,				1.0f,				1.0f,				1.0f,				1.0f,				},
		{1.5f,				1.0f,				2.0f,				1.0f,				2.0f,				},
		{17179869184.0f,	17179869184.0f,		17179869184.0f,		17179869184.0f,		17179869184.0f,		}, // 2^34.  Note that 2^34 + 1 is not representable, but 2^34 is the string of bits 0, 00100010, 10000000000000000000000,
		{-17179869184.0f,	-17179869184.0f,	-17179869184.0f,	-17179869184.0f,	-17179869184.0f,	}, // -2^34
		{1048576.6f,		1048576.0f,			1048577.0f,			1048576.0f,			1048577.0f,			}, // 2^20 + 0.6
		{-1048576.6f,		-1048576.0f,		-1048576.0f,		-1048577.0f,		-1048577.0f,		}, // -2^20 - 0.6
	};
	int IntTestCases[][4]{
		//					Trunc				Ceil				Floor				Round
		{					-1,					-1,					-2,					-1,					},
		{					-1,					-1,					-1,					-1,					},
		{					0,					0,					-1,					-1,					},
		{					0,					0,					-1,					0,					},
		{					0,					0,					-1,					0,					},
		{					0,					0,					0,					0,					},
		{					0,					1,					0,					0,					},
		{					0,					1,					0,					1,					},
		{					0,					1,					0,					1,					},
		{					1,					1,					1,					1,					},
		{					1,					2,					1,					2,					},
		{					0,					0,					0,					0,					}, // undefined, > MAX_INT32
		{					0,					0,					0,					0,					}, // undefined, < MIN_INT32
		{					1048576,			1048577,			1048576,			1048577,			},
		{					-1048576,			-1048576,			-1048577,			-1048577,			},
	};
	static_assert(UE_ARRAY_COUNT(FloatTestCases) == UE_ARRAY_COUNT(IntTestCases), "IntTestCases use the value from FloatTestCases and must be the same length");

	TCHAR TestNameBuffer[128];
	auto SubTestName = [&TestNameBuffer](const TCHAR* FunctionName, double Input) {
		FCString::Snprintf(TestNameBuffer, UE_ARRAY_COUNT(TestNameBuffer), TEXT("%s(%lf)"), FunctionName, Input);
		return TestNameBuffer;
	};

	for (uint32 TestCaseIndex = 0; TestCaseIndex < UE_ARRAY_COUNT(FloatTestCases); TestCaseIndex++)
	{
		float* FloatValues = FloatTestCases[TestCaseIndex];
		float Input = FloatValues[0];

		CHECK_MESSAGE(SubTestName(TEXT("TruncToFloat"), Input), FMath::TruncToFloat(Input) == FloatValues[1]);
		CHECK_MESSAGE(SubTestName(TEXT("CeilToFloat"), Input), FMath::CeilToFloat(Input) == FloatValues[2]);
		CHECK_MESSAGE(SubTestName(TEXT("FloorToFloat"), Input), FMath::FloorToFloat(Input) == FloatValues[3]);
		CHECK_MESSAGE(SubTestName(TEXT("RoundToFloat"), Input), FMath::RoundToFloat(Input) == FloatValues[4]);

		int* IntValues = IntTestCases[TestCaseIndex];
		if ((float)MIN_int32 <= Input && Input <= (float)MAX_int32)
		{
			CHECK_MESSAGE(SubTestName(TEXT("TruncToInt"), Input), FMath::TruncToInt(Input) == IntValues[0]);
			CHECK_MESSAGE(SubTestName(TEXT("CeilToInt"), Input), FMath::CeilToInt(Input) == IntValues[1]);
			CHECK_MESSAGE(SubTestName(TEXT("FloorToInt"), Input), FMath::FloorToInt(Input) == IntValues[2]);
			CHECK_MESSAGE(SubTestName(TEXT("RoundToInt"), Input), FMath::RoundToInt(Input) == IntValues[3]);
		}
	}

	// Double: 1-bit sign, 11-bit exponent, 52-bit mantissa, implicit 1
	double DoubleTestCases[][5]{
		//Value						Trunc					Ceil					Floor					Round		
		{-1.5,						-1.0,					-1.0,					-2.0,					-1.0,					}, // We do not use round half to even, we always round .5 up (towards +inf)
		{-1.0,						-1.0,					-1.0,					-1.0,					-1.0,					},
		{-0.75,						-0.0,					-0.0,					-1.0,					-1.0,					},
		{-0.5,						-0.0,					-0.0,					-1.0,					-0.0,					}, // We do not use round half to even, we always round .5 up (towards +inf)
		{-0.25,						-0.0,					-0.0,					-1.0,					-0.0,					},
		{0.0,						0.0,					0.0,					0.0,					0.0,					},
		{0.25,						0.0,					1.0,					0.0,					0.0,					},
		{0.5,						0.0,					1.0,					0.0,					1.0,					}, // We do not use round half to even, we always round .5 up (towards +inf)
		{0.75,						0.0,					1.0,					0.0,					1.0,					},
		{1.0,						1.0,					1.0,					1.0,					1.0,					},
		{1.5,						1.0,					2.0,					1.0,					2.0,					},
		{17179869184.0,				17179869184.0,			17179869184.0,			17179869184.0,			17179869184.0			}, // 2^34
		{-17179869184.0,			-17179869184.0,			-17179869184.0,			-17179869184.0,			-17179869184.0			},
		{1048576.6,					1048576.0,				1048577.0,				1048576.0,				1048577.0,				},
		{-1048576.6,				-1048576.0,				-1048576.0,				-1048577.0,				-1048577.0,				},
		{73786976294838206464.,		73786976294838206464.,	73786976294838206464.,	73786976294838206464.,	73786976294838206464.	}, // 2^66
		{-73786976294838206464.,	-73786976294838206464.,	-73786976294838206464.,	-73786976294838206464.,	-73786976294838206464.	},
		{281474976710656.6,			281474976710656.0,		281474976710657.0,		281474976710656.0,		281474976710657.0		}, // 2^48 + 0.6
		{-281474976710656.6,		-281474976710656.0,		-281474976710656.0,		-281474976710657.0,		-281474976710657.0		},
	};

	for (uint32 TestCaseIndex = 0; TestCaseIndex < UE_ARRAY_COUNT(DoubleTestCases); TestCaseIndex++)
	{
		double* DoubleValues = DoubleTestCases[TestCaseIndex];
		double Input = DoubleValues[0];

		CHECK_MESSAGE(SubTestName(TEXT("TruncToDouble"), Input), FMath::TruncToDouble(Input) == DoubleValues[1]);
		CHECK_MESSAGE(SubTestName(TEXT("CeilToDouble"), Input), FMath::CeilToDouble(Input) == DoubleValues[2]);
		CHECK_MESSAGE(SubTestName(TEXT("FloorToDouble"), Input), FMath::FloorToDouble(Input) == DoubleValues[3]);
		CHECK_MESSAGE(SubTestName(TEXT("RoundToDouble"), Input), FMath::RoundToDouble(Input) == DoubleValues[4]);
	}

#define MATH_TRUNCATION_SPEED_TEST
#ifdef MATH_TRUNCATION_SPEED_TEST
	volatile static float ForceCompileFloat;
	volatile static int ForceCompileInt;

	auto TimeIt = [](const TCHAR* SubFunctionName, float(*ComputeMath)(float Input), float(*ComputeGeneric)(float Input))
	{
		double StartTime = FPlatformTime::Seconds();
		const float StartInput = 0.6f;
		const float NumTrials = 10.f * 1000.f * 1000.f;
		const float MicroSecondsPerSecond = 1000 * 1000.f;
		for (float Input = StartInput; Input < NumTrials; Input += 1.0f)
		{
			ForceCompileFloat += ComputeMath(Input);
		}
		double EndTime = FPlatformTime::Seconds();
		double FMathDuration = EndTime - StartTime;
		StartTime = FPlatformTime::Seconds();
		for (float Input = 0.6f; Input < 10 * 1000 * 1000; Input += 1.0f)
		{
			ForceCompileFloat += ComputeGeneric(Input);
		}
		EndTime = FPlatformTime::Seconds();
		double GenericDuration = EndTime - StartTime;

		UE_LOGF(LogInit, Log, "%ls: FMath time: %lfus, Generic: %lfus", SubFunctionName, FMathDuration * MicroSecondsPerSecond / NumTrials, GenericDuration * MicroSecondsPerSecond / NumTrials);
	};

	TimeIt(TEXT("TruncToInt"), [](float Input) { return (float)FMath::TruncToInt(Input); }, [](float Input) { return (float)FGenericPlatformMath::TruncToInt(Input); });
	TimeIt(TEXT("CeilToInt"), [](float Input) { return (float)FMath::CeilToInt(Input); }, [](float Input) { return (float)FGenericPlatformMath::CeilToInt(Input); });
	TimeIt(TEXT("FloorToInt"), [](float Input) { return (float)FMath::FloorToInt(Input); }, [](float Input) { return (float)FGenericPlatformMath::FloorToInt(Input); });
	TimeIt(TEXT("RoundToInt"), [](float Input) { return (float)FMath::RoundToInt(Input); }, [](float Input) { return (float)FGenericPlatformMath::RoundToInt(Input); });

	TimeIt(TEXT("TruncToFloat"), [](float Input) { return FMath::TruncToFloat(Input); }, [](float Input) { return FGenericPlatformMath::TruncToFloat(Input); });
	TimeIt(TEXT("CeilToFloat"), [](float Input) { return FMath::CeilToFloat(Input); }, [](float Input) { return FGenericPlatformMath::CeilToFloat(Input); });
	TimeIt(TEXT("FloorToFloat"), [](float Input) { return FMath::FloorToFloat(Input); }, [](float Input) { return FGenericPlatformMath::FloorToFloat(Input); });
	TimeIt(TEXT("RoundToFloat"), [](float Input) { return FMath::RoundToFloat(Input); }, [](float Input) { return FGenericPlatformMath::RoundToFloat(Input); });

	TimeIt(TEXT("TruncToDouble"), [](float Input) { return (float)FMath::TruncToDouble((double)Input); }, [](float Input) { return (float)FGenericPlatformMath::TruncToDouble((double)Input); });
	TimeIt(TEXT("CeilToDouble"), [](float Input) { return (float)FMath::CeilToDouble((double)Input); }, [](float Input) { return (float)FGenericPlatformMath::CeilToDouble((double)Input); });
	TimeIt(TEXT("FloorToDouble"), [](float Input) { return (float)FMath::FloorToDouble((double)Input); }, [](float Input) { return (float)FGenericPlatformMath::FloorToDouble((double)Input); });
	TimeIt(TEXT("RoundToDouble"), [](float Input) { return (float)FMath::RoundToDouble((double)Input); }, [](float Input) { return (float)FGenericPlatformMath::RoundToDouble((double)Input); });
#endif
}

TEST_CASE_NAMED(FMathIntegerTests, "System::Core::Math::IntegerFunctions", "[ApplicationContextMask][SmokeFilter]")
{
	// Test CountLeadingZeros8
	CHECK_MESSAGE(TEXT("CountLeadingZeros8(0)"), FMath::CountLeadingZeros8(0) == 8);
	CHECK_MESSAGE(TEXT("CountLeadingZeros8(1)"), FMath::CountLeadingZeros8(1) == 7);
	CHECK_MESSAGE(TEXT("CountLeadingZeros8(2)"), FMath::CountLeadingZeros8(2) == 6);
	CHECK_MESSAGE(TEXT("CountLeadingZeros8(0x7f)"), FMath::CountLeadingZeros8(0x7f) == 1);
	CHECK_MESSAGE(TEXT("CountLeadingZeros8(0x80)"), FMath::CountLeadingZeros8(0x80) == 0);
	CHECK_MESSAGE(TEXT("CountLeadingZeros8(0xff)"), FMath::CountLeadingZeros8(0xff) == 0);

	// Test CountLeadingZeros
	CHECK_MESSAGE(TEXT("CountLeadingZeros(0)"), FMath::CountLeadingZeros(0) == 32);
	CHECK_MESSAGE(TEXT("CountLeadingZeros(1)"), FMath::CountLeadingZeros(1) == 31);
	CHECK_MESSAGE(TEXT("CountLeadingZeros(2)"), FMath::CountLeadingZeros(2) == 30);
	CHECK_MESSAGE(TEXT("CountLeadingZeros(0x7fffffff)"), FMath::CountLeadingZeros(0x7fffffff) == 1);
	CHECK_MESSAGE(TEXT("CountLeadingZeros(0x80000000)"), FMath::CountLeadingZeros(0x80000000) == 0);
	CHECK_MESSAGE(TEXT("CountLeadingZeros(0xffffffff)"), FMath::CountLeadingZeros(0xffffffff) == 0);

	// Test CountLeadingZeros64
	CHECK_MESSAGE(TEXT("CountLeadingZeros64(0)"), FMath::CountLeadingZeros64(0) == uint64(64));
	CHECK_MESSAGE(TEXT("CountLeadingZeros64(1)"), FMath::CountLeadingZeros64(1) == uint64(63));
	CHECK_MESSAGE(TEXT("CountLeadingZeros64(2)"), FMath::CountLeadingZeros64(2) == uint64(62));
	CHECK_MESSAGE(TEXT("CountLeadingZeros64(0x7fffffff'ffffffff)"), FMath::CountLeadingZeros64(0x7fffffff'ffffffff) == uint64(1));
	CHECK_MESSAGE(TEXT("CountLeadingZeros64(0x80000000'00000000)"), FMath::CountLeadingZeros64(0x80000000'00000000) == uint64(0));
	CHECK_MESSAGE(TEXT("CountLeadingZeros64(0xffffffff'ffffffff)"), FMath::CountLeadingZeros64(0xffffffff'ffffffff) == uint64(0));

	// Test FloorLog2
	CHECK_MESSAGE(TEXT("FloorLog2(0)"), FMath::FloorLog2(0) == 0);
	CHECK_MESSAGE(TEXT("FloorLog2(1)"), FMath::FloorLog2(1) == 0);
	CHECK_MESSAGE(TEXT("FloorLog2(2)"), FMath::FloorLog2(2) == 1);
	CHECK_MESSAGE(TEXT("FloorLog2(3)"), FMath::FloorLog2(3) == 1);
	CHECK_MESSAGE(TEXT("FloorLog2(4)"), FMath::FloorLog2(4) == 2);
	CHECK_MESSAGE(TEXT("FloorLog2(0x7fffffff)"), FMath::FloorLog2(0x7fffffff) == 30);
	CHECK_MESSAGE(TEXT("FloorLog2(0x80000000)"), FMath::FloorLog2(0x80000000) == 31);
	CHECK_MESSAGE(TEXT("FloorLog2(0xffffffff)"), FMath::FloorLog2(0xffffffff) == 31);

	// Test FloorLog2_64
	CHECK_MESSAGE(TEXT("FloorLog2_64(0)"), FMath::FloorLog2_64(0) == uint64(0));
	CHECK_MESSAGE(TEXT("FloorLog2_64(1)"), FMath::FloorLog2_64(1) == uint64(0));
	CHECK_MESSAGE(TEXT("FloorLog2_64(2)"), FMath::FloorLog2_64(2) == uint64(1));
	CHECK_MESSAGE(TEXT("FloorLog2_64(3)"), FMath::FloorLog2_64(3) == uint64(1));
	CHECK_MESSAGE(TEXT("FloorLog2_64(4)"), FMath::FloorLog2_64(4) == uint64(2));
	CHECK_MESSAGE(TEXT("FloorLog2_64(0x7fffffff)"), FMath::FloorLog2_64(0x7fffffff) == uint64(30));
	CHECK_MESSAGE(TEXT("FloorLog2_64(0x80000000)"), FMath::FloorLog2_64(0x80000000) == uint64(31));
	CHECK_MESSAGE(TEXT("FloorLog2_64(0xffffffff)"), FMath::FloorLog2_64(0xffffffff) == uint64(31));
	CHECK_MESSAGE(TEXT("FloorLog2_64(0x7fffffff'ffffffff)"), FMath::FloorLog2_64(0x7fffffff'ffffffff) == uint64(62));
	CHECK_MESSAGE(TEXT("FloorLog2_64(0x80000000'00000000)"), FMath::FloorLog2_64(0x80000000'00000000) == uint64(63));
	CHECK_MESSAGE(TEXT("FloorLog2_64(0xffffffff'ffffffff)"), FMath::FloorLog2_64(0xffffffff'ffffffff) == uint64(63));
}

TEST_CASE_NAMED(FNanInfVerificationTest, "System::Core::Math::NaNandInfTest", "[ApplicationContextMask][SmokeFilter]")
{
	static float FloatNan = FMath::Sqrt(-1.0f);
	static double DoubleNan = double(FloatNan);

	static float FloatInf = 1.0f / 0.0f;
	static double DoubleInf = 1.0 / 0.0;

	static float FloatStdNan = std::numeric_limits<float>::quiet_NaN();
	static double DoubleStdNan = std::numeric_limits<double>::quiet_NaN();

	static float FloatStdInf = std::numeric_limits<float>::infinity();
	static double DoubleStdInf = std::numeric_limits<double>::infinity();

	static double DoubleMax = std::numeric_limits<double>::max();
	static float FloatMax = std::numeric_limits<float>::max();

	CHECK_MESSAGE(TEXT("HasQuietNaNFloat"), std::numeric_limits<float>::has_quiet_NaN);
	CHECK_MESSAGE(TEXT("HasQuietNaNDouble"), std::numeric_limits<double>::has_quiet_NaN);
	CHECK_MESSAGE(TEXT("HasInfinityFloat"), std::numeric_limits<float>::has_infinity);
	CHECK_MESSAGE(TEXT("HasInfinityDouble"), std::numeric_limits<double>::has_infinity);

	CHECK_MESSAGE(TEXT("SqrtNegOneIsNanFloat"), std::isnan(FloatNan));
	CHECK_MESSAGE(TEXT("SqrtNegOneIsNanDouble"), std::isnan(DoubleNan));
	CHECK_MESSAGE(TEXT("OneOverZeroIsInfFloat"), !std::isfinite(FloatInf) && !std::isnan(FloatInf));
	CHECK_MESSAGE(TEXT("OneOverZeroIsInfDouble"), !std::isfinite(DoubleInf) && !std::isnan(DoubleInf));

	CHECK_MESSAGE(TEXT("UE4IsNanTrueFloat"), FPlatformMath::IsNaN(FloatNan));
	CHECK_MESSAGE(TEXT("UE4IsNanFalseFloat"), !FPlatformMath::IsNaN(0.0f));
	CHECK_MESSAGE(TEXT("UE4IsNanTrueDouble"), FPlatformMath::IsNaN(DoubleNan));
	CHECK_MESSAGE(TEXT("UE4IsNanFalseDouble"), !FPlatformMath::IsNaN(0.0));

	CHECK_MESSAGE(TEXT("UE4IsFiniteTrueFloat"), FPlatformMath::IsFinite(0.0f) && !FPlatformMath::IsNaN(0.0f));
	CHECK_MESSAGE(TEXT("UE4IsFiniteFalseFloat"), !FPlatformMath::IsFinite(FloatInf) && !FPlatformMath::IsNaN(FloatInf));
	CHECK_MESSAGE(TEXT("UE4IsFiniteTrueDouble"), FPlatformMath::IsFinite(0.0) && !FPlatformMath::IsNaN(0.0));
	CHECK_MESSAGE(TEXT("UE4IsFiniteFalseDouble"), !FPlatformMath::IsFinite(DoubleInf) && !FPlatformMath::IsNaN(DoubleInf));

	CHECK_MESSAGE(TEXT("UE4IsNanStdFloat"), FPlatformMath::IsNaN(FloatStdNan));
	CHECK_MESSAGE(TEXT("UE4IsNanStdDouble"), FPlatformMath::IsNaN(DoubleStdNan));

	CHECK_MESSAGE(TEXT("UE4IsFiniteStdFloat"), !FPlatformMath::IsFinite(FloatStdInf) && !FPlatformMath::IsNaN(FloatStdInf));
	CHECK_MESSAGE(TEXT("UE4IsFiniteStdDouble"), !FPlatformMath::IsFinite(DoubleStdInf) && !FPlatformMath::IsNaN(DoubleStdInf));

	// test for Mac/Linux regression where IsFinite did not have a double equivalent so would downcast to a float and return INF.
	CHECK_MESSAGE(TEXT("UE4IsFiniteDoubleMax"), FPlatformMath::IsFinite(DoubleMax) && !FPlatformMath::IsNaN(DoubleMax));
	CHECK_MESSAGE(TEXT("UE4IsFiniteFloatMax"), FPlatformMath::IsFinite(FloatMax) && !FPlatformMath::IsNaN(FloatMax));
}

TEST_CASE_NAMED(FBitCastTest, "System::Core::Math::Bitcast", "[ApplicationContextMask][SmokeFilter]")
{
	CHECK_MESSAGE(TEXT("CastFloatToInt32_0"), FPlatformMath::AsUInt(0.0f) == 0x00000000U);
	CHECK_MESSAGE(TEXT("CastFloatToInt32_P1"), FPlatformMath::AsUInt(+1.0f) == 0x3f800000U);
	CHECK_MESSAGE(TEXT("CastFloatToInt32_N1"), FPlatformMath::AsUInt(-1.0f) == 0xbf800000U);

	CHECK_MESSAGE(TEXT("CastFloatToInt64_0"), FPlatformMath::AsUInt(0.0) == 0x0000000000000000ULL);
	CHECK_MESSAGE(TEXT("CastFloatToInt64_P1"), FPlatformMath::AsUInt(+1.0) == 0x3ff0000000000000ULL);
	CHECK_MESSAGE(TEXT("CastFloatToInt64_N1"), FPlatformMath::AsUInt(-1.0) == 0xbff0000000000000ULL);

	CHECK_MESSAGE(TEXT("CastIntToFloat32_0"), FPlatformMath::AsFloat(static_cast<uint32>(0x00000000U)) == 0.0f);
	CHECK_MESSAGE(TEXT("CastIntToFloat32_P1"), FPlatformMath::AsFloat(static_cast<uint32>(0x3f800000U)) == +1.0f);
	CHECK_MESSAGE(TEXT("CastIntToFloat32_N1"), FPlatformMath::AsFloat(static_cast<uint32>(0xbf800000U)) == -1.0f);

	CHECK_MESSAGE(TEXT("CastIntToFloat64_0"), FPlatformMath::AsFloat(static_cast<uint64>(0x0000000000000000ULL)) == 0.0);
	CHECK_MESSAGE(TEXT("CastIntToFloat64_P1"), FPlatformMath::AsFloat(static_cast<uint64>(0x3ff0000000000000ULL)) == +1.0);
	CHECK_MESSAGE(TEXT("CastIntToFloat64_N1"), FPlatformMath::AsFloat(static_cast<uint64>(0xbff0000000000000ULL)) == -1.0);
}

TEST_CASE_NAMED(FMathWrapTest, "System::Core::Math::Wrap", "[ApplicationContextMask][SmokeFilter]")
{
	// Floats (with integral values)
	CHECK(FMath::Wrap(-17.0f, 10.0f, 20.0f) == 13.0f);
	CHECK(FMath::Wrap(-1.0f,  10.0f, 20.0f) == 19.0f);
	CHECK(FMath::Wrap(0.0f,   10.0f, 20.0f) == 10.0f);
	CHECK(FMath::Wrap(5.0f,   10.0f, 20.0f) == 15.0f);
	CHECK(FMath::Wrap(9.0f,   10.0f, 20.0f) == 19.0f);
	CHECK(FMath::Wrap(10.0f,  10.0f, 20.0f) == 10.0f);
	CHECK(FMath::Wrap(15.0f,  10.0f, 20.0f) == 15.0f);
	CHECK(FMath::Wrap(19.0f,  10.0f, 20.0f) == 19.0f);
	CHECK(FMath::Wrap(20.0f,  10.0f, 20.0f) == 20.0f);
	CHECK(FMath::Wrap(21.0f,  10.0f, 20.0f) == 11.0f);
	CHECK(FMath::Wrap(25.0f,  10.0f, 20.0f) == 15.0f);
	CHECK(FMath::Wrap(29.0f,  10.0f, 20.0f) == 19.0f);
	CHECK(FMath::Wrap(30.0f,  10.0f, 20.0f) == 20.0f);
	CHECK(FMath::Wrap(54.0f,  10.0f, 20.0f) == 14.0f);
	CHECK(FMath::Wrap(60.0f,  10.0f, 20.0f) == 20.0f);

	// Doubles (with integral values)
	CHECK(FMath::Wrap(-17.0, 10.0, 20.0) == 13.0);
	CHECK(FMath::Wrap( -1.0, 10.0, 20.0) == 19.0);
	CHECK(FMath::Wrap(  0.0, 10.0, 20.0) == 10.0);
	CHECK(FMath::Wrap(  5.0, 10.0, 20.0) == 15.0);
	CHECK(FMath::Wrap(  9.0, 10.0, 20.0) == 19.0);
	CHECK(FMath::Wrap( 10.0, 10.0, 20.0) == 10.0);
	CHECK(FMath::Wrap( 15.0, 10.0, 20.0) == 15.0);
	CHECK(FMath::Wrap( 19.0, 10.0, 20.0) == 19.0);
	CHECK(FMath::Wrap( 20.0, 10.0, 20.0) == 20.0);
	CHECK(FMath::Wrap( 21.0, 10.0, 20.0) == 11.0);
	CHECK(FMath::Wrap( 25.0, 10.0, 20.0) == 15.0);
	CHECK(FMath::Wrap( 29.0, 10.0, 20.0) == 19.0);
	CHECK(FMath::Wrap( 30.0, 10.0, 20.0) == 20.0);
	CHECK(FMath::Wrap( 54.0, 10.0, 20.0) == 14.0);
	CHECK(FMath::Wrap( 60.0, 10.0, 20.0) == 20.0);

	// Floats (with fractional values)
	CHECK(FMath::IsNearlyEqual(FMath::Wrap(-17.62f, 2.37f, 8.44f), 6.66f, UE_KINDA_SMALL_NUMBER));
	CHECK(FMath::IsNearlyEqual(FMath::Wrap( -1.89f, 2.37f, 8.44f), 4.18f, UE_KINDA_SMALL_NUMBER));
	CHECK(FMath::IsNearlyEqual(FMath::Wrap(  7.12f, 2.37f, 8.44f), 7.12f, UE_KINDA_SMALL_NUMBER));
	CHECK(FMath::IsNearlyEqual(FMath::Wrap( 11.31f, 2.37f, 8.44f), 5.24f, UE_KINDA_SMALL_NUMBER));
	CHECK(FMath::IsNearlyEqual(FMath::Wrap( 23.65f, 2.37f, 8.44f), 5.44f, UE_KINDA_SMALL_NUMBER));

	// Doubles (with fractional values)
	CHECK(FMath::IsNearlyEqual(FMath::Wrap(-17.62, 2.37, 8.44), 6.66, UE_DOUBLE_KINDA_SMALL_NUMBER));
	CHECK(FMath::IsNearlyEqual(FMath::Wrap( -1.89, 2.37, 8.44), 4.18, UE_DOUBLE_KINDA_SMALL_NUMBER));
	CHECK(FMath::IsNearlyEqual(FMath::Wrap(  7.12, 2.37, 8.44), 7.12, UE_DOUBLE_KINDA_SMALL_NUMBER));
	CHECK(FMath::IsNearlyEqual(FMath::Wrap( 11.31, 2.37, 8.44), 5.24, UE_DOUBLE_KINDA_SMALL_NUMBER));
	CHECK(FMath::IsNearlyEqual(FMath::Wrap( 23.65, 2.37, 8.44), 5.44, UE_DOUBLE_KINDA_SMALL_NUMBER));

	// Floats (large values far away from the range)
	CHECK(FMath::Wrap(-(float)(1 << 25), 1.0f, 2.0f) == 1.0f);
	CHECK(FMath::Wrap( (float)(1 << 25), 1.0f, 2.0f) == 2.0f);

	// Doubles (large values far away from the range)
	CHECK(FMath::Wrap(-(double)(1 << 25), 1.0, 2.0) == 1.0);
	CHECK(FMath::Wrap( (double)(1 << 25), 1.0, 2.0) == 2.0);
}

TEST_CASE_NAMED(FMathWrapExclusiveTest, "System::Core::Math::WrapExclusive", "[ApplicationContextMask][SmokeFilter]")
{
	// Ints (compile time)
	static_assert(FMath::WrapExclusive(-17, 10, 20) == 13);
	static_assert(FMath::WrapExclusive(-1,  10, 20) == 19);
	static_assert(FMath::WrapExclusive(0,   10, 20) == 10);
	static_assert(FMath::WrapExclusive(5,   10, 20) == 15);
	static_assert(FMath::WrapExclusive(9,   10, 20) == 19);
	static_assert(FMath::WrapExclusive(10,  10, 20) == 10);
	static_assert(FMath::WrapExclusive(15,  10, 20) == 15);
	static_assert(FMath::WrapExclusive(19,  10, 20) == 19);
	static_assert(FMath::WrapExclusive(20,  10, 20) == 10);
	static_assert(FMath::WrapExclusive(21,  10, 20) == 11);
	static_assert(FMath::WrapExclusive(25,  10, 20) == 15);
	static_assert(FMath::WrapExclusive(29,  10, 20) == 19);
	static_assert(FMath::WrapExclusive(30,  10, 20) == 10);
	static_assert(FMath::WrapExclusive(54,  10, 20) == 14);
	static_assert(FMath::WrapExclusive(60,  10, 20) == 10);

	// Ints (runtime)
	CHECK(FMath::WrapExclusive(-17, 10, 20) == 13);
	CHECK(FMath::WrapExclusive(-1,  10, 20) == 19);
	CHECK(FMath::WrapExclusive(0,   10, 20) == 10);
	CHECK(FMath::WrapExclusive(5,   10, 20) == 15);
	CHECK(FMath::WrapExclusive(9,   10, 20) == 19);
	CHECK(FMath::WrapExclusive(10,  10, 20) == 10);
	CHECK(FMath::WrapExclusive(15,  10, 20) == 15);
	CHECK(FMath::WrapExclusive(19,  10, 20) == 19);
	CHECK(FMath::WrapExclusive(20,  10, 20) == 10);
	CHECK(FMath::WrapExclusive(21,  10, 20) == 11);
	CHECK(FMath::WrapExclusive(25,  10, 20) == 15);
	CHECK(FMath::WrapExclusive(29,  10, 20) == 19);
	CHECK(FMath::WrapExclusive(30,  10, 20) == 10);
	CHECK(FMath::WrapExclusive(54,  10, 20) == 14);
	CHECK(FMath::WrapExclusive(60,  10, 20) == 10);

	// Ints (compile time - distance to range is greater than std::numeric_limits<T>::max())
	static_assert(FMath::WrapExclusive(MIN_int32, MAX_int32 - 8, MAX_int32) == MAX_int32 - 7);
	static_assert(FMath::WrapExclusive(MAX_int32, MIN_int32, MIN_int32 + 8) == MIN_int32 + 7);
	static_assert(FMath::WrapExclusive((int8)123, (int8)-20, (int8)-10) == (int8)-17);
	static_assert(FMath::WrapExclusive((int8)-123, (int8)10, (int8)20) == (int8)17);

	// Ints (runtime - distance to range is greater than std::numeric_limits<T>::max())
	CHECK(FMath::WrapExclusive(MIN_int32, MAX_int32 - 8, MAX_int32) == MAX_int32 - 7);
	CHECK(FMath::WrapExclusive(MAX_int32, MIN_int32, MIN_int32 + 8) == MIN_int32 + 7);
	CHECK(FMath::WrapExclusive((int8)123, (int8)-20, (int8)-10) == (int8)-17);
	CHECK(FMath::WrapExclusive((int8)-123, (int8)10, (int8)20) == (int8)17);

	// Ints (compile time - range-size is greater than std::numeric_limits<T>::max())
	static_assert(FMath::WrapExclusive(MAX_int32, MIN_int32 + 2, MAX_int32));
	static_assert(FMath::WrapExclusive((int8)-123, (int8)-120, (int8)120) == (int8)117);

	// Ints (runtime - range-size is greater than std::numeric_limits<T>::max())
	CHECK(FMath::WrapExclusive(MAX_int32, MIN_int32 + 2, MAX_int32));
	CHECK(FMath::WrapExclusive((int8)-123, (int8)-120, (int8)120) == (int8)117);
}

class FInitVectorTestClass {
public:

	bool InitVectorTest(){
	auto TestInitFromCompactString = [this](const FString& InTestName, const FVector& InExpected)
	{
		FVector Actual(13.37f, 13.37f, 13.37f);
		const bool bIsInitialized = Actual.InitFromCompactString(InExpected.ToCompactString());

		CHECK_MESSAGE(*(InTestName + " return value"), bIsInitialized);
		CHECK_MESSAGE(*InTestName, InExpected.Equals(Actual, UE_KINDA_SMALL_NUMBER)); //
	};

	TestInitFromCompactString(TEXT("InitFromCompactString Simple"), FVector(1.2f, 2.3f, 3.4f));
	TestInitFromCompactString(TEXT("InitFromCompactString Zero"), FVector(0, 0, 0));
	TestInitFromCompactString(TEXT("InitFromCompactString Int"), FVector(1, 2, 3));

	TestInitFromCompactString(TEXT("InitFromCompactString X == 0"), FVector(0, 2, 3));
	TestInitFromCompactString(TEXT("InitFromCompactString Y == 0"), FVector(1.3f, 0, 3.7f));
	TestInitFromCompactString(TEXT("InitFromCompactString Z == 0"), FVector(1.2f, 2.5f, 0));

	TestInitFromCompactString(TEXT("InitFromCompactString X < 0"), FVector(-433.2f, 6.5f, 0));
	TestInitFromCompactString(TEXT("InitFromCompactString Y < 0"), FVector(43.2f, -6.5f, 98));
	TestInitFromCompactString(TEXT("InitFromCompactString Z < 0"), FVector(33.8f, 0, -76));

	TestInitFromCompactString(TEXT("InitFromCompactString X == 0 && Y == 0"), FVector(0, 0, 32.8f));
	TestInitFromCompactString(TEXT("InitFromCompactString X == 0 && Z == 0"), FVector(0, 61.3f, 0));
	TestInitFromCompactString(TEXT("InitFromCompactString Y == 0 && Z == 0"), FVector(65.3f, 0, 0));

	CHECK_FALSE_MESSAGE(TEXT("InitFromCompactString BadString1"), FVector().InitFromCompactString(TEXT("W(0)")));
	CHECK_FALSE_MESSAGE(TEXT("InitFromCompactString BadString2"), FVector().InitFromCompactString(TEXT("V(XYZ)")));
	return true;
	}
};

TEST_CASE_NAMED(FInitVectorTest, "System::Core::Math::InitVector", "[ApplicationContextMask][SmokeFilter]")
{
	FInitVectorTestClass Instance;
	Instance.InitVectorTest();
}

// On GCC and Clang, setting -ffast-math or enabling some other unsafe floating point
// optimizations set these #defines. As long as these compiler flags are active, the code
// effectively promises that there are no infinites or NaNs, and although the original code
// is written to handle these correctly, the compiler will happily apply transforms that
// break it.
// 
// With these compiler flags on (and thus allowing the compiler to make transforms that
// assume Inf/NaN don't occur), there is no expectation that any code can handle Inf/NaN
// correctly (since the compiler is free to break whatever you write), so don't even test
// it.
#if defined(__FAST_MATH__) || defined(__FINITE_MATH_ONLY__)
static constexpr bool GColorConversionsTestInfNaNs = false;
#else
static constexpr bool GColorConversionsTestInfNaNs = true;
#endif

class FColorConversionTestClass {
public:

	bool ColorConversionTest() {
		// Round-trip conversion tests check that converting from uint8 color formats
	// to float and back gives the original value. We not only want to guarantee
	// this on its own, it also gives us useful coverage of the [0,1] value range.

	// Test that sRGB<->Linear conversions round-trip
		for (int Index = 0; Index < 256; ++Index)
		{
			// Make the inputs in R,G,B not the same in cases channels get swapped or similar
			// Alpha channel is already special because it gets treated differently
			FColor OriginalColor((uint8)Index, (uint8)(Index ^ 1), (uint8)(Index ^ 123), (uint8)Index);

			FLinearColor SRGBToLinear = FLinearColor::FromSRGBColor(OriginalColor);
			FColor SRGBConvertedBack = SRGBToLinear.ToFColorSRGB();
			CHECK_MESSAGE(FString::Printf(TEXT("sRGB to linear to sRGB round-trip: %d"), Index), OriginalColor == SRGBConvertedBack);

			FLinearColor UNORMToLinear = OriginalColor.ReinterpretAsLinear();
			FColor UNORMConvertedBack = UNORMToLinear.QuantizeRound();
			CHECK_MESSAGE(FString::Printf(TEXT("UNORM to linear to UNORM round-trip: %d"), Index), OriginalColor == UNORMConvertedBack);
		}

		// Test values near breakpoints/bucket boundaries to make sure they end up
		// on the intended side. Since we are interested in the boundaries between
		// values, this loop only goes to 255.
		auto ReferenceSRGBToLinear = [](float InValue) -> float
		{
			if (InValue < 0.04045f)
			{
				return InValue / 12.92f;
			}
			else
			{
				return (float)FMath::Pow((InValue + 0.055) / 1.055, 2.4); // Compute in doubles
			}
		};

		for (int Index = 0; Index < 255; ++Index)
		{
			float BucketMidpoint = float(Index) + 0.5f;

			// Worst-case error in the integer [0,255] sRGB scale is guaranteed to be below
			// 0.544403 by the conversion we use. (The minimum reachable is 0.5, because
			// we're quantizing to integers). That means that as long as we stay about
			// 0.045f units away from a breakpoint, we should always get the exact value.
			const float DistanceToMidpoint = 0.045f;

			float BelowBoundaryUNORM = (BucketMidpoint - DistanceToMidpoint) / 255.f;
			float BelowBoundarySRGB = ReferenceSRGBToLinear(BelowBoundaryUNORM);
			uint8 IndexU8 = (uint8)Index;
			FColor BelowExpected(IndexU8, IndexU8, IndexU8, IndexU8);
			FColor BelowConverted = FLinearColor(BelowBoundarySRGB, BelowBoundarySRGB, BelowBoundarySRGB, BelowBoundaryUNORM).ToFColorSRGB();
			CHECK_MESSAGE(FString::Printf(TEXT("sRGB Boundary below: %d"), Index), BelowConverted == BelowExpected);

			float AboveBoundaryUNORM = (BucketMidpoint + DistanceToMidpoint) / 255.f;
			float AboveBoundarySRGB = ReferenceSRGBToLinear(AboveBoundaryUNORM);
			uint8 IndexPlus1U8 = (uint8)(Index + 1);
			FColor AboveExpected(IndexPlus1U8, IndexPlus1U8, IndexPlus1U8, IndexPlus1U8);
			FColor AboveConverted = FLinearColor(AboveBoundarySRGB, AboveBoundarySRGB, AboveBoundarySRGB, AboveBoundaryUNORM).ToFColorSRGB();
			CHECK_MESSAGE(FString::Printf(TEXT("sRGB Boundary above: %d"), Index), AboveConverted == AboveExpected);
		}

		// Between the two tests above, we have good coverage of what happens inside [0,1];
		// for values outside, we expect the sRGB and UNORM paths to give the same results.
		// Just test a few values in interesting parts of the range.
		const float NaN = FPlatformMath::AsFloat((uint32)0x7fc00000u);
		const float PosInf = FPlatformMath::AsFloat((uint32)0x7f800000u);
		const float PosSubnormal = FPlatformMath::AsFloat((uint32)0x20000u);
		const float MediumLarge = 10.f; // Outside the range, but not hugely so
		const float VeryLarge = 1e+7f; // Far outside the range

		auto TestExtremalValue = [this](const TCHAR* InTestName, const float InValue, uint8 InExpected)
		{
			FLinearColor LinColor(InValue, InValue, InValue, InValue);
			FColor ConvertedColor = LinColor.ToFColorSRGB();
			FColor ExpectedColor(InExpected, InExpected, InExpected, InExpected);

			CHECK_MESSAGE(InTestName, ConvertedColor == ExpectedColor);
		};

		if (GColorConversionsTestInfNaNs)
		{
			TestExtremalValue(TEXT("Extremal NaN"), NaN, 0);
			TestExtremalValue(TEXT("Extremal -Inf"), -PosInf, 0);
			TestExtremalValue(TEXT("Extremal +Inf"), PosInf, 255);
		}

		TestExtremalValue(TEXT("Extremal -MediumLarge"), -MediumLarge, 0);
		TestExtremalValue(TEXT("Extremal 0"), 0.0f, 0);
		TestExtremalValue(TEXT("Extremal +subnorm"), PosSubnormal, 0);
		TestExtremalValue(TEXT("Extremal 1"), 1.0f, 255);
		TestExtremalValue(TEXT("Extremal Mediumlarge"), MediumLarge, 255);
		TestExtremalValue(TEXT("Extremal VeryLarge"), VeryLarge, 255);
		return true;
	}
};

TEST_CASE_NAMED(FColorConversionTest, "System::Core::Math::ColorConversion", "[ApplicationContextMask][EngineFilter]")
{
	FColorConversionTestClass Instance;
	Instance.ColorConversionTest();
}

// This is repeating the reference (scalar) linear->sRGB conversion from Color.cpp, 
// because the reference version is not necessarily exposed; we can end up using a different
// implementation depending on the platform (as of this writing, we have SSE2 implementation
// too), but we want them all to match.
// 
// Wrapping in a namespace to avoid name conflicts but otherwise keep the code as identical
// as possible.
namespace UnrealMathTestInternal {

	typedef union
	{
		uint32 u;
		float f;
	} stbir__FP32;

	static const uint32 stb_fp32_to_srgb8_tab4[104] = {
		0x0073000d, 0x007a000d, 0x0080000d, 0x0087000d, 0x008d000d, 0x0094000d, 0x009a000d, 0x00a1000d,
		0x00a7001a, 0x00b4001a, 0x00c1001a, 0x00ce001a, 0x00da001a, 0x00e7001a, 0x00f4001a, 0x0101001a,
		0x010e0033, 0x01280033, 0x01410033, 0x015b0033, 0x01750033, 0x018f0033, 0x01a80033, 0x01c20033,
		0x01dc0067, 0x020f0067, 0x02430067, 0x02760067, 0x02aa0067, 0x02dd0067, 0x03110067, 0x03440067,
		0x037800ce, 0x03df00ce, 0x044600ce, 0x04ad00ce, 0x051400ce, 0x057b00c5, 0x05dd00bc, 0x063b00b5,
		0x06970158, 0x07420142, 0x07e30130, 0x087b0120, 0x090b0112, 0x09940106, 0x0a1700fc, 0x0a9500f2,
		0x0b0f01cb, 0x0bf401ae, 0x0ccb0195, 0x0d950180, 0x0e56016e, 0x0f0d015e, 0x0fbc0150, 0x10630143,
		0x11070264, 0x1238023e, 0x1357021d, 0x14660201, 0x156601e9, 0x165a01d3, 0x174401c0, 0x182401af,
		0x18fe0331, 0x1a9602fe, 0x1c1502d2, 0x1d7e02ad, 0x1ed4028d, 0x201a0270, 0x21520256, 0x227d0240,
		0x239f0443, 0x25c003fe, 0x27bf03c4, 0x29a10392, 0x2b6a0367, 0x2d1d0341, 0x2ebe031f, 0x304d0300,
		0x31d105b0, 0x34a80555, 0x37520507, 0x39d504c5, 0x3c37048b, 0x3e7c0458, 0x40a8042a, 0x42bd0401,
		0x44c20798, 0x488e071e, 0x4c1c06b6, 0x4f76065d, 0x52a50610, 0x55ac05cc, 0x5892058f, 0x5b590559,
		0x5e0c0a23, 0x631c0980, 0x67db08f6, 0x6c55087f, 0x70940818, 0x74a007bd, 0x787d076c, 0x7c330723,
	};

	static uint8 stbir__linear_to_srgb_uchar_fast(float in)
	{
		static const stbir__FP32 almostone = { 0x3f7fffff }; // 1-eps
		static const stbir__FP32 minval = { (127 - 13) << 23 };
		uint32 tab, bias, scale, t;
		stbir__FP32 f;

		// Clamp to [2^(-13), 1-eps]; these two values map to 0 and 1, respectively.
		// The tests are carefully written so that NaNs map to 0, same as in the reference
		// implementation.
		if (!(in > minval.f)) // written this way to catch NaNs
			in = minval.f;
		if (in > almostone.f)
			in = almostone.f;

		// Do the table lookup and unpack bias, scale
		f.f = in;

		tab = stb_fp32_to_srgb8_tab4[(f.u - minval.u) >> 20];
		bias = (tab >> 16) << 9;
		scale = tab & 0xffff;

		// Grab next-highest mantissa bits and perform linear interpolation
		t = (f.u >> 12) & 0xff;
		return (uint8)((bias + scale * t) >> 16);
	}

}

TEST_CASE_NAMED(FColorConversionHeavyTest, "System::Core::Math::ColorConversionHeavy", "[.][ApplicationContextMask][EngineFilter][Slow]")
{
	// WARNING: This test runs for a good while (at time of writing, ~90s when using
	// a single core, proportionately less on many-core machines) with no user
	// feedback other than log messages being printed. You've been warned.

	// Chop up the 32-bit value range into pow2-sized buckets so we
	// give at least regular progress updates in the log
	const int BucketShift = 8;
	const int BucketCount = 1 << BucketShift;
	const uint32 ItemsInBucket = 1u << (32 - BucketShift);

	struct FSharedData
	{
		FCriticalSection Lock; // Protects everything in here
		bool Failed = false;
		int32 CompletionCounter = 0;

		// These are set at most once (when Failed is first set)
		uint32 FailedValue = 0; // Value we first noticed a mismatch on
		FColor FailedConverted;
		FColor FailedExpected;
	};

	FSharedData Shared;

	// Test that the platform specialization for ToFColorSRGB agrees with the reference
	// implementation. As of this writing, we have a vectorized version for SSE2 targets.
	ParallelFor(BucketCount,
		[BucketCount, ItemsInBucket, &Shared](int32 BucketIndex)
		{
			for (uint32 WithinBucketIndex = 0; WithinBucketIndex < ItemsInBucket; ++WithinBucketIndex)
			{
				uint32 CurrentBits = BucketIndex * ItemsInBucket + WithinBucketIndex;

				// When asked to not test Infs/NaNs, skip that exponent entirely.
				if (!GColorConversionsTestInfNaNs && (CurrentBits & 0x7f800000u) == 0x7f800000u)
				{
					continue;
				}

				// Don't put the same value in every color channel; we want to make sure we catch
				// accidental channel swaps too! Three different inputs only; A is special anyway.
				//
				// XOR constants here are chosen to keep exponent bits same.
				float R = FPlatformMath::AsFloat(CurrentBits);
				float G = FPlatformMath::AsFloat(CurrentBits ^ 1234567u);
				float B = FPlatformMath::AsFloat(CurrentBits ^ 5490682u);
				FLinearColor LinearInput(R, G, B, R);
				FColor Converted = LinearInput.ToFColorSRGB();

				// Reference conversion
				uint8 RgbResult[3];
				for (int Channel = 0; Channel < 3; ++Channel)
				{
					float Value = LinearInput.Component(Channel);
					RgbResult[Channel] = UnrealMathTestInternal::stbir__linear_to_srgb_uchar_fast(Value);
				}

				// Clamp A, mapping NaNs to 0
				float ClampedALo = (LinearInput.A > 0.0f) ? LinearInput.A : 0.0f;
				float ClampedA = (ClampedALo < 1.0f) ? ClampedALo : 1.0f;
				uint8 FinalA = (uint8)(ClampedA * 255.f + 0.5f);

				FColor Expected{ RgbResult[0], RgbResult[1], RgbResult[2], FinalA };

				// Test and note failures (get reported outside)
				if (Converted != Expected)
				{
					FScopeLock LockHolder(&Shared.Lock);

					// If we are the first to fail, set failure info
					if (!Shared.Failed)
					{
						Shared.Failed = true;
						Shared.FailedValue = CurrentBits;
						Shared.FailedConverted = Converted;
						Shared.FailedExpected = Expected;
					}

					// Break out of loop on first error
					break;
				}
			}

	// Check status and update count
	bool Failed = false;
	int32 Completed = 0;

	// Scope lock access to just grabbing the few values, not the log too
	{
		FScopeLock LockHolder(&Shared.Lock);
		Failed = Shared.Failed;
		Completed = ++Shared.CompletionCounter;
	}

	// If another instance failed, stop
	if (Failed)
		return;

	UE_LOGF(LogUnrealMathTest, Log, "Conversion heavy sRGB %d/%d buckets completed (%.2f%%)", Completed, BucketCount, double(Completed) * 100.0 / double(BucketCount));
		}
	);

	if (Shared.Failed)
	{
		CHECK_MESSAGE(FString::Printf(TEXT("Conversion heavy sRGB CurrentBits=0x%08x"), Shared.FailedValue), Shared.FailedConverted == Shared.FailedExpected);
		REQUIRE(false);
	}
	
}

TEST_CASE_NAMED(FBoxTest, "System::Core::Math::Box Test", "[ApplicationContextMask][SmokeFilter]")
{
	// Default ctor
	CHECK(!FBox3f {}.IsValid);
	CHECK(!FBox3d {}.IsValid);

	// Constructors from explicit coordinates and from spheres
	// as well as operator ==
	CHECK(FBox3f { FVector3f { -1.f, -1.f, -1.f }, FVector3f { 1.f, 1.f, 1.f} } == FBox3f { FSphere3f { FVector3f { 0.f, 0.f, 0.f, }, 1.f } });
	CHECK(FBox3d { FVector3d { -1.0,  0.0,  1.0 }, FVector3d { 1.0, 2.0, 3.0} } == FBox3d { FSphere3d { FVector3d { 0.0, 1.0, 2.0, }, 1.0 } });
	
	// Start with empty box then add points add one by one
	{
		FBox3f BoxF;
		BoxF += FVector3f { 1.f, 0.f, 1.f };
		BoxF = BoxF + FVector3f { 0.f, 1.f, 0.f };
		CHECK(BoxF == FBox3f { FVector3f { 0.f, 0.f, 0.f, }, FVector3f { 1.f, 1.f, 1.f } });
		CHECK(BoxF[0] == BoxF.Min);
		CHECK(BoxF[1] == BoxF.Max);

		FVector3d DoublePoints[2] { FVector3d { 1.0, -2.0, 3.0 }, FVector3d { -4.0, 5.0, -6.0 } };
		FBox3d BoxD(DoublePoints, 2);
		CHECK(BoxD == FBox3d { FVector3d { -4.0, -2.0, -6.0 }, FVector3d { 1.0, 5.0, 3.0 } });
	}
}

#endif //WITH_TESTS
