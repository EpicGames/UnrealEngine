// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Memory/CompositeBuffer.h"
#include "Templates/Function.h"

#include "Tests/TestHarnessAdapter.h"
#include <type_traits>

TEST_CASE_NAMED(FMemoryCompositeBufferTest, "System::Core::Memory::CompositeBuffer", "[Core][Memory][SmokeFilter]")
{
	SECTION("Static")
	{
		STATIC_REQUIRE(std::is_constructible_v<FCompositeBuffer>);
		STATIC_REQUIRE(std::is_constructible_v<FCompositeBuffer, FSharedBuffer&&>);
		STATIC_REQUIRE(std::is_constructible_v<FCompositeBuffer, FCompositeBuffer&&>);
		STATIC_REQUIRE(std::is_constructible_v<FCompositeBuffer, TArray<FSharedBuffer>&&>);
		STATIC_REQUIRE(std::is_constructible_v<FCompositeBuffer, TArray<const FSharedBuffer>&&>);
		STATIC_REQUIRE(std::is_constructible_v<FCompositeBuffer, const FSharedBuffer&>);
		STATIC_REQUIRE(std::is_constructible_v<FCompositeBuffer, const FCompositeBuffer&>);
		STATIC_REQUIRE(std::is_constructible_v<FCompositeBuffer, const TArray<FSharedBuffer>&>);
		STATIC_REQUIRE(std::is_constructible_v<FCompositeBuffer, const TArray<const FSharedBuffer>&>);
		STATIC_REQUIRE(std::is_constructible_v<FCompositeBuffer, const FSharedBuffer&, const FCompositeBuffer&, const TArray<FSharedBuffer>&, const TArray<const FSharedBuffer>&>);

		STATIC_REQUIRE(!std::is_constructible_v<FCompositeBuffer, FMemoryView>);
		STATIC_REQUIRE(!std::is_constructible_v<FCompositeBuffer, const uint8[4]>);
	}

	SECTION("Null")
	{
		FCompositeBuffer Buffer;
		CHECK(Buffer.IsNull());
		CHECK(Buffer.IsOwned());
		CHECK(Buffer.MakeOwned().IsNull());
		CHECK(Buffer.ToShared().IsNull());
		CHECK(Buffer.Mid(0, 0).IsNull());
		CHECK(Buffer.GetSize() == 0);
		CHECK(Buffer.GetSegments().Num() == 0);

		FUniqueBuffer CopyBuffer;
		CHECK(Buffer.ViewOrCopyRange(0, 0, CopyBuffer).IsEmpty());
		CHECK(CopyBuffer.IsNull());

		FMutableMemoryView CopyView;
		Buffer.CopyTo(CopyView);

		uint32 VisitCount = 0;
		Buffer.IterateRange(0, 0, [&VisitCount](FMemoryView) { ++VisitCount; });
		CHECK(VisitCount == 0);
	}

	SECTION("Null Remove")
	{
		FCompositeBuffer Buffer(FSharedBuffer(), FCompositeBuffer(), TArray<FSharedBuffer>{FSharedBuffer()});
		CHECK(Buffer.IsNull());
		CHECK(Buffer.GetSegments().Num() == 0);
	}

	SECTION("Empty")
	{
		const uint8 EmptyArray[]{0};
		const FSharedBuffer EmptyView = FSharedBuffer::MakeView(EmptyArray, 0);
		FCompositeBuffer Buffer(EmptyView);
		CHECK_FALSE(Buffer.IsNull());
		CHECK_FALSE(Buffer.IsOwned());
		CHECK_FALSE(Buffer.MakeOwned().IsNull());
		CHECK(Buffer.MakeOwned().IsOwned());
		CHECK(Buffer.ToShared() == EmptyView);
		CHECK(Buffer.Mid(0, 0).ToShared() == EmptyView);
		CHECK(Buffer.GetSize() == 0);
		CHECK(Buffer.GetSegments().Num() == 1);
		CHECK(Buffer.GetSegments()[0] == EmptyView);

		FUniqueBuffer CopyBuffer;
		CHECK(Buffer.ViewOrCopyRange(0, 0, CopyBuffer) == EmptyView.GetView());
		CHECK(CopyBuffer.IsNull());

		FMutableMemoryView CopyView;
		Buffer.CopyTo(CopyView);

		uint32 VisitCount = 0;
		Buffer.IterateRange(0, 0, [&VisitCount](FMemoryView) { ++VisitCount; });
		CHECK(VisitCount == 1);
	}

	SECTION("Empty Concatenation")
	{
		const uint8 EmptyArray[1]{};
		const FSharedBuffer EmptyView1 = FSharedBuffer::MakeView(EmptyArray, 0);
		const FSharedBuffer EmptyView2 = FSharedBuffer::MakeView(EmptyArray + 1, 0);
		FCompositeBuffer Buffer(TArray<FSharedBuffer>{EmptyView1}, FCompositeBuffer(EmptyView2));
		CHECK(Buffer.Mid(0, 0).ToShared() == EmptyView1);
		CHECK(Buffer.GetSize() == 0);
		CHECK(Buffer.GetSegments().Num() == 2);
		CHECK(Buffer.GetSegments()[0] == EmptyView1);
		CHECK(Buffer.GetSegments()[1] == EmptyView2);

		FUniqueBuffer CopyBuffer;
		CHECK(Buffer.ViewOrCopyRange(0, 0, CopyBuffer) == EmptyView1.GetView());
		CHECK(CopyBuffer.IsNull());

		FMutableMemoryView CopyView;
		Buffer.CopyTo(CopyView);

		uint32 VisitCount = 0;
		Buffer.IterateRange(0, 0, [&VisitCount](FMemoryView) { ++VisitCount; });
		CHECK(VisitCount == 1);
	}

	SECTION("Flat")
	{
		const uint8 FlatArray[]{1, 2, 3, 4, 5, 6, 7, 8};
		const FSharedBuffer FlatView = FSharedBuffer::Clone(MakeMemoryView(FlatArray));
		FCompositeBuffer Buffer(FlatView);
		CHECK_FALSE(Buffer.IsNull());
		CHECK(Buffer.IsOwned());
		CHECK(Buffer.ToShared() == FlatView);
		CHECK(Buffer.MakeOwned().ToShared() == FlatView);
		CHECK(Buffer.Mid(0).ToShared() == FlatView);
		CHECK(Buffer.Mid(4).ToShared().GetView() == FlatView.GetView().Mid(4));
		CHECK(Buffer.Mid(8).ToShared().GetView() == FlatView.GetView().Mid(8));
		CHECK(Buffer.Mid(4, 2).ToShared().GetView() == FlatView.GetView().Mid(4, 2));
		CHECK(Buffer.Mid(8).ToShared().GetView().GetData() == FlatView.GetView().Mid(8).GetData());
		CHECK(Buffer.Mid(4, 2).ToShared().GetView().GetData() == FlatView.GetView().Mid(4, 2).GetData());
		CHECK(Buffer.Mid(8, 0).ToShared().GetView().GetData() == FlatView.GetView().Mid(8, 0).GetData());
		CHECK(Buffer.GetSize() == sizeof(FlatArray));
		CHECK(Buffer.GetSegments().Num() == 1);
		CHECK(Buffer.GetSegments()[0] == FlatView);

		FUniqueBuffer CopyBuffer;
		CHECK(Buffer.ViewOrCopyRange(0, sizeof(FlatArray), CopyBuffer) == FlatView.GetView());
		CHECK(CopyBuffer.IsNull());

		uint8 CopyArray[sizeof(FlatArray) - 3];
		Buffer.CopyTo(MakeMemoryView(CopyArray), 3);
		CHECK(MakeMemoryView(CopyArray).EqualBytes(MakeMemoryView(FlatArray) + 3));

		uint32 VisitCount = 0;
		Buffer.IterateRange(0, sizeof(FlatArray), [&VisitCount](FMemoryView) { ++VisitCount; });
		CHECK(VisitCount == 1);
	}

	SECTION("Composite")
	{
		const uint8 FlatArray[]{1, 2, 3, 4, 5, 6, 7, 8};
		const FSharedBuffer FlatView1 = FSharedBuffer::MakeView(MakeMemoryView(FlatArray).Left(4));
		const FSharedBuffer FlatView2 = FSharedBuffer::MakeView(MakeMemoryView(FlatArray).Right(4));
		FCompositeBuffer Buffer(FlatView1, FlatView2);
		CHECK_FALSE(Buffer.IsNull());
		CHECK_FALSE(Buffer.IsOwned());
		CHECK(Buffer.ToShared().GetView().EqualBytes(MakeMemoryView(FlatArray)));
		CHECK(Buffer.Mid(2, 4).ToShared().GetView().EqualBytes(MakeMemoryView(FlatArray).Mid(2, 4)));
		CHECK(Buffer.Mid(0, 4).ToShared() == FlatView1);
		CHECK(Buffer.Mid(4, 4).ToShared() == FlatView2);
		CHECK(Buffer.Mid(4, 0).ToShared().GetView().GetData() == FlatArray + 4);
		CHECK(Buffer.Mid(8, 0).ToShared().GetView().GetData() == FlatArray + 8);
		CHECK(Buffer.GetSize() == sizeof(FlatArray));
		CHECK(Buffer.GetSegments().Num() == 2);
		CHECK(Buffer.GetSegments()[0] == FlatView1);
		CHECK(Buffer.GetSegments()[1] == FlatView2);

		FUniqueBuffer CopyBuffer;
		CHECK(Buffer.ViewOrCopyRange(0, 4, CopyBuffer) == FlatView1.GetView());
		CHECK(CopyBuffer.IsNull());
		CHECK(Buffer.ViewOrCopyRange(4, 4, CopyBuffer) == FlatView2.GetView());
		CHECK(CopyBuffer.IsNull());
		CHECK(Buffer.ViewOrCopyRange(3, 2, CopyBuffer).EqualBytes(MakeMemoryView(FlatArray).Mid(3, 2)));
		CHECK(CopyBuffer.GetSize() == 2);
		CHECK(Buffer.ViewOrCopyRange(1, 6, CopyBuffer).EqualBytes(MakeMemoryView(FlatArray).Mid(1, 6)));
		CHECK(CopyBuffer.GetSize() == 6);
		CHECK(Buffer.ViewOrCopyRange(2, 4, CopyBuffer).EqualBytes(MakeMemoryView(FlatArray).Mid(2, 4)));
		CHECK(CopyBuffer.GetSize() == 6);

		uint8 CopyArray[4];
		Buffer.CopyTo(MakeMemoryView(CopyArray), 2);
		CHECK(MakeMemoryView(CopyArray).EqualBytes(MakeMemoryView(FlatArray).Mid(2, 4)));

		uint32 VisitCount = 0;
		Buffer.IterateRange(0, sizeof(FlatArray), [&VisitCount](FMemoryView) { ++VisitCount; });
		CHECK(VisitCount == 2);

		const auto TestIterateRange = [&Buffer](uint64 Offset, uint64 Size, FMemoryView ExpectedView, const FSharedBuffer& ExpectedViewOuter)
		{
			uint32 VisitCount = 0;
			FMemoryView ActualView;
			FSharedBuffer ActualViewOuter;
			Buffer.IterateRange(Offset, Size, [&VisitCount, &ActualView, &ActualViewOuter](FMemoryView View, const FSharedBuffer& ViewOuter)
			{
				++VisitCount;
				ActualView = View;
				ActualViewOuter = ViewOuter;
			});
			CAPTURE(Offset, Size);
			CHECK(VisitCount == 1);
			CHECK(ActualView == ExpectedView);
			CHECK(ActualViewOuter == ExpectedViewOuter);
		};
		TestIterateRange(0, 4, MakeMemoryView(FlatArray).Mid(0, 4), FlatView1);
		TestIterateRange(4, 0, MakeMemoryView(FlatArray).Mid(4, 0), FlatView1);
		TestIterateRange(4, 4, MakeMemoryView(FlatArray).Mid(4, 4), FlatView2);
		TestIterateRange(8, 0, MakeMemoryView(FlatArray).Mid(8, 0), FlatView2);
	}

	SECTION("EqualBytes")
	{
		const uint8 FlatArray[]{1, 2, 3, 4, 5, 6, 7, 8};
		const FCompositeBuffer BufferA(FSharedBuffer::MakeView(FlatArray, 4), FSharedBuffer::MakeView(FlatArray + 4, 4));
		const FCompositeBuffer BufferB(FSharedBuffer::MakeView(FlatArray, 2), FSharedBuffer::MakeView(FlatArray + 2, 4), FSharedBuffer::MakeView(FlatArray + 6, 2));
		const FCompositeBuffer BufferC(FSharedBuffer::MakeView(FlatArray, 7));
		CHECK(BufferA.EqualBytes(BufferB));
		CHECK(BufferB.EqualBytes(BufferA));
		CHECK_FALSE(BufferA.EqualBytes(BufferC));
		CHECK_FALSE(BufferC.EqualBytes(BufferA));
	}
}

#endif // WITH_TESTS
