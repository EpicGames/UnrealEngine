// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Memory/LinearAllocator.h"

#include "Async/ParallelFor.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Templates/AlignmentTemplates.h"
#include "Tests/TestHarnessAdapter.h"

class FInnerMallocForLinearAllocatorTest : public FMalloc
{
public:
	virtual void* Malloc(SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT) override
	{
		void* Ptr = FMemory::Malloc(Count, Alignment);
		Ptrs.Add(Ptr);
		return Ptr;
	}

	virtual void* Realloc(void* Original, SIZE_T Count, uint32 Alignment = DEFAULT_ALIGNMENT) override
	{
		// LinearBlockAllocator does not use Realloc.
		check(false);
		return nullptr;
	}

	virtual void Free(void* Original)
	{
		if (Original)
		{
			int32 RemoveCount = Ptrs.Remove(Original);
			check(RemoveCount == 1);
		}
		FMemory::Free(Original);
	}
	TSet<void*> Ptrs;
};

static bool DoPointersOverlap(void* A, SIZE_T ASize, void* B, SIZE_T BSize)
{
	// Overlapping: B is before end of A, and A is before end of B.
	const UPTRINT PA = reinterpret_cast<UPTRINT>(A);
	const UPTRINT PB = reinterpret_cast<UPTRINT>(B);
	return (PB < PA + ASize) && (PA < PB + BSize);
}

TEST_CASE_NAMED(FLinearBlockAllocatorTest, "System::Core::Memory::LinearBlockAllocator::Base", "[ApplicationContextMask][EngineFilter]")
{
	SECTION("Single allocation is non-null and aligned to DefaultAlignment")
	{
		UE::FLinearBlockAllocator Allocator;
		void* Ptr = Allocator.Malloc(16);
		CHECK(Ptr != nullptr);
		CHECK(IsAligned(Ptr, 8));
	}

	SECTION("Two sequential allocations do not overlap")
	{
		UE::FLinearBlockAllocator Allocator;
		SIZE_T AllocSize = 32;
		void* A = Allocator.Malloc(AllocSize);
		void* B = Allocator.Malloc(AllocSize);
		CHECK(A != nullptr);
		CHECK(B != nullptr);
		CHECK(!DoPointersOverlap(A, AllocSize, B, AllocSize));
	}

	SECTION("Allocations returned pointers are writable across the full requested size")
	{
		UE::FLinearBlockAllocator Allocator;
		constexpr SIZE_T Size = 256;
		uint8* Ptr = static_cast<uint8*>(Allocator.Malloc(Size));
		REQUIRE(Ptr != nullptr);
		// Write and read back every byte.
		for (SIZE_T i = 0; i < Size; ++i)
		{
			Ptr[i] = static_cast<uint8>(i);
		}
		bool bAllCorrect = true;
		for (SIZE_T i = 0; i < Size; ++i)
		{
			if (Ptr[i] != static_cast<uint8>(i))
			{
				bAllCorrect = false;
				break;
			}
		}
		CHECK(bAllCorrect);
	}

	SECTION("Custom alignment is honoured")
	{
		UE::FLinearBlockAllocator Allocator;
		for (uint32 Alignment : {8u, 16u, 32u, 64u, 128u, 256u})
		{
			void* Ptr = Allocator.Malloc(1, Alignment);
			REQUIRE(Ptr != nullptr);
			CHECK(IsAligned(Ptr, Alignment));
		}
	}

	SECTION("Alignment smaller than DefaultAlignment is promoted to DefaultAlignment")
	{
		UE::FLinearBlockAllocator Allocator;
		// Requesting alignment of 1 should still give at least 8-byte alignment.
		void* Ptr = Allocator.Malloc(1, 1);
		REQUIRE(Ptr != nullptr);
		CHECK(IsAligned(Ptr, 8));
	}

	SECTION("Large number of small allocations all succeed and are non-overlapping")
	{
		UE::FLinearBlockAllocator Allocator(nullptr, 256);
		constexpr int32 Count = 100;
		constexpr SIZE_T AllocSize = 8;
		TArray<void*> Ptrs;
		Ptrs.Reserve(Count);

		for (int32 i = 0; i < Count; ++i)
		{
			void* Ptr = Allocator.Malloc(AllocSize);
			REQUIRE(Ptr != nullptr);
			Ptrs.Add(Ptr);
		}

		// Verify uniqueness — no two returned pointers overlap.
		for (int32 i = 0; i < Count; ++i)
		{
			for (int32 j = i + 1; j < Count; ++j)
			{
				CHECK(!DoPointersOverlap(Ptrs[i], AllocSize, Ptrs[j], AllocSize));
			}
		}
	}

	SECTION("Allocation larger than default block size succeeds (custom block path)")
	{
		// Use a small explicit block size so a standard allocation forces the custom block path.
		UE::FLinearBlockAllocator Allocator(nullptr, 256);
		constexpr SIZE_T LargeSize = 512;
		void* Ptr = Allocator.Malloc(LargeSize);
		CHECK(Ptr != nullptr);
		CHECK(IsAligned(Ptr, 8));
		// Write to the full extent to confirm it is backed by real memory.
		FMemory::Memset(Ptr, 0xAB, LargeSize);
	}

	SECTION("GetAllocatedMemorySize increases after allocations that require new blocks")
	{
		UE::FLinearBlockAllocator Allocator;
		const SIZE_T Before = Allocator.GetAllocatedMemorySize();

		// Exhaust one full default block.
		constexpr uint32 SmallBlockSize = 256;
		UE::FLinearBlockAllocator SmallBlockAllocator(nullptr, SmallBlockSize);
		const SIZE_T BeforeSmall = SmallBlockAllocator.GetAllocatedMemorySize();

		// Allocate enough to force a second block.
		SmallBlockAllocator.Malloc(SmallBlockSize);
		SmallBlockAllocator.Malloc(SmallBlockSize);

		CHECK(SmallBlockAllocator.GetAllocatedMemorySize() > BeforeSmall);
		// Baseline allocator with no allocations should remain at its initial size.
		CHECK(Allocator.GetAllocatedMemorySize() == Before);
	}

	SECTION("Allocations from different blocks do not overlap")
	{
		// Force multiple blocks by using a tiny block size.
		UE::FLinearBlockAllocator Allocator(nullptr, 64);
		TArray<TPair<void*, SIZE_T>> Ranges;
		constexpr SIZE_T AllocSize = 48;
		constexpr int32 Count = 8;

		for (int32 i = 0; i < Count; ++i)
		{
			void* Ptr = Allocator.Malloc(AllocSize);
			REQUIRE(Ptr != nullptr);
			Ranges.Add({Ptr, AllocSize});
		}

		for (int32 i = 0; i < Ranges.Num(); ++i)
		{
			for (int32 j = i + 1; j < Ranges.Num(); ++j)
			{
				CHECK(!DoPointersOverlap(Ranges[i].Key, Ranges[i].Value, Ranges[j].Key, Ranges[j].Value));
			}
		}
	}

	SECTION("Zero-size allocation returns non-null and does not corrupt adjacent allocation")
	{
		UE::FLinearBlockAllocator Allocator;
		void* ZeroPtr = Allocator.Malloc(0);
		CHECK(ZeroPtr != nullptr);

		// The allocation after a zero-size one should still be valid and writable.
		constexpr SIZE_T GuardSize = 64;
		uint8* GuardPtr = static_cast<uint8*>(Allocator.Malloc(GuardSize));
		REQUIRE(GuardPtr != nullptr);
		FMemory::Memset(GuardPtr, 0xCD, GuardSize);
		CHECK(GuardPtr[0] == 0xCD);
		CHECK(GuardPtr[GuardSize - 1] == 0xCD);
	}

	SECTION("Allocator does not leak memory with small allocations")
	{
		FInnerMallocForLinearAllocatorTest InnerMalloc;
		uint32 BlockSize = 128; // Allocate a small block size so we can trigger multiple blocks
		uint32 AllocationSize = 30;
		uint32 NumAllocs = 50;
		CHECK(AllocationSize * NumAllocs > BlockSize);

		{
			UE::FLinearBlockAllocator Allocator(&InnerMalloc, BlockSize);
			for (uint32 Index = 0; Index < NumAllocs; ++Index)
			{
				Allocator.Malloc(AllocationSize);
			}
		}
		CHECK(InnerMalloc.Ptrs.IsEmpty());
	}

	SECTION("Allocator does not leak memory with custom allocations")
	{
		FInnerMallocForLinearAllocatorTest InnerMalloc;
		uint32 BlockSize = 128; // Allocate a small block size to avoid using a lot of memory
		uint32 AllocationSize = 256;
		uint32 NumAllocs = 4;
		CHECK(AllocationSize * NumAllocs > BlockSize);

		{
			UE::FLinearBlockAllocator Allocator(&InnerMalloc, BlockSize);
			for (uint32 Index = 0; Index < NumAllocs; ++Index)
			{
				Allocator.Malloc(AllocationSize);
			}
		}
		CHECK(InnerMalloc.Ptrs.IsEmpty());
	}

	SECTION("Allocator does not leak memory with interleaved small and custom allocations, small first")
	{
		FInnerMallocForLinearAllocatorTest InnerMalloc;
		uint32 BlockSize = 128; // Allocate a small block size to avoid using a lot of memory
		uint32 SmallAllocationSize = 30;
		uint32 LargeAllocationSize = 256;
		uint32 NumTrials = 4;

		{
			UE::FLinearBlockAllocator Allocator(&InnerMalloc, BlockSize);
			for (uint32 Index = 0; Index < NumTrials; ++Index)
			{
				Allocator.Malloc(SmallAllocationSize);
				Allocator.Malloc(LargeAllocationSize);
			}
		}
		CHECK(InnerMalloc.Ptrs.IsEmpty());
	}

	SECTION("Allocator does not leak memory with interleaved small and custom allocations, large first")
	{
		FInnerMallocForLinearAllocatorTest InnerMalloc;
		uint32 BlockSize = 128; // Allocate a small block size to avoid using a lot of memory
		uint32 SmallAllocationSize = 30;
		uint32 LargeAllocationSize = 256;
		uint32 NumTrials = 4;

		{
			UE::FLinearBlockAllocator Allocator(&InnerMalloc, BlockSize);
			for (uint32 Index = 0; Index < NumTrials; ++Index)
			{
				Allocator.Malloc(LargeAllocationSize);
				Allocator.Malloc(SmallAllocationSize);
			}
		}
		CHECK(InnerMalloc.Ptrs.IsEmpty());
	}
}

TEST_CASE_NAMED(FLinearBlockAllocatorThreadAccessorTest,
	"System::Core::Memory::LinearBlockAllocator::ThreadAccessor", "[ApplicationContextMask][EngineFilter]")
{
	SECTION("ThreadAccessor Malloc returns valid aligned pointer")
	{
		UE::FLinearBlockAllocator Allocator(nullptr, 0, ESPMode::NotThreadSafe);
		UE::FLinearBlockAllocatorThreadAccessor Accessor(Allocator);
		void* Ptr = Accessor.Malloc(32);
		CHECK(Ptr != nullptr);
		CHECK(IsAligned(Ptr, 8));
	}

	SECTION("ThreadAccessor write does not corrupt neighbouring accessor allocation")
	{
		UE::FLinearBlockAllocator Allocator(nullptr, 1024, ESPMode::NotThreadSafe);
		UE::FLinearBlockAllocatorThreadAccessor AccessorA(Allocator);
		UE::FLinearBlockAllocatorThreadAccessor AccessorB(Allocator);

		constexpr SIZE_T Size = 128;
		uint8* A = static_cast<uint8*>(AccessorA.Malloc(Size));
		uint8* B = static_cast<uint8*>(AccessorB.Malloc(Size));
		REQUIRE(A != nullptr);
		REQUIRE(B != nullptr);

		FMemory::Memset(A, 0xAA, Size);
		FMemory::Memset(B, 0xBB, Size);

		bool bACorrect = true, bBCorrect = true;
		for (SIZE_T i = 0; i < Size; ++i)
		{
			if (A[i] != 0xAA) { bACorrect = false; }
			if (B[i] != 0xBB) { bBCorrect = false; }
		}
		CHECK(bACorrect);
		CHECK(bBCorrect);
	}

	SECTION("Move constructor transfers ownership and clears source")
	{
		UE::FLinearBlockAllocator Allocator(nullptr, 0, ESPMode::NotThreadSafe);
		UE::FLinearBlockAllocatorThreadAccessor Original(Allocator);
		// Allocate via Original to give it a cached block.
		void* FirstPtr = Original.Malloc(16);
		CHECK(FirstPtr != nullptr);

		UE::FLinearBlockAllocatorThreadAccessor Moved(MoveTemp(Original));
		// Moved accessor should work.
		void* SecondPtr = Moved.Malloc(16);
		CHECK(SecondPtr != nullptr);
	}

	SECTION("Multiple ThreadAccessors in parallel each produce non-overlapping allocations")
	{
		UE::FLinearBlockAllocator Allocator(nullptr, 0, ESPMode::NotThreadSafe);

		constexpr int32 NumThreads = 4;
		constexpr int32 AllocsPerThread = 256;
		constexpr SIZE_T AllocSize = 16;

		TArray<TArray<void*>> AllPtrs;
		AllPtrs.SetNum(NumThreads);

		ParallelFor(NumThreads, [&](int32 ThreadIdx)
		{
			UE::FLinearBlockAllocatorThreadAccessor Accessor(Allocator);
			AllPtrs[ThreadIdx].Reserve(AllocsPerThread);
			for (int32 i = 0; i < AllocsPerThread; ++i)
			{
				void* Ptr = Accessor.Malloc(AllocSize);
				REQUIRE(Ptr != nullptr);
				AllPtrs[ThreadIdx].Add(Ptr);
			}
		});

		// Flatten and verify all pointers are unique.
		TSet<void*> Seen;
		for (const TArray<void*>& ThreadPtrs : AllPtrs)
		{
			for (void* Ptr : ThreadPtrs)
			{
				bool bAlreadyInSet = false;
				Seen.Add(Ptr, &bAlreadyInSet);
				CHECK_FALSE(bAlreadyInSet);
			}
		}
	}
}

#endif // WITH_TESTS
