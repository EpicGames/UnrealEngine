// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Memory/MemoryView.h"

#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "Tests/TestHarnessAdapter.h"
#include "UObject/NameTypes.h"

#include <type_traits>

TEST_CASE_NAMED(FMemoryMemoryViewTest, "System::Core::Memory::MemoryView", "[Core][Memory][SmokeFilter]")
{
	auto TestMemoryView = [](const FMemoryView& View, const void* Data, uint64 Size)
	{
		CHECK(View.GetData() == Data);
		CHECK(View.GetDataEnd() == static_cast<const uint8*>(Data) + Size);
		CHECK(View.GetSize() == Size);
		CHECK(View.IsEmpty() == (Size == 0));
	};

	auto TestMutableMemoryView = [](const FMutableMemoryView& View, void* Data, uint64 Size)
	{
		CHECK(View.GetData() == Data);
		CHECK(View.GetDataEnd() == static_cast<uint8*>(Data) + Size);
		CHECK(View.GetSize() == Size);
		CHECK(View.IsEmpty() == (Size == 0));
	};

	struct
	{
		uint8 BeforeByteArray[4];
		uint8 ByteArray[16]{};
		uint8 AfterByteArray[4];
	} ByteArrayContainer;

	uint8 (&ByteArray)[16] = ByteArrayContainer.ByteArray;
	uint32 IntArray[12]{};

	SECTION("Constructor/Assignment Availability")
	{
		STATIC_CHECK(std::is_trivially_copyable<FMemoryView>::value);
		STATIC_CHECK(std::is_trivially_copy_constructible<FMemoryView>::value);
		STATIC_CHECK(std::is_trivially_move_constructible<FMemoryView>::value);
		STATIC_CHECK(std::is_trivially_copy_assignable<FMemoryView>::value);
		STATIC_CHECK(std::is_trivially_move_assignable<FMemoryView>::value);
		STATIC_CHECK(std::is_trivially_destructible<FMemoryView>::value);

		STATIC_CHECK(std::is_trivially_copyable<FMutableMemoryView>::value);
		STATIC_CHECK(std::is_trivially_copy_constructible<FMutableMemoryView>::value);
		STATIC_CHECK(std::is_trivially_move_constructible<FMutableMemoryView>::value);
		STATIC_CHECK(std::is_trivially_copy_assignable<FMutableMemoryView>::value);
		STATIC_CHECK(std::is_trivially_move_assignable<FMutableMemoryView>::value);
		STATIC_CHECK(std::is_trivially_destructible<FMutableMemoryView>::value);

		STATIC_CHECK(std::is_constructible<FMemoryView, const FMutableMemoryView&>::value);
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView, const FMemoryView&>::value);
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView, const void*, uint64>::value);

		STATIC_CHECK(std::is_assignable<FMemoryView, const FMutableMemoryView&>::value);
		STATIC_CHECK(!std::is_assignable<FMutableMemoryView, const FMemoryView&>::value);

		// FMemoryView is constructible from TArray
		STATIC_CHECK( std::is_constructible<FMemoryView,       TArray<      uint8>&>::value);
		STATIC_CHECK( std::is_constructible<FMemoryView,       TArray<const uint8>&>::value);
		STATIC_CHECK( std::is_constructible<FMemoryView, const TArray<      uint8>&>::value);
		STATIC_CHECK( std::is_constructible<FMemoryView, const TArray<const uint8>&>::value);

		// FMemoryView is not constructible from a temporary TArray
		STATIC_CHECK(!std::is_constructible<FMemoryView,       TArray<      uint8>>::value);
		STATIC_CHECK(!std::is_constructible<FMemoryView,       TArray<const uint8>>::value);

		// FMemoryView is constructible from TArrayView
		STATIC_CHECK( std::is_constructible<FMemoryView,       TArrayView<      uint8>&>::value);
		STATIC_CHECK( std::is_constructible<FMemoryView,       TArrayView<const uint8>&>::value);
		STATIC_CHECK( std::is_constructible<FMemoryView, const TArrayView<      uint8>&>::value);
		STATIC_CHECK( std::is_constructible<FMemoryView, const TArrayView<const uint8>&>::value);

		// FMemoryView is not constructible from a temporary TArrayView
		STATIC_CHECK(!std::is_constructible<FMemoryView,       TArrayView<      uint8>>::value);
		STATIC_CHECK(!std::is_constructible<FMemoryView,       TArrayView<const uint8>>::value);

		// FMemoryView is constructible from a raw array
		STATIC_CHECK( std::is_constructible<FMemoryView,              uint8(&)[256]>::value);
		STATIC_CHECK( std::is_constructible<FMemoryView,        const uint8(&)[256]>::value);

		// FMemoryView is not constructible from a temporary raw array
		STATIC_CHECK(!std::is_constructible<FMemoryView,              uint8   [256]>::value);

		// FMutableMemoryView is constructible from TArray when appropriate. Respects constness.
		STATIC_CHECK( std::is_constructible<FMutableMemoryView,       TArray<      uint8>&>::value);
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView,       TArray<const uint8>&>::value);
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView, const TArray<      uint8>&>::value);
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView, const TArray<const uint8>&>::value);

		// FMutableMemoryView is not constructible from a temporary TArray
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView,       TArray<      uint8>>::value);

		// FMutableMemoryView is constructible from TArrayView when appropriate. Respects constness.
		STATIC_CHECK( std::is_constructible<FMutableMemoryView,       TArrayView<      uint8>&>::value);
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView,       TArrayView<const uint8>&>::value);
		STATIC_CHECK( std::is_constructible<FMutableMemoryView, const TArrayView<      uint8>&>::value);
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView, const TArrayView<const uint8>&>::value);

		// FMutableMemoryView is not constructible from a temporary TArrayView
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView,       TArrayView<      uint8>>::value);

		// FMutableMemoryView is constructible from a raw array when appropriate. Respects constness.
		STATIC_CHECK( std::is_constructible<FMutableMemoryView,       uint8(&)[256]>::value);
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView, const uint8(&)[256]>::value);

		// FMutableMemoryView is not constructible from a temporary raw array
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView,       uint8   [256]>::value);

		// Implicitly converts TArray to a memory view when appropriate. FMutableMemoryView respects constness.
		STATIC_CHECK( std::is_convertible<      TArray<      uint8>&, FMemoryView>::value);
		STATIC_CHECK( std::is_convertible<      TArray<      uint8>&, FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<      TArray<const uint8>&, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      TArray<const uint8>&, FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<const TArray<      uint8>&, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<const TArray<      uint8>&, FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<const TArray<const uint8>&, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<const TArray<const uint8>&, FMutableMemoryView>::value);

		// Can't implicitly convert a temporary TArray to a non-owning memory view
		STATIC_CHECK(!std::is_convertible<      TArray<      uint8>, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      TArray<      uint8>, FMutableMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      TArray<const uint8>, FMemoryView>::value);

		// Implicitly converts TStaticArray to a memory view when appropriate. FMutableMemoryView respects constness.
		STATIC_CHECK( std::is_convertible<      TStaticArray<      uint8, 32>&, FMemoryView>::value);
		STATIC_CHECK( std::is_convertible<      TStaticArray<      uint8, 32>&, FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<      TStaticArray<const uint8, 32>&, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      TStaticArray<const uint8, 32>&, FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<const TStaticArray<      uint8, 32>&, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<const TStaticArray<      uint8, 32>&, FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<const TStaticArray<const uint8, 32>&, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<const TStaticArray<const uint8, 32>&, FMutableMemoryView>::value);

		// Can't implicitly convert a temporary TStaticArray to a non-owning memory view
		STATIC_CHECK(!std::is_convertible<      TStaticArray<      uint8, 32>, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      TStaticArray<      uint8, 32>, FMutableMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      TStaticArray<const uint8, 32>, FMemoryView>::value);

		// Implicitly converts TArrayView to a memory view when appropriate. FMutableMemoryView respects constness.
		STATIC_CHECK( std::is_convertible<      TArrayView<      uint8>&, FMemoryView>::value);
		STATIC_CHECK( std::is_convertible<      TArrayView<      uint8>&, FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<      TArrayView<const uint8>&, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      TArrayView<const uint8>&, FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<const TArrayView<      uint8>&, FMemoryView>::value);
		STATIC_CHECK( std::is_convertible<const TArrayView<      uint8>&, FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<const TArrayView<const uint8>&, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<const TArrayView<const uint8>&, FMutableMemoryView>::value);

		// Can't implicitly convert a temporary TArrayView to a non-owning memory view
		// Note that ideally this would be allowed (this is safe for any non-owning range) but we, as of now,
		// don't have a way to differentiate owning from non-owning ranges so choose to take the safe route.
		STATIC_CHECK(!std::is_convertible<      TArrayView<      uint8>, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      TArrayView<      uint8>, FMutableMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      TArrayView<const uint8>, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      TArrayView<const uint8>, FMutableMemoryView>::value);

		// Implicitly converts raw arrays to a memory view when appropriate. FMutableMemoryView respects constness.
		STATIC_CHECK( std::is_convertible<      uint8(&)[256], FMemoryView>::value);
		STATIC_CHECK( std::is_convertible<      uint8(&)[256], FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<const uint8(&)[256], FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<const uint8(&)[256], FMutableMemoryView>::value);
		STATIC_CHECK( std::is_convertible<const uint8(&)[  0], FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<const uint8(&)[  0], FMutableMemoryView>::value);

		// Can't implicitly convert a raw array temporary
		STATIC_CHECK(!std::is_convertible<      uint8   [256], FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<      uint8   [256], FMutableMemoryView>::value);

		// Implicitly converts TEXT() when appropriate
		STATIC_CHECK( std::is_convertible<decltype(TEXT("HELLO")), FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<decltype(TEXT("HELLO")), FMutableMemoryView>::value);
		
		// Allows assignment of an ArrayView to a MemoryView when appropriate. FMutableMemoryView respects constness.
		STATIC_CHECK( std::is_assignable<      FMemoryView,        TArrayView<      uint8>&>::value);
		STATIC_CHECK( std::is_assignable<      FMutableMemoryView, TArrayView<      uint8>&>::value);
		STATIC_CHECK( std::is_assignable<      FMemoryView,        TArrayView<const uint8>&>::value);
		STATIC_CHECK(!std::is_assignable<      FMutableMemoryView, TArrayView<const uint8>&>::value);
		STATIC_CHECK(!std::is_assignable<const FMemoryView,        TArrayView<      uint8>&>::value);
		STATIC_CHECK(!std::is_assignable<const FMutableMemoryView, TArrayView<      uint8>&>::value);
		STATIC_CHECK(!std::is_assignable<const FMemoryView,        TArrayView<const uint8>&>::value);
		STATIC_CHECK(!std::is_assignable<const FMutableMemoryView, TArrayView<const uint8>&>::value);

		// Can't assign a temporary ArrayView to a MemoryView
		// Note that ideally this would be allowed (this is safe for any non-owning range) but we, as of now,
		// don't have a way to differentiate owning from non-owning ranges so choose to take the safe route.
		STATIC_CHECK(!std::is_assignable<      FMemoryView,        TArrayView<      uint8>>::value);
		STATIC_CHECK(!std::is_assignable<      FMutableMemoryView, TArrayView<      uint8>>::value);
		STATIC_CHECK(!std::is_assignable<      FMemoryView,        TArrayView<const uint8>>::value);
		STATIC_CHECK(!std::is_assignable<const FMemoryView,        TArrayView<      uint8>>::value);
		STATIC_CHECK(!std::is_assignable<const FMutableMemoryView, TArrayView<      uint8>>::value);
		STATIC_CHECK(!std::is_assignable<const FMemoryView,        TArrayView<const uint8>>::value);
		STATIC_CHECK(!std::is_assignable<const FMutableMemoryView, TArrayView<const uint8>>::value);

		// Some implementations of std::is_assignable<> incorrectly evaluate assignment from raw arrays. If a 
		// basic assignment from an array to a pointer of the same type doesn't work, then we expect all of the
		// following positive tests to fail too. Still checking for failure makes sure we're not incorrectly
		// expecting failure on a platform where is_assignable<> would actually work for these tests.
		constexpr bool bIsAssignableWorksForRawArrays = std::is_assignable<uint8*&, uint8(&)[256]>::value;
		STATIC_CHECK( std::is_assignable<FMemoryView,              uint8(&)[256]>::value == bIsAssignableWorksForRawArrays);
		STATIC_CHECK( std::is_assignable<FMutableMemoryView,       uint8(&)[256]>::value == bIsAssignableWorksForRawArrays);
		STATIC_CHECK( std::is_assignable<FMemoryView,        const uint8(&)[256]>::value == bIsAssignableWorksForRawArrays);
		STATIC_CHECK( std::is_assignable<FMutableMemoryView, const uint8(&)[256]>::value == false);

		// Can't assign a temporary raw array to a MemoryView
		STATIC_CHECK( std::is_assignable<FMemoryView,              uint8   [256]>::value == false);
		STATIC_CHECK( std::is_assignable<FMutableMemoryView,       uint8   [256]>::value == false);

		// Can't implicitly cast underlying shapeless memory via assignment of a MemoryView to an ArrayView
		STATIC_CHECK(!std::is_assignable<      TArrayView<      uint8>, FMemoryView&>::value);
		STATIC_CHECK(!std::is_assignable<      TArrayView<      uint8>, FMutableMemoryView&>::value);
		STATIC_CHECK(!std::is_assignable<      TArrayView<const uint8>, FMemoryView&>::value);
		STATIC_CHECK(!std::is_assignable<      TArrayView<const uint8>, FMutableMemoryView&>::value);
		STATIC_CHECK(!std::is_assignable<const TArrayView<      uint8>, FMemoryView&>::value);
		STATIC_CHECK(!std::is_assignable<const TArrayView<      uint8>, FMutableMemoryView&>::value);
		STATIC_CHECK(!std::is_assignable<const TArrayView<const uint8>, FMemoryView&>::value);
		STATIC_CHECK(!std::is_assignable<const TArrayView<const uint8>, FMutableMemoryView&>::value);

		// Doesn't allows FMutableMemoryView construction from FString
		STATIC_CHECK( std::is_constructible<FMemoryView, FString&>::value);
		STATIC_CHECK(!std::is_constructible<FMutableMemoryView, FString&>::value);

		// Can't construct a MemoryView from a temporary FString
		STATIC_CHECK(!std::is_constructible<FMemoryView, FString>::value);

		// Doesn't implicitly convert FName
		STATIC_CHECK(!std::is_convertible<FName&, FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<FName&, FMutableMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<FName , FMemoryView>::value);
		STATIC_CHECK(!std::is_convertible<FName , FMutableMemoryView>::value);
	}

	SECTION("Empty Views")
	{
		TestMemoryView(FMemoryView(), nullptr, 0);
		TestMemoryView(FMutableMemoryView(), nullptr, 0);
		TestMutableMemoryView(FMutableMemoryView(), nullptr, 0);
	}

	SECTION("Construction from Type[], TArrayView, (Type*, uint64), (Type*, Type*)")
	{
		TestMemoryView(MakeMemoryView(AsConst(IntArray)), IntArray, sizeof(IntArray));
		TestMemoryView(MakeMemoryView(MakeArrayView(AsConst(IntArray))), IntArray, sizeof(IntArray));
		TestMemoryView(MakeMemoryView(AsConst(IntArray), sizeof(IntArray)), IntArray, sizeof(IntArray));
		TestMemoryView(MakeMemoryView(AsConst(IntArray), AsConst(IntArray) + 6), IntArray, sizeof(*IntArray) * 6);
		TestMutableMemoryView(MakeMemoryView(IntArray), IntArray, sizeof(IntArray));
		TestMutableMemoryView(MakeMemoryView(MakeArrayView(IntArray)), IntArray, sizeof(IntArray));
		TestMutableMemoryView(MakeMemoryView(IntArray, sizeof(IntArray)), IntArray, sizeof(IntArray));
		TestMutableMemoryView(MakeMemoryView(IntArray, IntArray + 6), IntArray, sizeof(*IntArray) * 6);
	}

	SECTION("Construction from std::initializer_list")
	{
		//MakeMemoryView({1, 2, 3}); // fail because the type must be deduced
		std::initializer_list<uint8> InitializerList{1, 2, 3};
		TestMemoryView(MakeMemoryView(InitializerList), GetData(InitializerList), GetNum(InitializerList) * sizeof(uint8));
	}

	SECTION("Construction from TArray")
	{
		TArray64<uint32> Array(GetData(IntArray), GetNum(IntArray));
		const TArray64<uint32> ConstArray(GetData(IntArray), GetNum(IntArray));

		FMemoryView MemoryView(Array);
		TestMemoryView(FMemoryView(MemoryView), Array.GetData(), Array.NumBytes());

		FMemoryView MemoryViewOfConst(ConstArray);
		TestMemoryView(FMemoryView(MemoryViewOfConst), ConstArray.GetData(), ConstArray.NumBytes());

		FMutableMemoryView MutableMemoryView(Array);
		TestMutableMemoryView(MutableMemoryView, Array.GetData(), ConstArray.NumBytes());
	}

	SECTION("Construction from TArrayView")
	{
		const TArrayView<uint32> ArrayView = MakeArrayView(IntArray);
		const TArrayView<const uint32> ConstArrayView = MakeConstArrayView(IntArray);

		FMemoryView MemoryView(ArrayView);
		TestMemoryView(FMemoryView(MemoryView), IntArray, sizeof(IntArray));

		FMemoryView MemoryViewOfConst(ConstArrayView);
		TestMemoryView(FMemoryView(MemoryViewOfConst), GetData(IntArray), sizeof(IntArray));

		FMutableMemoryView MutableMemoryView(ArrayView);
		TestMutableMemoryView(MutableMemoryView, GetData(IntArray), sizeof(IntArray));
	}

	SECTION("Construction from TStaticArray")
	{
		TStaticArray<uint32, 16> StaticArray = {};
		const TStaticArray<uint32, 16> ConstStaticArray = {};

		const uint64 StaticArrayBytes = StaticArray.Num() * sizeof(decltype(StaticArray)::ElementType);
		const uint64 ConstStaticArrayBytes = ConstStaticArray.Num() * sizeof(decltype(ConstStaticArray)::ElementType);

		FMemoryView MemoryView(StaticArray);
		TestMemoryView(FMemoryView(MemoryView), GetData(StaticArray), StaticArrayBytes);

		FMemoryView MemoryViewOfConst(ConstStaticArray);
		TestMemoryView(FMemoryView(MemoryViewOfConst), GetData(ConstStaticArray), ConstStaticArrayBytes);

		FMutableMemoryView MutableMemoryView(StaticArray);
		TestMutableMemoryView(MutableMemoryView, GetData(StaticArray), StaticArrayBytes);
	}

	SECTION("Construction from Raw Array")
	{
		TestMemoryView(FMemoryView(IntArray), IntArray, sizeof(IntArray)); 
		TestMutableMemoryView(FMutableMemoryView(IntArray), IntArray, sizeof(IntArray));
	}

	SECTION("Reset")
	{
		FMutableMemoryView View = MakeMemoryView(IntArray);
		View.Reset();
		CHECK(View == FMutableMemoryView());
	}

	SECTION("Left")
	{
		STATIC_CHECK(MakeMemoryView(IntArray).Left(0).IsEmpty());
		STATIC_CHECK(MakeMemoryView(IntArray).Left(1) == MakeMemoryView(IntArray, 1));
		STATIC_CHECK(MakeMemoryView(IntArray).Left(sizeof(IntArray)) == MakeMemoryView(IntArray));
		STATIC_CHECK(MakeMemoryView(IntArray).Left(sizeof(IntArray) + 1) == MakeMemoryView(IntArray));
		STATIC_CHECK(MakeMemoryView(IntArray).Left(MAX_uint64) == MakeMemoryView(IntArray));
	}

	SECTION("LeftChop")
	{
		STATIC_CHECK(MakeMemoryView(IntArray).LeftChop(0) == MakeMemoryView(IntArray));
		STATIC_CHECK(MakeMemoryView(IntArray).LeftChop(1) == MakeMemoryView(IntArray, sizeof(IntArray) - 1));
		STATIC_CHECK(MakeMemoryView(IntArray).LeftChop(sizeof(IntArray)).IsEmpty());
		STATIC_CHECK(MakeMemoryView(IntArray).LeftChop(sizeof(IntArray) + 1).IsEmpty());
		STATIC_CHECK(MakeMemoryView(IntArray).LeftChop(MAX_uint64).IsEmpty());
	}

	SECTION("Right")
	{
		CHECK(MakeMemoryView(IntArray).Right(0) == FMutableMemoryView());
		CHECK(MakeMemoryView(IntArray).Right(1) == MakeMemoryView(reinterpret_cast<uint8*>(IntArray) + sizeof(IntArray) - 1, 1));
		CHECK(MakeMemoryView(IntArray).Right(sizeof(IntArray)) == MakeMemoryView(IntArray));
		CHECK(MakeMemoryView(IntArray).Right(sizeof(IntArray) + 1) == MakeMemoryView(IntArray));
		CHECK(MakeMemoryView(IntArray).Right(MAX_uint64) == MakeMemoryView(IntArray));
	}

	SECTION("RightChop")
	{
		CHECK(MakeMemoryView(IntArray).RightChop(0) == MakeMemoryView(IntArray));
		CHECK(MakeMemoryView(IntArray).RightChop(1) == MakeMemoryView(reinterpret_cast<uint8*>(IntArray) + 1, sizeof(IntArray) - 1));
		CHECK(MakeMemoryView(IntArray).RightChop(sizeof(IntArray)) == FMutableMemoryView());
		CHECK(MakeMemoryView(IntArray).RightChop(sizeof(IntArray) + 1) == FMutableMemoryView());
		CHECK(MakeMemoryView(IntArray).RightChop(MAX_uint64) == FMutableMemoryView());
	}

	SECTION("Mid")
	{
		CHECK(MakeMemoryView(IntArray).Mid(0) == MakeMemoryView(IntArray));
		CHECK(MakeMemoryView(IntArray).Mid(1) == MakeMemoryView(reinterpret_cast<uint8*>(IntArray) + 1, sizeof(IntArray) - 1));
		CHECK(MakeMemoryView(IntArray).Mid(sizeof(IntArray)) == FMutableMemoryView());
		CHECK(MakeMemoryView(IntArray).Mid(sizeof(IntArray) + 1) == FMutableMemoryView());
		CHECK(MakeMemoryView(IntArray).Mid(MAX_uint64) == FMutableMemoryView());
		CHECK(MakeMemoryView(IntArray).Mid(0, 0) == FMutableMemoryView());
		CHECK(MakeMemoryView(IntArray).Mid(0, 1) == MakeMemoryView(IntArray, 1));
		CHECK(MakeMemoryView(IntArray).Mid(1, sizeof(IntArray) - 2) == MakeMemoryView(reinterpret_cast<uint8*>(IntArray) + 1, sizeof(IntArray) - 2));
		CHECK(MakeMemoryView(IntArray).Mid(1, sizeof(IntArray) - 1) == MakeMemoryView(reinterpret_cast<uint8*>(IntArray) + 1, sizeof(IntArray) - 1));
		CHECK(MakeMemoryView(IntArray).Mid(1, sizeof(IntArray)) == MakeMemoryView(reinterpret_cast<uint8*>(IntArray) + 1, sizeof(IntArray) - 1));
		CHECK(MakeMemoryView(IntArray).Mid(0, MAX_uint64) == MakeMemoryView(IntArray));
		CHECK(MakeMemoryView(IntArray).Mid(MAX_uint64, MAX_uint64) == FMutableMemoryView());
	}

	SECTION("Contains")
	{
		CHECK(FMemoryView().Contains(FMutableMemoryView()));
		CHECK(FMutableMemoryView().Contains(FMemoryView()));
		CHECK(MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArray)));
		CHECK(MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArray + 1, 15)));
		CHECK(MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArray, 15)));
		CHECK(MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArray + 1, 14)));
		CHECK(MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArray, 0)));
		CHECK(MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArray + 8, 0)));
		CHECK(MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArray + 16, 0)));
		CHECK(!MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArrayContainer.BeforeByteArray + 3, 0)));
		CHECK(!MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArrayContainer.AfterByteArray + 1, 0)));
		CHECK(!MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArrayContainer.BeforeByteArray + 3, 1)));
		CHECK(!MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArray + 16, 1)));
		CHECK(!MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArrayContainer.BeforeByteArray + 3, 17)));
		CHECK(!MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArray, 17)));
		CHECK(!MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArrayContainer.BeforeByteArray + 3, 18)));
		CHECK(!MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArrayContainer.BeforeByteArray + 3, 8)));
		CHECK(!MakeMemoryView(ByteArray).Contains(MakeMemoryView(ByteArray + 9, 8)));
	}

	SECTION("Intersects")
	{
		CHECK(MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArray)));
		CHECK(MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArray + 1, 15)));
		CHECK(MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArray, 15)));
		CHECK(MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArray + 1, 14)));
		CHECK(MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArrayContainer.BeforeByteArray + 3, 8)));
		CHECK(MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArray + 9, 8)));
		CHECK(MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArrayContainer.BeforeByteArray + 3, 17)));
		CHECK(MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArray, 17)));
		CHECK(MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArrayContainer.BeforeByteArray + 3, 18)));
		CHECK(MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArray + 8, 0)));
		CHECK(!FMemoryView().Intersects(FMutableMemoryView()));
		CHECK(!FMutableMemoryView().Intersects(FMemoryView()));
		CHECK(!MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArray, 0)));
		CHECK(!MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArray + 16, 0)));
		CHECK(!MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArrayContainer.BeforeByteArray + 3, 0)));
		CHECK(!MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArrayContainer.AfterByteArray + 1, 0)));
		CHECK(!MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArrayContainer.BeforeByteArray + 3, 1)));
		CHECK(!MakeMemoryView(ByteArray).Intersects(MakeMemoryView(ByteArray + 16, 1)));
	}

	SECTION("CompareBytes")
	{
		const uint8 CompareBytes[8] = { 5, 4, 6, 2, 4, 7, 1, 3 };
		CHECK(FMemoryView().CompareBytes(FMutableMemoryView()) == 0);
		CHECK(FMutableMemoryView().CompareBytes(FMemoryView()) == 0);
		CHECK(MakeMemoryView(ByteArray).CompareBytes(MakeMemoryView(ByteArray)) == 0);
		CHECK(MakeMemoryView(ByteArray, 8).CompareBytes(MakeMemoryView(ByteArray + 8, 8)) == 0);
		CHECK(FMemoryView().CompareBytes(MakeMemoryView(ByteArray)) < 0);
		CHECK(MakeMemoryView(ByteArray).CompareBytes(FMemoryView()) > 0);
		CHECK(MakeMemoryView(ByteArray, 8).CompareBytes(MakeMemoryView(ByteArray)) < 0);
		CHECK(MakeMemoryView(ByteArray).CompareBytes(MakeMemoryView(ByteArray, 8)) > 0);
		CHECK(MakeMemoryView(IntArray, 8).CompareBytes(MakeMemoryView(ByteArray)) < 0);
		CHECK(MakeMemoryView(ByteArray).CompareBytes(MakeMemoryView(IntArray, 8)) > 0);
		CHECK(MakeMemoryView(ByteArray, 4).CompareBytes(MakeMemoryView(ByteArray, 8)) < 0);
		CHECK(MakeMemoryView(ByteArray, 8).CompareBytes(MakeMemoryView(ByteArray, 4)) > 0);
		CHECK(MakeMemoryView(CompareBytes, 2).CompareBytes(MakeMemoryView(CompareBytes + 2, 2)) < 0);
		CHECK(MakeMemoryView(CompareBytes, 3).CompareBytes(MakeMemoryView(CompareBytes + 3, 3)) > 0);
	}

	SECTION("EqualBytes")
	{
		const uint8 CompareBytes[8] = { 5, 4, 6, 2, 4, 7, 1, 3 };
		CHECK(FMemoryView().EqualBytes(FMutableMemoryView()));
		CHECK(FMutableMemoryView().EqualBytes(FMemoryView()));
		CHECK(MakeMemoryView(ByteArray).EqualBytes(MakeMemoryView(ByteArray)));
		CHECK(MakeMemoryView(ByteArray, 8).EqualBytes(MakeMemoryView(ByteArray + 8, 8)));
		CHECK(!MakeMemoryView(ByteArray, 8).EqualBytes(MakeMemoryView(ByteArray, 4)));
		CHECK(!MakeMemoryView(ByteArray, 4).EqualBytes(MakeMemoryView(ByteArray, 8)));
		CHECK(!MakeMemoryView(CompareBytes, 4).EqualBytes(MakeMemoryView(CompareBytes + 4, 4)));
	}

	SECTION("Equals")
	{
		CHECK(FMemoryView().Equals(FMemoryView()));
		CHECK(FMemoryView().Equals(FMutableMemoryView()));
		CHECK(FMutableMemoryView().Equals(FMemoryView()));
		CHECK(FMutableMemoryView().Equals(FMutableMemoryView()));
		CHECK(MakeMemoryView(IntArray).Equals(MakeMemoryView(AsConst(IntArray))));
		CHECK(!MakeMemoryView(IntArray).Equals(MakeMemoryView(IntArray + 1, sizeof(IntArray) - sizeof(*IntArray))));
		CHECK(!MakeMemoryView(IntArray).Equals(MakeMemoryView(IntArray, sizeof(*IntArray))));
		CHECK(!MakeMemoryView(IntArray).Equals(FMutableMemoryView()));
	}

	SECTION("operator==")
	{
		STATIC_CHECK(MakeMemoryView(ByteArrayContainer.ByteArray) == MakeMemoryView(ByteArrayContainer.ByteArray)); //-V501
		STATIC_CHECK(MakeMemoryView(ByteArrayContainer.ByteArray) == MakeMemoryView(AsConst(ByteArrayContainer.ByteArray)));
		STATIC_CHECK(MakeMemoryView(AsConst(ByteArrayContainer.ByteArray)) == MakeMemoryView(ByteArrayContainer.ByteArray));
		STATIC_CHECK(MakeMemoryView(AsConst(ByteArrayContainer.ByteArray)) == MakeMemoryView(AsConst(ByteArrayContainer.ByteArray))); //-V501
	}

	SECTION("operator!=")
	{
		STATIC_CHECK(MakeMemoryView(ByteArrayContainer.ByteArray) != MakeMemoryView(IntArray));
		STATIC_CHECK(MakeMemoryView(ByteArrayContainer.ByteArray) != MakeMemoryView(AsConst(IntArray)));
		STATIC_CHECK(MakeMemoryView(AsConst(ByteArrayContainer.ByteArray)) != MakeMemoryView(IntArray));
		STATIC_CHECK(MakeMemoryView(AsConst(ByteArrayContainer.ByteArray)) != MakeMemoryView(AsConst(IntArray)));
	}

	SECTION("operator+=")
	{
		CHECK((MakeMemoryView(ByteArray) += 0) == MakeMemoryView(ByteArray));
		CHECK((MakeMemoryView(ByteArray) += 8) == MakeMemoryView(ByteArray + 8, 8));
		CHECK((MakeMemoryView(ByteArray) += 16) == MakeMemoryView(ByteArray + 16, 0));
		CHECK((MakeMemoryView(ByteArray) += 32) == MakeMemoryView(ByteArray + 16, 0));
	}

	SECTION("operator+")
	{
		CHECK((MakeMemoryView(ByteArray) + 0) == MakeMemoryView(ByteArray));
		CHECK((0 + MakeMemoryView(ByteArray)) == MakeMemoryView(ByteArray));
		CHECK((MakeMemoryView(ByteArray) + 8) == MakeMemoryView(ByteArray + 8, 8));
		CHECK((8 + MakeMemoryView(ByteArray)) == MakeMemoryView(ByteArray + 8, 8));
		CHECK((MakeMemoryView(ByteArray) + 16) == MakeMemoryView(ByteArray + 16, 0));
		CHECK((16 + MakeMemoryView(ByteArray)) == MakeMemoryView(ByteArray + 16, 0));
		CHECK((MakeMemoryView(ByteArray) + 32) == MakeMemoryView(ByteArray + 16, 0));
		CHECK((32 + MakeMemoryView(ByteArray)) == MakeMemoryView(ByteArray + 16, 0));
	}

	SECTION("Assignment from TArrayView")
	{
		const TArrayView<uint32> ArrayView = MakeArrayView(IntArray);
		const TArrayView<const uint32> ConstArrayView = MakeConstArrayView(IntArray);

		FMemoryView MemoryView = ArrayView;
		TestMemoryView(MemoryView, IntArray, sizeof(IntArray));
		
		FMemoryView MemoryViewOfConst = ConstArrayView;
		TestMemoryView(MemoryViewOfConst, IntArray, sizeof(IntArray));

		FMutableMemoryView MutableMemoryView = ArrayView;
		TestMutableMemoryView(MutableMemoryView, IntArray, sizeof(IntArray));
	}

	SECTION("Assignment from TArray")
	{
		TArray64<uint32> Array(GetData(IntArray), GetNum(IntArray));
		const TArray64<uint32> ConstArray(GetData(IntArray), GetNum(IntArray));

		FMemoryView MemoryView = Array;
		TestMemoryView(MemoryView, Array.GetData(), Array.NumBytes());
		
		FMemoryView MemoryViewOfConst = ConstArray;
		TestMemoryView(MemoryViewOfConst, ConstArray.GetData(), ConstArray.NumBytes());

		FMutableMemoryView MutableMemoryView = Array;
		TestMutableMemoryView(MutableMemoryView, Array.GetData(), Array.NumBytes());
	}

	SECTION("Assignment from TStaticArray")
	{
		TStaticArray<uint64, 21> Array = {};
		const TStaticArray<uint16, 44> ConstArray = {};

		FMemoryView MemoryView = Array;
		TestMemoryView(MemoryView, Array.GetData(), Array.Num() * sizeof(decltype(Array)::ElementType));
		
		FMemoryView MemoryViewOfConst = ConstArray;
		TestMemoryView(MemoryViewOfConst, ConstArray.GetData(), ConstArray.Num() * sizeof(decltype(ConstArray)::ElementType));

		FMutableMemoryView MutableMemoryView = Array;
		TestMutableMemoryView(MutableMemoryView, Array.GetData(), Array.Num() * sizeof(decltype(Array)::ElementType));
	}

	SECTION("Assignment from Raw Array")
	{
		FMemoryView MemoryView = IntArray;
		TestMemoryView(MemoryView, IntArray, sizeof(IntArray));

		FMutableMemoryView MutableMemoryView = IntArray;
		TestMutableMemoryView(MutableMemoryView, IntArray, sizeof(IntArray));
	}
}

#endif // WITH_TESTS
