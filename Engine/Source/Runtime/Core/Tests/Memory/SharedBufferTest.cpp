// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Memory/SharedBuffer.h"
#include "Tests/TestHarnessAdapter.h"
#include <type_traits>
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert( std::is_constructible<FUniqueBuffer>::value, "Missing constructor");
static_assert( std::is_constructible<FUniqueBuffer, FUniqueBuffer&&>::value, "Missing constructor");
static_assert(!std::is_constructible<FUniqueBuffer, FSharedBuffer&&>::value, "Invalid constructor");
static_assert(!std::is_constructible<FUniqueBuffer, FWeakSharedBuffer&&>::value, "Invalid constructor");
static_assert(!std::is_constructible<FUniqueBuffer, const FUniqueBuffer&>::value, "Invalid constructor");
static_assert(!std::is_constructible<FUniqueBuffer, const FSharedBuffer&>::value, "Invalid constructor");
static_assert(!std::is_constructible<FUniqueBuffer, const FWeakSharedBuffer&>::value, "Invalid constructor");

static_assert( std::is_constructible<FSharedBuffer>::value, "Missing constructor");
static_assert(!std::is_constructible<FSharedBuffer, FUniqueBuffer&&>::value, "Invalid constructor");
static_assert( std::is_constructible<FSharedBuffer, FSharedBuffer&&>::value, "Missing constructor");
static_assert(!std::is_constructible<FSharedBuffer, FWeakSharedBuffer&&>::value, "Invalid constructor");
static_assert(!std::is_constructible<FSharedBuffer, const FUniqueBuffer&>::value, "Invalid constructor");
static_assert( std::is_constructible<FSharedBuffer, const FSharedBuffer&>::value, "Missing constructor");
static_assert(!std::is_constructible<FSharedBuffer, const FWeakSharedBuffer&>::value, "Invalid constructor");

static_assert( std::is_constructible<FWeakSharedBuffer>::value, "Missing constructor");
static_assert(!std::is_constructible<FWeakSharedBuffer, FUniqueBuffer&&>::value, "Invalid constructor");
static_assert( std::is_constructible<FWeakSharedBuffer, FSharedBuffer&&>::value, "Missing constructor");
static_assert( std::is_constructible<FWeakSharedBuffer, FWeakSharedBuffer&&>::value, "Missing constructor");
static_assert(!std::is_constructible<FWeakSharedBuffer, const FUniqueBuffer&>::value, "Invalid constructor");
static_assert( std::is_constructible<FWeakSharedBuffer, const FSharedBuffer&>::value, "Missing constructor");
static_assert( std::is_constructible<FWeakSharedBuffer, const FWeakSharedBuffer&>::value, "Missing constructor");

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert( std::is_assignable<FUniqueBuffer, FUniqueBuffer&&>::value, "Missing assignment");
static_assert(!std::is_assignable<FUniqueBuffer, FSharedBuffer&&>::value, "Invalid assignment");
static_assert(!std::is_assignable<FUniqueBuffer, FWeakSharedBuffer&&>::value, "Invalid assignment");
static_assert(!std::is_assignable<FUniqueBuffer, const FUniqueBuffer&>::value, "Invalid assignment");
static_assert(!std::is_assignable<FUniqueBuffer, const FSharedBuffer&>::value, "Invalid assignment");
static_assert(!std::is_assignable<FUniqueBuffer, const FWeakSharedBuffer&>::value, "Invalid assignment");

static_assert(!std::is_assignable<FSharedBuffer, FUniqueBuffer&&>::value, "Invalid assignment");
static_assert( std::is_assignable<FSharedBuffer, FSharedBuffer&&>::value, "Missing assignment");
static_assert(!std::is_assignable<FSharedBuffer, FWeakSharedBuffer&&>::value, "Invalid assignment");
static_assert(!std::is_assignable<FSharedBuffer, const FUniqueBuffer&>::value, "Invalid assignment");
static_assert( std::is_assignable<FSharedBuffer, const FSharedBuffer&>::value, "Missing assignment");
static_assert(!std::is_assignable<FSharedBuffer, const FWeakSharedBuffer&>::value, "Invalid assignment");

static_assert(!std::is_assignable<FWeakSharedBuffer, FUniqueBuffer&&>::value, "Invalid assignment");
static_assert( std::is_assignable<FWeakSharedBuffer, FSharedBuffer&&>::value, "Missing assignment");
static_assert( std::is_assignable<FWeakSharedBuffer, FWeakSharedBuffer&&>::value, "Missing assignment");
static_assert(!std::is_assignable<FWeakSharedBuffer, const FUniqueBuffer&>::value, "Invalid assignment");
static_assert( std::is_assignable<FWeakSharedBuffer, const FSharedBuffer&>::value, "Missing assignment");
static_assert( std::is_assignable<FWeakSharedBuffer, const FWeakSharedBuffer&>::value, "Missing assignment");

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert(std::is_same<      void*, decltype(DeclVal<      FUniqueBuffer>().GetData())>::value, "Invalid accessor");
static_assert(std::is_same<const void*, decltype(DeclVal<const FUniqueBuffer>().GetData())>::value, "Invalid accessor");
static_assert(std::is_same<const void*, decltype(DeclVal<      FSharedBuffer>().GetData())>::value, "Invalid accessor");
static_assert(std::is_same<const void*, decltype(DeclVal<const FSharedBuffer>().GetData())>::value, "Invalid accessor");

static_assert(std::is_same<uint64, decltype(DeclVal<FUniqueBuffer>().GetSize())>::value, "Invalid accessor");
static_assert(std::is_same<uint64, decltype(DeclVal<FSharedBuffer>().GetSize())>::value, "Invalid accessor");

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert(std::is_same<FMutableMemoryView, decltype(DeclVal<      FUniqueBuffer>().GetView())>::value, "Invalid accessor");
static_assert(std::is_same<       FMemoryView, decltype(DeclVal<const FUniqueBuffer>().GetView())>::value, "Invalid accessor");
static_assert(std::is_same<       FMemoryView, decltype(DeclVal<      FSharedBuffer>().GetView())>::value, "Invalid accessor");
static_assert(std::is_same<       FMemoryView, decltype(DeclVal<const FSharedBuffer>().GetView())>::value, "Invalid accessor");

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert( std::is_convertible<      FUniqueBuffer, FMutableMemoryView>::value, "Missing conversion");
static_assert(!std::is_convertible<const FUniqueBuffer, FMutableMemoryView>::value, "Invalid conversion");
static_assert( std::is_convertible<      FUniqueBuffer, FMemoryView>::value, "Missing conversion");
static_assert( std::is_convertible<const FUniqueBuffer, FMemoryView>::value, "Missing conversion");

static_assert(!std::is_convertible<      FSharedBuffer, FMutableMemoryView>::value, "Invalid conversion");
static_assert(!std::is_convertible<const FSharedBuffer, FMutableMemoryView>::value, "Invalid conversion");
static_assert( std::is_convertible<      FSharedBuffer, FMemoryView>::value, "Missing conversion");
static_assert( std::is_convertible<const FSharedBuffer, FMemoryView>::value, "Missing conversion");

static_assert(!std::is_convertible<      FWeakSharedBuffer, FMutableMemoryView>::value, "Invalid conversion");
static_assert(!std::is_convertible<const FWeakSharedBuffer, FMutableMemoryView>::value, "Invalid conversion");
static_assert(!std::is_convertible<      FWeakSharedBuffer, FMemoryView>::value, "Invalid conversion");
static_assert(!std::is_convertible<const FWeakSharedBuffer, FMemoryView>::value, "Invalid conversion");

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert(std::is_same<bool, decltype(DeclVal<const FUniqueBuffer&>() == DeclVal<const FUniqueBuffer&>())>::value, "Invalid equality");
static_assert(std::is_same<bool, decltype(DeclVal<const FUniqueBuffer&>() == DeclVal<const FSharedBuffer&>())>::value, "Invalid equality");
static_assert(std::is_same<bool, decltype(DeclVal<const FUniqueBuffer&>() == DeclVal<const FWeakSharedBuffer&>())>::value, "Invalid equality");
static_assert(std::is_same<bool, decltype(DeclVal<const FSharedBuffer&>() == DeclVal<const FUniqueBuffer&>())>::value, "Invalid equality");
static_assert(std::is_same<bool, decltype(DeclVal<const FSharedBuffer&>() == DeclVal<const FSharedBuffer&>())>::value, "Invalid equality");
static_assert(std::is_same<bool, decltype(DeclVal<const FSharedBuffer&>() == DeclVal<const FWeakSharedBuffer&>())>::value, "Invalid equality");
static_assert(std::is_same<bool, decltype(DeclVal<const FWeakSharedBuffer&>() == DeclVal<const FUniqueBuffer&>())>::value, "Invalid equality");
static_assert(std::is_same<bool, decltype(DeclVal<const FWeakSharedBuffer&>() == DeclVal<const FSharedBuffer&>())>::value, "Invalid equality");
static_assert(std::is_same<bool, decltype(DeclVal<const FWeakSharedBuffer&>() == DeclVal<const FWeakSharedBuffer&>())>::value, "Invalid equality");

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert(std::is_same<bool, decltype(DeclVal<const FUniqueBuffer&>() != DeclVal<const FUniqueBuffer&>())>::value, "Invalid inequality");
static_assert(std::is_same<bool, decltype(DeclVal<const FUniqueBuffer&>() != DeclVal<const FSharedBuffer&>())>::value, "Invalid inequality");
static_assert(std::is_same<bool, decltype(DeclVal<const FUniqueBuffer&>() != DeclVal<const FWeakSharedBuffer&>())>::value, "Invalid inequality");
static_assert(std::is_same<bool, decltype(DeclVal<const FSharedBuffer&>() != DeclVal<const FUniqueBuffer&>())>::value, "Invalid inequality");
static_assert(std::is_same<bool, decltype(DeclVal<const FSharedBuffer&>() != DeclVal<const FSharedBuffer&>())>::value, "Invalid inequality");
static_assert(std::is_same<bool, decltype(DeclVal<const FSharedBuffer&>() != DeclVal<const FWeakSharedBuffer&>())>::value, "Invalid inequality");
static_assert(std::is_same<bool, decltype(DeclVal<const FWeakSharedBuffer&>() != DeclVal<const FUniqueBuffer&>())>::value, "Invalid inequality");
static_assert(std::is_same<bool, decltype(DeclVal<const FWeakSharedBuffer&>() != DeclVal<const FSharedBuffer&>())>::value, "Invalid inequality");
static_assert(std::is_same<bool, decltype(DeclVal<const FWeakSharedBuffer&>() != DeclVal<const FWeakSharedBuffer&>())>::value, "Invalid inequality");

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert(std::is_same<uint32, decltype(GetTypeHash(DeclVal<const FUniqueBuffer>()))>::value, "Missing or invalid hash function");
static_assert(std::is_same<uint32, decltype(GetTypeHash(DeclVal<const FSharedBuffer>()))>::value, "Missing or invalid hash function");
static_assert(std::is_same<uint32, decltype(GetTypeHash(DeclVal<const FWeakSharedBuffer>()))>::value, "Missing or invalid hash function");

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FUniqueBufferTest, "System::Core::Memory::UniqueBuffer", "[ApplicationContextMask][SmokeFilter]")
{
	// Test Null
	{
		FUniqueBuffer Buffer;
		CHECK_MESSAGE(TEXT("FUniqueBuffer().IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FUniqueBuffer().GetSize()"), Buffer.GetSize(), uint64(0));
		CHECK_EQUALS(TEXT("FUniqueBuffer().GetView().GetSize()"), Buffer.GetView().GetSize(), uint64(0));
		CHECK_EQUALS(TEXT("FUniqueBuffer().GetView().GetData()"), Buffer.GetView().GetData(), Buffer.GetData());
	}

	// Test Alloc
	{
		constexpr uint64 Size = 64;
		FUniqueBuffer Buffer = FUniqueBuffer::Alloc(Size);
		CHECK_MESSAGE(TEXT("FUniqueBuffer::Alloc().IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FUniqueBuffer::Alloc().GetSize()"), Buffer.GetSize(), Size);
		CHECK_EQUALS(TEXT("FUniqueBuffer::Alloc().GetView().GetSize()"), Buffer.GetView().GetSize(), Size);
		CHECK_EQUALS(TEXT("FUniqueBuffer::Alloc().GetView().GetData()"), Buffer.GetView().GetData(), Buffer.GetData());
	}

	// Test Clone
	{
		constexpr uint64 Size = 64;
		const uint8 Data[Size]{};
		FUniqueBuffer Buffer = FUniqueBuffer::Clone(Data, Size);
		CHECK_MESSAGE(TEXT("FUniqueBuffer::Clone().IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FUniqueBuffer::Clone().GetSize()"), Buffer.GetSize(), Size);
		CHECK_NOT_EQUALS(TEXT("FUniqueBuffer::Clone().GetData()"), static_cast<const void*>(Buffer.GetData()), static_cast<const void*>(Data));
	}

	// Test MakeView
	{
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FUniqueBuffer Buffer = FUniqueBuffer::MakeView(Data, Size);
		CHECK_FALSE_MESSAGE(TEXT("FUniqueBuffer::MakeView().IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FUniqueBuffer::MakeView().GetSize()"), Buffer.GetSize(), Size);
		CHECK_EQUALS(TEXT("FUniqueBuffer::MakeView().GetData()"), Buffer.GetData(), static_cast<void*>(Data));
	}

	// Test TakeOwnership
	{
		bool bDeleted = false;
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FUniqueBuffer Buffer = FUniqueBuffer::TakeOwnership(Data, Size, [&bDeleted](void*, uint64) { bDeleted = true; });
		CHECK_MESSAGE(TEXT("FUniqueBuffer::TakeOwnership().IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FUniqueBuffer::TakeOwnership().GetSize()"), Buffer.GetSize(), Size);
		CHECK_EQUALS(TEXT("FUniqueBuffer::TakeOwnership().GetData()"), Buffer.GetData(), static_cast<void*>(Data));
		Buffer.Reset();
		CHECK_MESSAGE(TEXT("FUniqueBuffer::TakeOwnership() Deleted"), bDeleted);
	}

	// Test MakeOwned
	{
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FUniqueBuffer Buffer = FUniqueBuffer::MakeView(Data, Size).MakeOwned();
		CHECK_MESSAGE(TEXT("FUniqueBuffer::MakeOwned(View).IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FUniqueBuffer::MakeOwned(View).GetSize()"), Buffer.GetSize(), Size);
		CHECK_FALSE_MESSAGE(TEXT("FUniqueBuffer::MakeOwned(View).GetData()"), Buffer.GetData() == Data);
		void* const OwnedData = Buffer.GetData();
		Buffer = MoveTemp(Buffer).MakeOwned();
		CHECK_MESSAGE(TEXT("FUniqueBuffer::MakeOwned(Owned).IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FUniqueBuffer::MakeOwned(Owned).GetSize()"), Buffer.GetSize(), Size);
		CHECK_EQUALS(TEXT("FUniqueBuffer::MakeOwned(Owned).GetData()"), Buffer.GetData(), OwnedData);
	}
}

TEST_CASE_NAMED(FSharedBufferTest, "System::Core::Memory::SharedBuffer", "[ApplicationContextMask][SmokeFilter]")
{
	// Test Null
	{
		FSharedBuffer Buffer;
		CHECK_MESSAGE(TEXT("FSharedBuffer().IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer().GetSize()"), Buffer.GetSize(), uint64(0));
		CHECK_EQUALS(TEXT("FSharedBuffer().GetView().GetSize()"), Buffer.GetView().GetSize(), uint64(0));
		CHECK_EQUALS(TEXT("FSharedBuffer().GetView().GetData()"), Buffer.GetView().GetData(), Buffer.GetData());
	}

	// Test Clone
	{
		constexpr uint64 Size = 64;
		const uint8 Data[Size]{};
		FSharedBuffer Buffer = FSharedBuffer::Clone(Data, Size);
		CHECK_MESSAGE(TEXT("FSharedBuffer::Clone().IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::Clone().GetSize()"), Buffer.GetSize(), Size);
		CHECK_NOT_EQUALS(TEXT("FSharedBuffer::Clone().GetData()"), Buffer.GetData(), static_cast<const void*>(Data));
	}

	// Test MakeView
	{
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FSharedBuffer Buffer = FSharedBuffer::MakeView(Data, Size);
		CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer::MakeView().IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::MakeView().GetSize()"), Buffer.GetSize(), Size);
		CHECK_EQUALS(TEXT("FSharedBuffer::MakeView().GetData()"), Buffer.GetData(), static_cast<const void*>(Data));
	}

	// Test MakeView Outer View
	{
		constexpr uint64 Size = 64;
		const uint8 Data[Size]{};
		FWeakSharedBuffer WeakBuffer;
		{
			FSharedBuffer Buffer;
			{
				FSharedBuffer OuterBuffer = FSharedBuffer::MakeView(Data, Size);
				Buffer = FSharedBuffer::MakeView(OuterBuffer.GetData(), OuterBuffer.GetSize() / 2, OuterBuffer);
				WeakBuffer = OuterBuffer;
			}
			CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer::MakeView(OuterView).IsOwned()"), Buffer.IsOwned());
			CHECK_EQUALS(TEXT("FSharedBuffer::MakeView(OuterView).GetSize()"), Buffer.GetSize(), Size / 2);
			CHECK_EQUALS(TEXT("FSharedBuffer::MakeView(OuterView).GetData()"), Buffer.GetData(), static_cast<const void*>(Data));
			CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer::MakeView(OuterView) Outer Not Null"), WeakBuffer.Pin().IsNull());
		}
		CHECK_MESSAGE(TEXT("FSharedBuffer::MakeView(OuterView) Outer Null"), WeakBuffer.Pin().IsNull());
	}

	// Test MakeView Outer Owned
	{
		constexpr uint64 Size = 64;
		FWeakSharedBuffer WeakBuffer;
		{
			FSharedBuffer Buffer;
			{
				FSharedBuffer OuterBuffer = FUniqueBuffer::Alloc(Size).MoveToShared();
				Buffer = FSharedBuffer::MakeView(OuterBuffer.GetData(), OuterBuffer.GetSize(), OuterBuffer);
				CHECK_EQUALS(TEXT("FSharedBuffer::MakeView(OuterOwned, SameView)"), Buffer, OuterBuffer);
				Buffer = FSharedBuffer::MakeView(OuterBuffer.GetData(), OuterBuffer.GetSize() / 2, OuterBuffer);
				WeakBuffer = OuterBuffer;
			}
			CHECK_MESSAGE(TEXT("FSharedBuffer::MakeView(OuterOwned).IsOwned()"), Buffer.IsOwned());
			CHECK_EQUALS(TEXT("FSharedBuffer::MakeView(OuterOwned).GetSize()"), Buffer.GetSize(), Size / 2);
			CHECK_EQUALS(TEXT("FSharedBuffer::MakeView(OuterOwned).GetData()"), Buffer.GetData(), WeakBuffer.Pin().GetData());
			CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer::MakeView(OuterOwned) Outer Not Null"), WeakBuffer.Pin().IsNull());
		}
		CHECK_MESSAGE(TEXT("FSharedBuffer::MakeView(OuterOwned) Outer  Null"), WeakBuffer.Pin().IsNull());
	}

	// Test TakeOwnership
	{
		bool bDeleted = false;
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FSharedBuffer Buffer = FSharedBuffer::TakeOwnership(Data, Size, [&bDeleted](void*, uint64) { bDeleted = true; });
		CHECK_MESSAGE(TEXT("FSharedBuffer::TakeOwnership().IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::TakeOwnership().GetSize()"), Buffer.GetSize(), Size);
		CHECK_EQUALS(TEXT("FSharedBuffer::TakeOwnership().GetData()"), Buffer.GetData(), static_cast<const void*>(Data));
		Buffer.Reset();
		CHECK_MESSAGE(TEXT("FSharedBuffer::TakeOwnership() Deleted"), bDeleted);
	}

	// Test MakeOwned
	{
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FSharedBuffer Buffer = FSharedBuffer::MakeView(Data, Size).MakeOwned();
		CHECK_MESSAGE(TEXT("FSharedBuffer::MakeOwned(View).IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::MakeOwned(View).GetSize()"), Buffer.GetSize(), Size);
		CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer::MakeOwned(View).GetData()"), Buffer.GetData() == Data);
		FSharedBuffer BufferCopy = Buffer.MakeOwned();
		CHECK_MESSAGE(TEXT("FSharedBuffer::MakeOwned(Owned).IsOwned()"), BufferCopy.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::MakeOwned(Owned).GetSize()"), BufferCopy.GetSize(), Size);
		CHECK_EQUALS(TEXT("FSharedBuffer::MakeOwned(Owned).GetData()"), BufferCopy.GetData(), Buffer.GetData());
	}

	// Test MoveToUnique
	{
		FUniqueBuffer Buffer = FSharedBuffer().MoveToUnique();
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(Null).IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::MoveToUnique(Null).GetSize()"), Buffer.GetSize(), uint64(0));
	}
	{
		FSharedBuffer Shared = FSharedBuffer::Clone(MakeMemoryView<uint8>({ 1, 2, 3, 4 }));
		FUniqueBuffer Buffer = FSharedBuffer(Shared).MoveToUnique();
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(Shared).IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::MoveToUnique(Shared).GetSize()"), Buffer.GetSize(), Shared.GetSize());
		CHECK_NOT_EQUALS(TEXT("FSharedBuffer::MoveToUnique(Shared).GetData()"), const_cast<const void*>(Buffer.GetData()), Shared.GetData());
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(Shared)->Equal"), Buffer.GetView().EqualBytes(Shared.GetView()));
	}
	{
		bool bDeleted = false;
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FSharedBuffer Shared = FUniqueBuffer::TakeOwnership(Data, Size, [&bDeleted](void*) { bDeleted = true; }).MoveToShared();
		FUniqueBuffer Buffer = Shared.MoveToUnique();
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(Owned).IsNull()"), Shared.IsNull());
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(Owned).IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::MoveToUnique(Owned).GetSize()"), Buffer.GetSize(), Size);
		CHECK_EQUALS(TEXT("FSharedBuffer::MoveToUnique(Owned).GetData()"), Buffer.GetData(), static_cast<void*>(Data));
		Buffer.Reset();
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(Owned) Deleted"), bDeleted);
	}
	{
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FSharedBuffer Shared = FSharedBuffer::TakeOwnership(Data, Size, [](void*) {});
		FUniqueBuffer Buffer = Shared.MoveToUnique();
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(ImmutableOwned).IsNull()"), Shared.IsNull());
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(ImmutableOwned).IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::MoveToUnique(ImmutableOwned).GetSize()"), Buffer.GetSize(), Size);
		CHECK_NOT_EQUALS(TEXT("FSharedBuffer::MoveToUnique(ImmutableOwned).GetData()"), Buffer.GetData(), static_cast<void*>(Data));
	}
	{
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FSharedBuffer Shared = FUniqueBuffer::MakeView(MakeMemoryView(Data)).MoveToShared();
		FUniqueBuffer Buffer = Shared.MoveToUnique();
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(View).IsNull()"), Shared.IsNull());
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(View).IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::MoveToUnique(View).GetSize()"), Buffer.GetSize(), Size);
		CHECK_NOT_EQUALS(TEXT("FSharedBuffer::MoveToUnique(View).GetData()"), Buffer.GetData(), static_cast<void*>(Data));
	}
	{
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FSharedBuffer Shared = FSharedBuffer::MakeView(MakeMemoryView(Data));
		FUniqueBuffer Buffer = Shared.MoveToUnique();
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(ImmutableView).IsNull()"), Shared.IsNull());
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(ImmutableView).IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::MoveToUnique(ImmutableView).GetSize()"), Buffer.GetSize(), Size);
		CHECK_NOT_EQUALS(TEXT("FSharedBuffer::MoveToUnique(ImmutableView).GetData()"), Buffer.GetData(), static_cast<void*>(Data));
	}
	{
		constexpr uint64 Size = 64;
		uint8 Data[Size]{};
		FSharedBuffer Shared = FUniqueBuffer::TakeOwnership(Data, Size, [](void*) {}).MoveToShared();
		FUniqueBuffer Buffer = FSharedBuffer::MakeView(Shared.GetView().Left(Size / 2), Shared).MoveToUnique();
		CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(OwnedView).IsNull()"), Shared.IsNull());
		CHECK_MESSAGE(TEXT("FSharedBuffer::MoveToUnique(OwnedView).IsOwned()"), Buffer.IsOwned());
		CHECK_EQUALS(TEXT("FSharedBuffer::MoveToUnique(OwnedView).GetSize()"), Buffer.GetSize(), Size / 2);
		CHECK_NOT_EQUALS(TEXT("FSharedBuffer::MoveToUnique(OwnedView).GetData()"), Buffer.GetData(), static_cast<void*>(Data));
	}

}

TEST_CASE_NAMED(FBufferOwnerTest, "System::Core::Memory::BufferOwner", "[ApplicationContextMask][SmokeFilter]")
{
	class FTestBufferOwner final : public FBufferOwner
	{
	public:
		FTestBufferOwner(bool& bInMaterializedRef, bool& bInFreedRef, bool& bInDeletedRef)
			: bMaterializedRef(bInMaterializedRef)
			, bFreedRef(bInFreedRef)
			, bDeletedRef(bInDeletedRef)
		{
			SetIsOwned();
		}

	private:
		virtual void MaterializeBuffer()
		{
			SetIsMaterialized();
			bMaterializedRef = true;
		}

		virtual void FreeBuffer() final
		{
			bFreedRef = true;
		}

		virtual ~FTestBufferOwner()
		{
			bDeletedRef = true;
		}

		bool& bMaterializedRef;
		bool& bFreedRef;
		bool& bDeletedRef;
	};

	bool bMaterialized = false;
	bool bFreed = false;
	bool bDeleted = false;

	FSharedBuffer Buffer(new FTestBufferOwner(bMaterialized, bFreed, bDeleted));
	CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->!Materialized"), bMaterialized);
	CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->!Freed"), bFreed);
	CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->!Deleted"), bDeleted);

	(void)Buffer.GetData();
	CHECK_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->GetData()->Materialized"), bMaterialized);
	bMaterialized = false;
	(void)Buffer.GetData();
	CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->GetData()->!Materialized"), bMaterialized);
	bMaterialized = false;

	Buffer = FSharedBuffer(new FTestBufferOwner(bMaterialized, bFreed, bDeleted));
	CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Assign(FSharedBuffer)->!Materialized"), bMaterialized);
	CHECK_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Assign(FSharedBuffer)->Freed"), bFreed);
	CHECK_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Assign(FSharedBuffer)->Deleted"), bDeleted);
	bFreed = false;
	bDeleted = false;

	(void)Buffer.GetSize();
	CHECK_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->GetSize()->Materialized"), bMaterialized);
	bMaterialized = false;
	(void)Buffer.GetSize();
	CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->GetSize()->!Materialized"), bMaterialized);
	bMaterialized = false;

	FWeakSharedBuffer WeakBuffer = Buffer;
	Buffer.Reset();
	CHECK_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Reset(Shared)->Freed"), bFreed);
	CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Reset(Shared)->!Deleted"), bDeleted);
	WeakBuffer.Reset();
	CHECK_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Reset(Weak)->Deleted"), bDeleted);

	bMaterialized = false;
	bFreed = false;
	bDeleted = false;
	Buffer = FSharedBuffer(new FTestBufferOwner(bMaterialized, bFreed, bDeleted));

	CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->IsMaterialized()"), Buffer.IsMaterialized());
	Buffer.Materialize();
	CHECK_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Materialize()->IsMaterialized()"), Buffer.IsMaterialized());
	CHECK_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Materialize()->Materialized"), bMaterialized);
	bMaterialized = false;
	Buffer.Materialize();
	CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Materialize()->!Materialized"), bMaterialized);
	bMaterialized = false;
	Buffer.Reset();

	bFreed = false;
	bDeleted = false;
	Buffer = FSharedBuffer(new FTestBufferOwner(bMaterialized, bFreed, bDeleted));
	Buffer.Reset();
	CHECK_FALSE_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Reset(!Materialized)->!Materialized"), bMaterialized);
	CHECK_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Reset(!Materialized)->Freed"), bFreed);
	CHECK_MESSAGE(TEXT("FSharedBuffer(BufferOwner)->Reset(!Materialized)->Deleted"), bDeleted);
}

TEST_CASE_NAMED(FUniqueBufferFromArrayTest, "System::Core::Memory::UniqueBufferFromArray", "[ApplicationContextMask][SmokeFilter]")
{
	TArray<uint16> Array{ 1, 2, 3, 4, 5, 6, 7, 8 };
	const void* const ExpectedData = Array.GetData();
	const uint64 ExpectedSize = Array.Num() * sizeof(uint16);
	const FUniqueBuffer Buffer = MakeUniqueBufferFromArray(MoveTemp(Array));
	CHECK_MESSAGE(TEXT("MakeUniqueBufferFromArray -> Array.IsEmpty"), Array.IsEmpty());
	CHECK_EQUALS(TEXT("MakeUniqueBufferFromArray -> Buffer.GetData()"), Buffer.GetData(), ExpectedData);
	CHECK_EQUALS(TEXT("MakeUniqueBufferFromArray -> Buffer.GetSize()"), Buffer.GetSize(), ExpectedSize);
}

TEST_CASE_NAMED(FSharedBufferFromArrayTest, "System::Core::Memory::SharedBufferFromArray", "[ApplicationContextMask][SmokeFilter]")
{
	TArray<uint16> Array{1, 2, 3, 4, 5, 6, 7, 8};
	const void* const ExpectedData = Array.GetData();
	const uint64 ExpectedSize = Array.Num() * sizeof(uint16);
	const FSharedBuffer Buffer = MakeSharedBufferFromArray(MoveTemp(Array));
	CHECK_MESSAGE(TEXT("MakeSharedBufferFromArray -> Array.IsEmpty"), Array.IsEmpty());
	CHECK_EQUALS(TEXT("MakeSharedBufferFromArray -> Buffer.GetData()"), Buffer.GetData(), ExpectedData);
	CHECK_EQUALS(TEXT("MakeSharedBufferFromArray -> Buffer.GetSize()"), Buffer.GetSize(), ExpectedSize);
}

#endif // WITH_TESTS
