// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Misc/AES.h"

#include "HAL/UnrealMemory.h"
#include "String/HexToBytes.h"
#include "String/BytesToHex.h"
#include "Containers/StringView.h"
#include "Containers/Array.h"
#include "Misc/StringBuilder.h"

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FMiscAESTest, "System::Core::Misc::AES", "[Core][AES][SmokeFilter]")
{
	auto RunTest = [](FAnsiStringView Key, FAnsiStringView PlainText, FAnsiStringView CipherText, int IterCount = 1)
	{
		checkf((Key.Len()/2) == FAES::FAESKey::KeySize, TEXT("key size must be 32 bytes = 256 bits"));
		checkf((PlainText.Len()/2) % FAES::AESBlockSize == 0, TEXT("plain-text must be multiple of block size"));
		checkf(CipherText.Len() == PlainText.Len(), TEXT("expected cipher-text should be same length as plain-text"));

		TArray<uint8> KeyBytes;
		KeyBytes.AddUninitialized(FAES::FAESKey::KeySize);
		UE::String::HexToBytes(Key, KeyBytes.GetData());

		TArray<uint8> PlainTextBytes;
		PlainTextBytes.AddUninitialized(PlainText.Len() / 2);
		UE::String::HexToBytes(PlainText, PlainTextBytes.GetData());

		TArray<uint8> CipherTextBytes;
		CipherTextBytes.AddUninitialized(CipherText.Len() / 2);
		UE::String::HexToBytes(CipherText, CipherTextBytes.GetData());

		// encryption
		{
			TArray<uint8> Encrypted = PlainTextBytes;
			for (int Iter = 0; Iter < IterCount; Iter++)
			{
				FAES::EncryptData(Encrypted.GetData(), Encrypted.Num(), KeyBytes.GetData(), KeyBytes.Num());
			}

			TAnsiStringBuilder<32> EncryptedHex;
			UE::String::BytesToHex(Encrypted, EncryptedHex);

			REQUIRE(EncryptedHex.ToView() == CipherText);
		}

		// decryption
		{
			TArray<uint8> Decrypted = CipherTextBytes;
			for (int Iter = 0; Iter < IterCount; Iter++)
			{
				FAES::DecryptData(Decrypted.GetData(), Decrypted.Num(), KeyBytes.GetData(), KeyBytes.Num());
			}

			TAnsiStringBuilder<32> DecryptedHex;
			UE::String::BytesToHex(Decrypted, DecryptedHex);

			REQUIRE(DecryptedHex.ToView() == PlainText);
		}
	};

	struct AesTestCase
	{
		const ANSICHAR* Key = nullptr;
		const ANSICHAR* PlainText = nullptr;
		const ANSICHAR* CipherText = nullptr;
	};

	// Appendix C, https://csrc.nist.gov/csrc/media/publications/fips/197/final/documents/fips-197.pdf
	SECTION("Example Vector")
	{
		RunTest("000102030405060708090a0b0c0d0e0f101112131415161718191a1b1c1d1e1f", "00112233445566778899aabbccddeeff", "8ea2b7ca516745bfeafc49904b496089");
	}

	// https://csrc.nist.gov/Projects/cryptographic-algorithm-validation-program/Block-Ciphers#AES

	// Appendix B
	SECTION("GFSbox Known Answer")
	{
		static const AesTestCase GFSboxCases[] =
		{
			{ "0000000000000000000000000000000000000000000000000000000000000000", "014730f80ac625fe84f026c60bfd547d", "5c9d844ed46f9885085e5d6a4f94c7d7" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "0b24af36193ce4665f2825d7b4749c98", "a9ff75bd7cf6613d3731c77c3b6d0c04" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "761c1fe41a18acf20d241650611d90f1", "623a52fcea5d443e48d9181ab32c7421" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "8a560769d605868ad80d819bdba03771", "38f2c7ae10612415d27ca190d27da8b4" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "91fbef2d15a97816060bee1feaa49afe", "1bc704f1bce135ceb810341b216d7abe" },
		};
		for (auto& Test : GFSboxCases)
		{
			RunTest(Test.Key, Test.PlainText, Test.CipherText);
		}
	}

	// Appendix C
	SECTION("KeySbox Known Answer")
	{
		static const AesTestCase KeySboxCases[] =
		{
			{ "c47b0294dbbbee0fec4757f22ffeee3587ca4730c3d33b691df38bab076bc558", "00000000000000000000000000000000", "46f2fb342d6f0ab477476fc501242c5f" },
			{ "28d46cffa158533194214a91e712fc2b45b518076675affd910edeca5f41ac64", "00000000000000000000000000000000", "4bf3b0a69aeb6657794f2901b1440ad4" },
			{ "c1cc358b449909a19436cfbb3f852ef8bcb5ed12ac7058325f56e6099aab1a1c", "00000000000000000000000000000000", "352065272169abf9856843927d0674fd" },
			{ "984ca75f4ee8d706f46c2d98c0bf4a45f5b00d791c2dfeb191b5ed8e420fd627", "00000000000000000000000000000000", "4307456a9e67813b452e15fa8fffe398" },
			{ "b43d08a447ac8609baadae4ff12918b9f68fc1653f1269222f123981ded7a92f", "00000000000000000000000000000000", "4663446607354989477a5c6f0f007ef4" },
			{ "1d85a181b54cde51f0e098095b2962fdc93b51fe9b88602b3f54130bf76a5bd9", "00000000000000000000000000000000", "531c2c38344578b84d50b3c917bbb6e1" },
			{ "dc0eba1f2232a7879ded34ed8428eeb8769b056bbaf8ad77cb65c3541430b4cf", "00000000000000000000000000000000", "fc6aec906323480005c58e7e1ab004ad" },
			{ "f8be9ba615c5a952cabbca24f68f8593039624d524c816acda2c9183bd917cb9", "00000000000000000000000000000000", "a3944b95ca0b52043584ef02151926a8" },
			{ "797f8b3d176dac5b7e34a2d539c4ef367a16f8635f6264737591c5c07bf57a3e", "00000000000000000000000000000000", "a74289fe73a4c123ca189ea1e1b49ad5" },
			{ "6838d40caf927749c13f0329d331f448e202c73ef52c5f73a37ca635d4c47707", "00000000000000000000000000000000", "b91d4ea4488644b56cf0812fa7fcf5fc" },
			{ "ccd1bc3c659cd3c59bc437484e3c5c724441da8d6e90ce556cd57d0752663bbc", "00000000000000000000000000000000", "304f81ab61a80c2e743b94d5002a126b" },
			{ "13428b5e4c005e0636dd338405d173ab135dec2a25c22c5df0722d69dcc43887", "00000000000000000000000000000000", "649a71545378c783e368c9ade7114f6c" },
			{ "07eb03a08d291d1b07408bf3512ab40c91097ac77461aad4bb859647f74f00ee", "00000000000000000000000000000000", "47cb030da2ab051dfc6c4bf6910d12bb" },
			{ "90143ae20cd78c5d8ebdd6cb9dc1762427a96c78c639bccc41a61424564eafe1", "00000000000000000000000000000000", "798c7c005dee432b2c8ea5dfa381ecc3" },
			{ "b7a5794d52737475d53d5a377200849be0260a67a2b22ced8bbef12882270d07", "00000000000000000000000000000000", "637c31dc2591a07636f646b72daabbe7" },
			{ "fca02f3d5011cfc5c1e23165d413a049d4526a991827424d896fe3435e0bf68e", "00000000000000000000000000000000", "179a49c712154bbffbe6e7a84a18e220" },
		};
		for (auto& Test : KeySboxCases)
		{
			RunTest(Test.Key, Test.PlainText, Test.CipherText);
		}
	}

	// Appendix D
	SECTION("VarTxt Known Answer")
	{
		static const AesTestCase VarTxtCases[] =
		{
			{ "0000000000000000000000000000000000000000000000000000000000000000", "80000000000000000000000000000000", "ddc6bf790c15760d8d9aeb6f9a75fd4e" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "c0000000000000000000000000000000", "0a6bdc6d4c1e6280301fd8e97ddbe601" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "e0000000000000000000000000000000", "9b80eefb7ebe2d2b16247aa0efc72f5d" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "f0000000000000000000000000000000", "7f2c5ece07a98d8bee13c51177395ff7" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "f8000000000000000000000000000000", "7818d800dcf6f4be1e0e94f403d1e4c2" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fc000000000000000000000000000000", "e74cd1c92f0919c35a0324123d6177d3" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fe000000000000000000000000000000", "8092a4dcf2da7e77e93bdd371dfed82e" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ff000000000000000000000000000000", "49af6b372135acef10132e548f217b17" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ff800000000000000000000000000000", "8bcd40f94ebb63b9f7909676e667f1e7" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffc00000000000000000000000000000", "fe1cffb83f45dcfb38b29be438dbd3ab" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffe00000000000000000000000000000", "0dc58a8d886623705aec15cb1e70dc0e" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fff00000000000000000000000000000", "c218faa16056bd0774c3e8d79c35a5e4" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fff80000000000000000000000000000", "047bba83f7aa841731504e012208fc9e" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffc0000000000000000000000000000", "dc8f0e4915fd81ba70a331310882f6da" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffe0000000000000000000000000000", "1569859ea6b7206c30bf4fd0cbfac33c" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffff0000000000000000000000000000", "300ade92f88f48fa2df730ec16ef44cd" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffff8000000000000000000000000000", "1fe6cc3c05965dc08eb0590c95ac71d0" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffc000000000000000000000000000", "59e858eaaa97fec38111275b6cf5abc0" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffe000000000000000000000000000", "2239455e7afe3b0616100288cc5a723b" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffff000000000000000000000000000", "3ee500c5c8d63479717163e55c5c4522" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffff800000000000000000000000000", "d5e38bf15f16d90e3e214041d774daa8" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffc00000000000000000000000000", "b1f4066e6f4f187dfe5f2ad1b17819d0" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffe00000000000000000000000000", "6ef4cc4de49b11065d7af2909854794a" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffff00000000000000000000000000", "ac86bc606b6640c309e782f232bf367f" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffff80000000000000000000000000", "36aff0ef7bf3280772cf4cac80a0d2b2" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffc0000000000000000000000000", "1f8eedea0f62a1406d58cfc3ecea72cf" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffe0000000000000000000000000", "abf4154a3375a1d3e6b1d454438f95a6" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffff0000000000000000000000000", "96f96e9d607f6615fc192061ee648b07" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffff8000000000000000000000000", "cf37cdaaa0d2d536c71857634c792064" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffc000000000000000000000000", "fbd6640c80245c2b805373f130703127" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffe000000000000000000000000", "8d6a8afe55a6e481badae0d146f436db" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffff000000000000000000000000", "6a4981f2915e3e68af6c22385dd06756" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffff800000000000000000000000", "42a1136e5f8d8d21d3101998642d573b" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffc00000000000000000000000", "9b471596dc69ae1586cee6158b0b0181" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffe00000000000000000000000", "753665c4af1eff33aa8b628bf8741cfd" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffff00000000000000000000000", "9a682acf40be01f5b2a4193c9a82404d" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffff80000000000000000000000", "54fafe26e4287f17d1935f87eb9ade01" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffc0000000000000000000000", "49d541b2e74cfe73e6a8e8225f7bd449" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffe0000000000000000000000", "11a45530f624ff6f76a1b3826626ff7b" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffff0000000000000000000000", "f96b0c4a8bc6c86130289f60b43b8fba" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffff8000000000000000000000", "48c7d0e80834ebdc35b6735f76b46c8b" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffc000000000000000000000", "2463531ab54d66955e73edc4cb8eaa45" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffe000000000000000000000", "ac9bd8e2530469134b9d5b065d4f565b" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffff000000000000000000000", "3f5f9106d0e52f973d4890e6f37e8a00" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffff800000000000000000000", "20ebc86f1304d272e2e207e59db639f0" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffc00000000000000000000", "e67ae6426bf9526c972cff072b52252c" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffe00000000000000000000", "1a518dddaf9efa0d002cc58d107edfc8" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffff00000000000000000000", "ead731af4d3a2fe3b34bed047942a49f" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffff80000000000000000000", "b1d4efe40242f83e93b6c8d7efb5eae9" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffc0000000000000000000", "cd2b1fec11fd906c5c7630099443610a" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffe0000000000000000000", "a1853fe47fe29289d153161d06387d21" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffff0000000000000000000", "4632154179a555c17ea604d0889fab14" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffff8000000000000000000", "dd27cac6401a022e8f38f9f93e774417" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffc000000000000000000", "c090313eb98674f35f3123385fb95d4d" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffe000000000000000000", "cc3526262b92f02edce548f716b9f45c" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffff000000000000000000", "c0838d1a2b16a7c7f0dfcc433c399c33" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffff800000000000000000", "0d9ac756eb297695eed4d382eb126d26" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffc00000000000000000", "56ede9dda3f6f141bff1757fa689c3e1" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffe00000000000000000", "768f520efe0f23e61d3ec8ad9ce91774" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffff00000000000000000", "b1144ddfa75755213390e7c596660490" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffff80000000000000000", "1d7c0c4040b355b9d107a99325e3b050" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffc0000000000000000", "d8e2bb1ae8ee3dcf5bf7d6c38da82a1a" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffe0000000000000000", "faf82d178af25a9886a47e7f789b98d7" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffff0000000000000000", "9b58dbfd77fe5aca9cfc190cd1b82d19" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffff8000000000000000", "77f392089042e478ac16c0c86a0b5db5" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffc000000000000000", "19f08e3420ee69b477ca1420281c4782" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffe000000000000000", "a1b19beee4e117139f74b3c53fdcb875" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffff000000000000000", "a37a5869b218a9f3a0868d19aea0ad6a" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffff800000000000000", "bc3594e865bcd0261b13202731f33580" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffc00000000000000", "811441ce1d309eee7185e8c752c07557" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffe00000000000000", "959971ce4134190563518e700b9874d1" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffff00000000000000", "76b5614a042707c98e2132e2e805fe63" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffff80000000000000", "7d9fa6a57530d0f036fec31c230b0cc6" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffc0000000000000", "964153a83bf6989a4ba80daa91c3e081" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffe0000000000000", "a013014d4ce8054cf2591d06f6f2f176" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffff0000000000000", "d1c5f6399bf382502e385eee1474a869" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffff8000000000000", "0007e20b8298ec354f0f5fe7470f36bd" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffc000000000000", "b95ba05b332da61ef63a2b31fcad9879" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffe000000000000", "4620a49bd967491561669ab25dce45f4" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffff000000000000", "12e71214ae8e04f0bb63d7425c6f14d5" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffff800000000000", "4cc42fc1407b008fe350907c092e80ac" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffc00000000000", "08b244ce7cbc8ee97fbba808cb146fda" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffe00000000000", "39b333e8694f21546ad1edd9d87ed95b" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffff00000000000", "3b271f8ab2e6e4a20ba8090f43ba78f3" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffff80000000000", "9ad983f3bf651cd0393f0a73cccdea50" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffc0000000000", "8f476cbff75c1f725ce18e4bbcd19b32" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffe0000000000", "905b6267f1d6ab5320835a133f096f2a" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffff0000000000", "145b60d6d0193c23f4221848a892d61a" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffff8000000000", "55cfb3fb6d75cad0445bbc8dafa25b0f" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffc000000000", "7b8e7098e357ef71237d46d8b075b0f5" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffe000000000", "2bf27229901eb40f2df9d8398d1505ae" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffff000000000", "83a63402a77f9ad5c1e931a931ecd706" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffff800000000", "6f8ba6521152d31f2bada1843e26b973" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffc00000000", "e5c3b8e30fd2d8e6239b17b44bd23bbd" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffe00000000", "1ac1f7102c59933e8b2ddc3f14e94baa" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffff00000000", "21d9ba49f276b45f11af8fc71a088e3d" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffff80000000", "649f1cddc3792b4638635a392bc9bade" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffc0000000", "e2775e4b59c1bc2e31a2078c11b5a08c" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffe0000000", "2be1fae5048a25582a679ca10905eb80" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffff0000000", "da86f292c6f41ea34fb2068df75ecc29" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffff8000000", "220df19f85d69b1b562fa69a3c5beca5" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffc000000", "1f11d5d0355e0b556ccdb6c7f5083b4d" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffe000000", "62526b78be79cb384633c91f83b4151b" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffff000000", "90ddbcb950843592dd47bbef00fdc876" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffff800000", "2fd0e41c5b8402277354a7391d2618e2" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffc00000", "3cdf13e72dee4c581bafec70b85f9660" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffe00000", "afa2ffc137577092e2b654fa199d2c43" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffff00000", "8d683ee63e60d208e343ce48dbc44cac" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffff80000", "705a4ef8ba2133729c20185c3d3a4763" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffffc0000", "0861a861c3db4e94194211b77ed761b9" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffffe0000", "4b00c27e8b26da7eab9d3a88dec8b031" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffff0000", "5f397bf03084820cc8810d52e5b666e9" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffff8000", "63fafabb72c07bfbd3ddc9b1203104b8" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffffc000", "683e2140585b18452dd4ffbb93c95df9" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffffe000", "286894e48e537f8763b56707d7d155c8" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffffff000", "a423deabc173dcf7e2c4c53e77d37cd1" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffffff800", "eb8168313e1cfdfdb5e986d5429cf172" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffffffc00", "27127daafc9accd2fb334ec3eba52323" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffffffe00", "ee0715b96f72e3f7a22a5064fc592f4c" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffffff00", "29ee526770f2a11dcfa989d1ce88830f" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffffff80", "0493370e054b09871130fe49af730a5a" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffffffc0", "9b7b940f6c509f9e44a4ee140448ee46" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffffffe0", "2915be4a1ecfdcbe3e023811a12bb6c7" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffffffff0", "7240e524bc51d8c4d440b1be55d1062c" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffffffff8", "da63039d38cb4612b2dc36ba26684b93" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffffffffc", "0f59cb5a4b522e2ac56c1a64f558ad9a" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "fffffffffffffffffffffffffffffffe", "7bfe9d876c6d63c1d035da8fe21c409d" },
			{ "0000000000000000000000000000000000000000000000000000000000000000", "ffffffffffffffffffffffffffffffff", "acdace8078a32b1a182bfa4987ca1347" },
		};
		for (auto& Test : VarTxtCases)
		{
			RunTest(Test.Key, Test.PlainText, Test.CipherText);
		}
	}

	// Appendix E
	SECTION("VarKey Known Answer")
	{
		static const AesTestCase VarKeyCases[] =
		{
			{ "8000000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "e35a6dcb19b201a01ebcfa8aa22b5759" },
			{ "c000000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "b29169cdcf2d83e838125a12ee6aa400" },
			{ "e000000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "d8f3a72fc3cdf74dfaf6c3e6b97b2fa6" },
			{ "f000000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "1c777679d50037c79491a94da76a9a35" },
			{ "f800000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "9cf4893ecafa0a0247a898e040691559" },
			{ "fc00000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "8fbb413703735326310a269bd3aa94b2" },
			{ "fe00000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "60e32246bed2b0e859e55c1cc6b26502" },
			{ "ff00000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "ec52a212f80a09df6317021bc2a9819e" },
			{ "ff80000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "f23e5b600eb70dbccf6c0b1d9a68182c" },
			{ "ffc0000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "a3f599d63a82a968c33fe26590745970" },
			{ "ffe0000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "d1ccb9b1337002cbac42c520b5d67722" },
			{ "fff0000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "cc111f6c37cf40a1159d00fb59fb0488" },
			{ "fff8000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "dc43b51ab609052372989a26e9cdd714" },
			{ "fffc000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "4dcede8da9e2578f39703d4433dc6459" },
			{ "fffe000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "1a4c1c263bbccfafc11782894685e3a8" },
			{ "ffff000000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "937ad84880db50613423d6d527a2823d" },
			{ "ffff800000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "610b71dfc688e150d8152c5b35ebc14d" },
			{ "ffffc00000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "27ef2495dabf323885aab39c80f18d8b" },
			{ "ffffe00000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "633cafea395bc03adae3a1e2068e4b4e" },
			{ "fffff00000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "6e1b482b53761cf631819b749a6f3724" },
			{ "fffff80000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "976e6f851ab52c771998dbb2d71c75a9" },
			{ "fffffc0000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "85f2ba84f8c307cf525e124c3e22e6cc" },
			{ "fffffe0000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "6bcca98bf6a835fa64955f72de4115fe" },
			{ "ffffff0000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "2c75e2d36eebd65411f14fd0eb1d2a06" },
			{ "ffffff8000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "bd49295006250ffca5100b6007a0eade" },
			{ "ffffffc000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "a190527d0ef7c70f459cd3940df316ec" },
			{ "ffffffe000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "bbd1097a62433f79449fa97d4ee80dbf" },
			{ "fffffff000000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "07058e408f5b99b0e0f061a1761b5b3b" },
			{ "fffffff800000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "5fd1f13fa0f31e37fabde328f894eac2" },
			{ "fffffffc00000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "fc4af7c948df26e2ef3e01c1ee5b8f6f" },
			{ "fffffffe00000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "829fd7208fb92d44a074a677ee9861ac" },
			{ "ffffffff00000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "ad9fc613a703251b54c64a0e76431711" },
			{ "ffffffff80000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "33ac9eccc4cc75e2711618f80b1548e8" },
			{ "ffffffffc0000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "2025c74b8ad8f4cda17ee2049c4c902d" },
			{ "ffffffffe0000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "f85ca05fe528f1ce9b790166e8d551e7" },
			{ "fffffffff0000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "6f6238d8966048d4967154e0dad5a6c9" },
			{ "fffffffff8000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "f2b21b4e7640a9b3346de8b82fb41e49" },
			{ "fffffffffc000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "f836f251ad1d11d49dc344628b1884e1" },
			{ "fffffffffe000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "077e9470ae7abea5a9769d49182628c3" },
			{ "ffffffffff000000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "e0dcc2d27fc9865633f85223cf0d611f" },
			{ "ffffffffff800000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "be66cfea2fecd6bf0ec7b4352c99bcaa" },
			{ "ffffffffffc00000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "df31144f87a2ef523facdcf21a427804" },
			{ "ffffffffffe00000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "b5bb0f5629fb6aae5e1839a3c3625d63" },
			{ "fffffffffff00000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "3c9db3335306fe1ec612bdbfae6b6028" },
			{ "fffffffffff80000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "3dd5c34634a79d3cfcc8339760e6f5f4" },
			{ "fffffffffffc0000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "82bda118a3ed7af314fa2ccc5c07b761" },
			{ "fffffffffffe0000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "2937a64f7d4f46fe6fea3b349ec78e38" },
			{ "ffffffffffff0000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "225f068c28476605735ad671bb8f39f3" },
			{ "ffffffffffff8000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "ae682c5ecd71898e08942ac9aa89875c" },
			{ "ffffffffffffc000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "5e031cb9d676c3022d7f26227e85c38f" },
			{ "ffffffffffffe000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "a78463fb064db5d52bb64bfef64f2dda" },
			{ "fffffffffffff000000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "8aa9b75e784593876c53a00eae5af52b" },
			{ "fffffffffffff800000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "3f84566df23da48af692722fe980573a" },
			{ "fffffffffffffc00000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "31690b5ed41c7eb42a1e83270a7ff0e6" },
			{ "fffffffffffffe00000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "77dd7702646d55f08365e477d3590eda" },
			{ "ffffffffffffff00000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "4c022ac62b3cb78d739cc67b3e20bb7e" },
			{ "ffffffffffffff80000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "092fa137ce18b5dfe7906f550bb13370" },
			{ "ffffffffffffffc0000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "3e0cdadf2e68353c0027672c97144dd3" },
			{ "ffffffffffffffe0000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "d8c4b200b383fc1f2b2ea677618a1d27" },
			{ "fffffffffffffff0000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "11825f99b0e9bb3477c1c0713b015aac" },
			{ "fffffffffffffff8000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "f8b9fffb5c187f7ddc7ab10f4fb77576" },
			{ "fffffffffffffffc000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "ffb4e87a32b37d6f2c8328d3b5377802" },
			{ "fffffffffffffffe000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "d276c13a5d220f4da9224e74896391ce" },
			{ "ffffffffffffffff000000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "94efe7a0e2e031e2536da01df799c927" },
			{ "ffffffffffffffff800000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "8f8fd822680a85974e53a5a8eb9d38de" },
			{ "ffffffffffffffffc00000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "e0f0a91b2e45f8cc37b7805a3042588d" },
			{ "ffffffffffffffffe00000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "597a6252255e46d6364dbeeda31e279c" },
			{ "fffffffffffffffff00000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "f51a0f694442b8f05571797fec7ee8bf" },
			{ "fffffffffffffffff80000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "9ff071b165b5198a93dddeebc54d09b5" },
			{ "fffffffffffffffffc0000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "c20a19fd5758b0c4bc1a5df89cf73877" },
			{ "fffffffffffffffffe0000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "97120166307119ca2280e9315668e96f" },
			{ "ffffffffffffffffff0000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "4b3b9f1e099c2a09dc091e90e4f18f0a" },
			{ "ffffffffffffffffff8000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "eb040b891d4b37f6851f7ec219cd3f6d" },
			{ "ffffffffffffffffffc000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "9f0fdec08b7fd79aa39535bea42db92a" },
			{ "ffffffffffffffffffe000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "2e70f168fc74bf911df240bcd2cef236" },
			{ "fffffffffffffffffff000000000000000000000000000000000000000000000", "00000000000000000000000000000000", "462ccd7f5fd1108dbc152f3cacad328b" },
			{ "fffffffffffffffffff800000000000000000000000000000000000000000000", "00000000000000000000000000000000", "a4af534a7d0b643a01868785d86dfb95" },
			{ "fffffffffffffffffffc00000000000000000000000000000000000000000000", "00000000000000000000000000000000", "ab980296197e1a5022326c31da4bf6f3" },
			{ "fffffffffffffffffffe00000000000000000000000000000000000000000000", "00000000000000000000000000000000", "f97d57b3333b6281b07d486db2d4e20c" },
			{ "ffffffffffffffffffff00000000000000000000000000000000000000000000", "00000000000000000000000000000000", "f33fa36720231afe4c759ade6bd62eb6" },
			{ "ffffffffffffffffffff80000000000000000000000000000000000000000000", "00000000000000000000000000000000", "fdcfac0c02ca538343c68117e0a15938" },
			{ "ffffffffffffffffffffc0000000000000000000000000000000000000000000", "00000000000000000000000000000000", "ad4916f5ee5772be764fc027b8a6e539" },
			{ "ffffffffffffffffffffe0000000000000000000000000000000000000000000", "00000000000000000000000000000000", "2e16873e1678610d7e14c02d002ea845" },
			{ "fffffffffffffffffffff0000000000000000000000000000000000000000000", "00000000000000000000000000000000", "4e6e627c1acc51340053a8236d579576" },
			{ "fffffffffffffffffffff8000000000000000000000000000000000000000000", "00000000000000000000000000000000", "ab0c8410aeeead92feec1eb430d652cb" },
			{ "fffffffffffffffffffffc000000000000000000000000000000000000000000", "00000000000000000000000000000000", "e86f7e23e835e114977f60e1a592202e" },
			{ "fffffffffffffffffffffe000000000000000000000000000000000000000000", "00000000000000000000000000000000", "e68ad5055a367041fade09d9a70a794b" },
			{ "ffffffffffffffffffffff000000000000000000000000000000000000000000", "00000000000000000000000000000000", "0791823a3c666bb6162825e78606a7fe" },
			{ "ffffffffffffffffffffff800000000000000000000000000000000000000000", "00000000000000000000000000000000", "dcca366a9bf47b7b868b77e25c18a364" },
			{ "ffffffffffffffffffffffc00000000000000000000000000000000000000000", "00000000000000000000000000000000", "684c9efc237e4a442965f84bce20247a" },
			{ "ffffffffffffffffffffffe00000000000000000000000000000000000000000", "00000000000000000000000000000000", "a858411ffbe63fdb9c8aa1bfaed67b52" },
			{ "fffffffffffffffffffffff00000000000000000000000000000000000000000", "00000000000000000000000000000000", "04bc3da2179c3015498b0e03910db5b8" },
			{ "fffffffffffffffffffffff80000000000000000000000000000000000000000", "00000000000000000000000000000000", "40071eeab3f935dbc25d00841460260f" },
			{ "fffffffffffffffffffffffc0000000000000000000000000000000000000000", "00000000000000000000000000000000", "0ebd7c30ed2016e08ba806ddb008bcc8" },
			{ "fffffffffffffffffffffffe0000000000000000000000000000000000000000", "00000000000000000000000000000000", "15c6becf0f4cec7129cbd22d1a79b1b8" },
			{ "ffffffffffffffffffffffff0000000000000000000000000000000000000000", "00000000000000000000000000000000", "0aeede5b91f721700e9e62edbf60b781" },
			{ "ffffffffffffffffffffffff8000000000000000000000000000000000000000", "00000000000000000000000000000000", "266581af0dcfbed1585e0a242c64b8df" },
			{ "ffffffffffffffffffffffffc000000000000000000000000000000000000000", "00000000000000000000000000000000", "6693dc911662ae473216ba22189a511a" },
			{ "ffffffffffffffffffffffffe000000000000000000000000000000000000000", "00000000000000000000000000000000", "7606fa36d86473e6fb3a1bb0e2c0adf5" },
			{ "fffffffffffffffffffffffff000000000000000000000000000000000000000", "00000000000000000000000000000000", "112078e9e11fbb78e26ffb8899e96b9a" },
			{ "fffffffffffffffffffffffff800000000000000000000000000000000000000", "00000000000000000000000000000000", "40b264e921e9e4a82694589ef3798262" },
			{ "fffffffffffffffffffffffffc00000000000000000000000000000000000000", "00000000000000000000000000000000", "8d4595cb4fa7026715f55bd68e2882f9" },
			{ "fffffffffffffffffffffffffe00000000000000000000000000000000000000", "00000000000000000000000000000000", "b588a302bdbc09197df1edae68926ed9" },
			{ "ffffffffffffffffffffffffff00000000000000000000000000000000000000", "00000000000000000000000000000000", "33f7502390b8a4a221cfecd0666624ba" },
			{ "ffffffffffffffffffffffffff80000000000000000000000000000000000000", "00000000000000000000000000000000", "3d20253adbce3be2373767c4d822c566" },
			{ "ffffffffffffffffffffffffffc0000000000000000000000000000000000000", "00000000000000000000000000000000", "a42734a3929bf84cf0116c9856a3c18c" },
			{ "ffffffffffffffffffffffffffe0000000000000000000000000000000000000", "00000000000000000000000000000000", "e3abc4939457422bb957da3c56938c6d" },
			{ "fffffffffffffffffffffffffff0000000000000000000000000000000000000", "00000000000000000000000000000000", "972bdd2e7c525130fadc8f76fc6f4b3f" },
			{ "fffffffffffffffffffffffffff8000000000000000000000000000000000000", "00000000000000000000000000000000", "84a83d7b94c699cbcb8a7d9b61f64093" },
			{ "fffffffffffffffffffffffffffc000000000000000000000000000000000000", "00000000000000000000000000000000", "ce61d63514aded03d43e6ebfc3a9001f" },
			{ "fffffffffffffffffffffffffffe000000000000000000000000000000000000", "00000000000000000000000000000000", "6c839dd58eeae6b8a36af48ed63d2dc9" },
			{ "ffffffffffffffffffffffffffff000000000000000000000000000000000000", "00000000000000000000000000000000", "cd5ece55b8da3bf622c4100df5de46f9" },
			{ "ffffffffffffffffffffffffffff800000000000000000000000000000000000", "00000000000000000000000000000000", "3b6f46f40e0ac5fc0a9c1105f800f48d" },
			{ "ffffffffffffffffffffffffffffc00000000000000000000000000000000000", "00000000000000000000000000000000", "ba26d47da3aeb028de4fb5b3a854a24b" },
			{ "ffffffffffffffffffffffffffffe00000000000000000000000000000000000", "00000000000000000000000000000000", "87f53bf620d3677268445212904389d5" },
			{ "fffffffffffffffffffffffffffff00000000000000000000000000000000000", "00000000000000000000000000000000", "10617d28b5e0f4605492b182a5d7f9f6" },
			{ "fffffffffffffffffffffffffffff80000000000000000000000000000000000", "00000000000000000000000000000000", "9aaec4fabbf6fae2a71feff02e372b39" },
			{ "fffffffffffffffffffffffffffffc0000000000000000000000000000000000", "00000000000000000000000000000000", "3a90c62d88b5c42809abf782488ed130" },
			{ "fffffffffffffffffffffffffffffe0000000000000000000000000000000000", "00000000000000000000000000000000", "f1f1c5a40899e15772857ccb65c7a09a" },
			{ "ffffffffffffffffffffffffffffff0000000000000000000000000000000000", "00000000000000000000000000000000", "190843d29b25a3897c692ce1dd81ee52" },
			{ "ffffffffffffffffffffffffffffff8000000000000000000000000000000000", "00000000000000000000000000000000", "a866bc65b6941d86e8420a7ffb0964db" },
			{ "ffffffffffffffffffffffffffffffc000000000000000000000000000000000", "00000000000000000000000000000000", "8193c6ff85225ced4255e92f6e078a14" },
			{ "ffffffffffffffffffffffffffffffe000000000000000000000000000000000", "00000000000000000000000000000000", "9661cb2424d7d4a380d547f9e7ec1cb9" },
			{ "fffffffffffffffffffffffffffffff000000000000000000000000000000000", "00000000000000000000000000000000", "86f93d9ec08453a071e2e2877877a9c8" },
			{ "fffffffffffffffffffffffffffffff800000000000000000000000000000000", "00000000000000000000000000000000", "27eefa80ce6a4a9d598e3fec365434d2" },
			{ "fffffffffffffffffffffffffffffffc00000000000000000000000000000000", "00000000000000000000000000000000", "d62068444578e3ab39ce7ec95dd045dc" },
			{ "fffffffffffffffffffffffffffffffe00000000000000000000000000000000", "00000000000000000000000000000000", "b5f71d4dd9a71fe5d8bc8ba7e6ea3048" },
			{ "ffffffffffffffffffffffffffffffff00000000000000000000000000000000", "00000000000000000000000000000000", "6825a347ac479d4f9d95c5cb8d3fd7e9" },
			{ "ffffffffffffffffffffffffffffffff80000000000000000000000000000000", "00000000000000000000000000000000", "e3714e94a5778955cc0346358e94783a" },
			{ "ffffffffffffffffffffffffffffffffc0000000000000000000000000000000", "00000000000000000000000000000000", "d836b44bb29e0c7d89fa4b2d4b677d2a" },
			{ "ffffffffffffffffffffffffffffffffe0000000000000000000000000000000", "00000000000000000000000000000000", "5d454b75021d76d4b84f873a8f877b92" },
			{ "fffffffffffffffffffffffffffffffff0000000000000000000000000000000", "00000000000000000000000000000000", "c3498f7eced2095314fc28115885b33f" },
			{ "fffffffffffffffffffffffffffffffff8000000000000000000000000000000", "00000000000000000000000000000000", "6e668856539ad8e405bd123fe6c88530" },
			{ "fffffffffffffffffffffffffffffffffc000000000000000000000000000000", "00000000000000000000000000000000", "8680db7f3a87b8605543cfdbe6754076" },
			{ "fffffffffffffffffffffffffffffffffe000000000000000000000000000000", "00000000000000000000000000000000", "6c5d03b13069c3658b3179be91b0800c" },
			{ "ffffffffffffffffffffffffffffffffff000000000000000000000000000000", "00000000000000000000000000000000", "ef1b384ac4d93eda00c92add0995ea5f" },
			{ "ffffffffffffffffffffffffffffffffff800000000000000000000000000000", "00000000000000000000000000000000", "bf8115805471741bd5ad20a03944790f" },
			{ "ffffffffffffffffffffffffffffffffffc00000000000000000000000000000", "00000000000000000000000000000000", "c64c24b6894b038b3c0d09b1df068b0b" },
			{ "ffffffffffffffffffffffffffffffffffe00000000000000000000000000000", "00000000000000000000000000000000", "3967a10cffe27d0178545fbf6a40544b" },
			{ "fffffffffffffffffffffffffffffffffff00000000000000000000000000000", "00000000000000000000000000000000", "7c85e9c95de1a9ec5a5363a8a053472d" },
			{ "fffffffffffffffffffffffffffffffffff80000000000000000000000000000", "00000000000000000000000000000000", "a9eec03c8abec7ba68315c2c8c2316e0" },
			{ "fffffffffffffffffffffffffffffffffffc0000000000000000000000000000", "00000000000000000000000000000000", "cac8e414c2f388227ae14986fc983524" },
			{ "fffffffffffffffffffffffffffffffffffe0000000000000000000000000000", "00000000000000000000000000000000", "5d942b7f4622ce056c3ce3ce5f1dd9d6" },
			{ "ffffffffffffffffffffffffffffffffffff0000000000000000000000000000", "00000000000000000000000000000000", "d240d648ce21a3020282c3f1b528a0b6" },
			{ "ffffffffffffffffffffffffffffffffffff8000000000000000000000000000", "00000000000000000000000000000000", "45d089c36d5c5a4efc689e3b0de10dd5" },
			{ "ffffffffffffffffffffffffffffffffffffc000000000000000000000000000", "00000000000000000000000000000000", "b4da5df4becb5462e03a0ed00d295629" },
			{ "ffffffffffffffffffffffffffffffffffffe000000000000000000000000000", "00000000000000000000000000000000", "dcf4e129136c1a4b7a0f38935cc34b2b" },
			{ "fffffffffffffffffffffffffffffffffffff000000000000000000000000000", "00000000000000000000000000000000", "d9a4c7618b0ce48a3d5aee1a1c0114c4" },
			{ "fffffffffffffffffffffffffffffffffffff800000000000000000000000000", "00000000000000000000000000000000", "ca352df025c65c7b0bf306fbee0f36ba" },
			{ "fffffffffffffffffffffffffffffffffffffc00000000000000000000000000", "00000000000000000000000000000000", "238aca23fd3409f38af63378ed2f5473" },
			{ "fffffffffffffffffffffffffffffffffffffe00000000000000000000000000", "00000000000000000000000000000000", "59836a0e06a79691b36667d5380d8188" },
			{ "ffffffffffffffffffffffffffffffffffffff00000000000000000000000000", "00000000000000000000000000000000", "33905080f7acf1cdae0a91fc3e85aee4" },
			{ "ffffffffffffffffffffffffffffffffffffff80000000000000000000000000", "00000000000000000000000000000000", "72c9e4646dbc3d6320fc6689d93e8833" },
			{ "ffffffffffffffffffffffffffffffffffffffc0000000000000000000000000", "00000000000000000000000000000000", "ba77413dea5925b7f5417ea47ff19f59" },
			{ "ffffffffffffffffffffffffffffffffffffffe0000000000000000000000000", "00000000000000000000000000000000", "6cae8129f843d86dc786a0fb1a184970" },
			{ "fffffffffffffffffffffffffffffffffffffff0000000000000000000000000", "00000000000000000000000000000000", "fcfefb534100796eebbd990206754e19" },
			{ "fffffffffffffffffffffffffffffffffffffff8000000000000000000000000", "00000000000000000000000000000000", "8c791d5fdddf470da04f3e6dc4a5b5b5" },
			{ "fffffffffffffffffffffffffffffffffffffffc000000000000000000000000", "00000000000000000000000000000000", "c93bbdc07a4611ae4bb266ea5034a387" },
			{ "fffffffffffffffffffffffffffffffffffffffe000000000000000000000000", "00000000000000000000000000000000", "c102e38e489aa74762f3efc5bb23205a" },
			{ "ffffffffffffffffffffffffffffffffffffffff000000000000000000000000", "00000000000000000000000000000000", "93201481665cbafc1fcc220bc545fb3d" },
			{ "ffffffffffffffffffffffffffffffffffffffff800000000000000000000000", "00000000000000000000000000000000", "4960757ec6ce68cf195e454cfd0f32ca" },
			{ "ffffffffffffffffffffffffffffffffffffffffc00000000000000000000000", "00000000000000000000000000000000", "feec7ce6a6cbd07c043416737f1bbb33" },
			{ "ffffffffffffffffffffffffffffffffffffffffe00000000000000000000000", "00000000000000000000000000000000", "11c5413904487a805d70a8edd9c35527" },
			{ "fffffffffffffffffffffffffffffffffffffffff00000000000000000000000", "00000000000000000000000000000000", "347846b2b2e36f1f0324c86f7f1b98e2" },
			{ "fffffffffffffffffffffffffffffffffffffffff80000000000000000000000", "00000000000000000000000000000000", "332eee1a0cbd19ca2d69b426894044f0" },
			{ "fffffffffffffffffffffffffffffffffffffffffc0000000000000000000000", "00000000000000000000000000000000", "866b5b3977ba6efa5128efbda9ff03cd" },
			{ "fffffffffffffffffffffffffffffffffffffffffe0000000000000000000000", "00000000000000000000000000000000", "cc1445ee94c0f08cdee5c344ecd1e233" },
			{ "ffffffffffffffffffffffffffffffffffffffffff0000000000000000000000", "00000000000000000000000000000000", "be288319029363c2622feba4b05dfdfe" },
			{ "ffffffffffffffffffffffffffffffffffffffffff8000000000000000000000", "00000000000000000000000000000000", "cfd1875523f3cd21c395651e6ee15e56" },
			{ "ffffffffffffffffffffffffffffffffffffffffffc000000000000000000000", "00000000000000000000000000000000", "cb5a408657837c53bf16f9d8465dce19" },
			{ "ffffffffffffffffffffffffffffffffffffffffffe000000000000000000000", "00000000000000000000000000000000", "ca0bf42cb107f55ccff2fc09ee08ca15" },
			{ "fffffffffffffffffffffffffffffffffffffffffff000000000000000000000", "00000000000000000000000000000000", "fdd9bbb4a7dc2e4a23536a5880a2db67" },
			{ "fffffffffffffffffffffffffffffffffffffffffff800000000000000000000", "00000000000000000000000000000000", "ede447b362c484993dec9442a3b46aef" },
			{ "fffffffffffffffffffffffffffffffffffffffffffc00000000000000000000", "00000000000000000000000000000000", "10dffb05904bff7c4781df780ad26837" },
			{ "fffffffffffffffffffffffffffffffffffffffffffe00000000000000000000", "00000000000000000000000000000000", "c33bc13e8de88ac25232aa7496398783" },
			{ "ffffffffffffffffffffffffffffffffffffffffffff00000000000000000000", "00000000000000000000000000000000", "ca359c70803a3b2a3d542e8781dea975" },
			{ "ffffffffffffffffffffffffffffffffffffffffffff80000000000000000000", "00000000000000000000000000000000", "bcc65b526f88d05b89ce8a52021fdb06" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffc0000000000000000000", "00000000000000000000000000000000", "db91a38855c8c4643851fbfb358b0109" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffe0000000000000000000", "00000000000000000000000000000000", "ca6e8893a114ae8e27d5ab03a5499610" },
			{ "fffffffffffffffffffffffffffffffffffffffffffff0000000000000000000", "00000000000000000000000000000000", "6629d2b8df97da728cdd8b1e7f945077" },
			{ "fffffffffffffffffffffffffffffffffffffffffffff8000000000000000000", "00000000000000000000000000000000", "4570a5a18cfc0dd582f1d88d5c9a1720" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffc000000000000000000", "00000000000000000000000000000000", "72bc65aa8e89562e3f274d45af1cd10b" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffe000000000000000000", "00000000000000000000000000000000", "98551da1a6503276ae1c77625f9ea615" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffff000000000000000000", "00000000000000000000000000000000", "0ddfe51ced7e3f4ae927daa3fe452cee" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffff800000000000000000", "00000000000000000000000000000000", "db826251e4ce384b80218b0e1da1dd4c" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffc00000000000000000", "00000000000000000000000000000000", "2cacf728b88abbad7011ed0e64a1680c" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffe00000000000000000", "00000000000000000000000000000000", "330d8ee7c5677e099ac74c9994ee4cfb" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffff00000000000000000", "00000000000000000000000000000000", "edf61ae362e882ddc0167474a7a77f3a" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffff80000000000000000", "00000000000000000000000000000000", "6168b00ba7859e0970ecfd757efecf7c" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffc0000000000000000", "00000000000000000000000000000000", "d1415447866230d28bb1ea18a4cdfd02" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffe0000000000000000", "00000000000000000000000000000000", "516183392f7a8763afec68a060264141" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffff0000000000000000", "00000000000000000000000000000000", "77565c8d73cfd4130b4aa14d8911710f" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffff8000000000000000", "00000000000000000000000000000000", "37232a4ed21ccc27c19c9610078cabac" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffc000000000000000", "00000000000000000000000000000000", "804f32ea71828c7d329077e712231666" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffe000000000000000", "00000000000000000000000000000000", "d64424f23cb97215e9c2c6f28d29eab7" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffff000000000000000", "00000000000000000000000000000000", "023e82b533f68c75c238cebdb2ee89a2" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffff800000000000000", "00000000000000000000000000000000", "193a3d24157a51f1ee0893f6777417e7" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffc00000000000000", "00000000000000000000000000000000", "84ecacfcd400084d078612b1945f2ef5" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffe00000000000000", "00000000000000000000000000000000", "1dcd8bb173259eb33a5242b0de31a455" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffff00000000000000", "00000000000000000000000000000000", "35e9eddbc375e792c19992c19165012b" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffff80000000000000", "00000000000000000000000000000000", "8a772231c01dfdd7c98e4cfddcc0807a" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffc0000000000000", "00000000000000000000000000000000", "6eda7ff6b8319180ff0d6e65629d01c3" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffe0000000000000", "00000000000000000000000000000000", "c267ef0e2d01a993944dd397101413cb" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffff0000000000000", "00000000000000000000000000000000", "e9f80e9d845bcc0f62926af72eabca39" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffff8000000000000", "00000000000000000000000000000000", "6702990727aa0878637b45dcd3a3b074" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffc000000000000", "00000000000000000000000000000000", "2e2e647d5360e09230a5d738ca33471e" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffe000000000000", "00000000000000000000000000000000", "1f56413c7add6f43d1d56e4f02190330" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffff000000000000", "00000000000000000000000000000000", "69cd0606e15af729d6bca143016d9842" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffff800000000000", "00000000000000000000000000000000", "a085d7c1a500873a20099c4caa3c3f5b" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffc00000000000", "00000000000000000000000000000000", "4fc0d230f8891415b87b83f95f2e09d1" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffe00000000000", "00000000000000000000000000000000", "4327d08c523d8eba697a4336507d1f42" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffff00000000000", "00000000000000000000000000000000", "7a15aab82701efa5ae36ab1d6b76290f" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffff80000000000", "00000000000000000000000000000000", "5bf0051893a18bb30e139a58fed0fa54" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffc0000000000", "00000000000000000000000000000000", "97e8adf65638fd9cdf3bc22c17fe4dbd" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffe0000000000", "00000000000000000000000000000000", "1ee6ee326583a0586491c96418d1a35d" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffff0000000000", "00000000000000000000000000000000", "26b549c2ec756f82ecc48008e529956b" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffff8000000000", "00000000000000000000000000000000", "70377b6da669b072129e057cc28e9ca5" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffc000000000", "00000000000000000000000000000000", "9c94b8b0cb8bcc919072262b3fa05ad9" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffe000000000", "00000000000000000000000000000000", "2fbb83dfd0d7abcb05cd28cad2dfb523" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffff000000000", "00000000000000000000000000000000", "96877803de77744bb970d0a91f4debae" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffff800000000", "00000000000000000000000000000000", "7379f3370cf6e5ce12ae5969c8eea312" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffc00000000", "00000000000000000000000000000000", "02dc99fa3d4f98ce80985e7233889313" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffe00000000", "00000000000000000000000000000000", "1e38e759075ba5cab6457da51844295a" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffff00000000", "00000000000000000000000000000000", "70bed8dbf615868a1f9d9b05d3e7a267" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffff80000000", "00000000000000000000000000000000", "234b148b8cb1d8c32b287e896903d150" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffc0000000", "00000000000000000000000000000000", "294b033df4da853f4be3e243f7e513f4" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffe0000000", "00000000000000000000000000000000", "3f58c950f0367160adec45f2441e7411" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffff0000000", "00000000000000000000000000000000", "37f655536a704e5ace182d742a820cf4" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffff8000000", "00000000000000000000000000000000", "ea7bd6bb63418731aeac790fe42d61e8" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffc000000", "00000000000000000000000000000000", "e74a4c999b4c064e48bb1e413f51e5ea" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffe000000", "00000000000000000000000000000000", "ba9ebefdb4ccf30f296cecb3bc1943e8" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffff000000", "00000000000000000000000000000000", "3194367a4898c502c13bb7478640a72d" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffff800000", "00000000000000000000000000000000", "da797713263d6f33a5478a65ef60d412" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffc00000", "00000000000000000000000000000000", "d1ac39bb1ef86b9c1344f214679aa376" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffe00000", "00000000000000000000000000000000", "2fdea9e650532be5bc0e7325337fd363" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffff00000", "00000000000000000000000000000000", "d3a204dbd9c2af158b6ca67a5156ce4a" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffff80000", "00000000000000000000000000000000", "3a0a0e75a8da36735aee6684d965a778" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffffc0000", "00000000000000000000000000000000", "52fc3e620492ea99641ea168da5b6d52" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffffe0000", "00000000000000000000000000000000", "d2e0c7f15b4772467d2cfc873000b2ca" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffff0000", "00000000000000000000000000000000", "563531135e0c4d70a38f8bdb190ba04e" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffff8000", "00000000000000000000000000000000", "a8a39a0f5663f4c0fe5f2d3cafff421a" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffc000", "00000000000000000000000000000000", "d94b5e90db354c1e42f61fabe167b2c0" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffe000", "00000000000000000000000000000000", "50e6d3c9b6698a7cd276f96b1473f35a" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffffff000", "00000000000000000000000000000000", "9338f08e0ebee96905d8f2e825208f43" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffffff800", "00000000000000000000000000000000", "8b378c86672aa54a3a266ba19d2580ca" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffc00", "00000000000000000000000000000000", "cca7c3086f5f9511b31233da7cab9160" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffe00", "00000000000000000000000000000000", "5b40ff4ec9be536ba23035fa4f06064c" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffff00", "00000000000000000000000000000000", "60eb5af8416b257149372194e8b88749" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffff80", "00000000000000000000000000000000", "2f005a8aed8a361c92e440c15520cbd1" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffc0", "00000000000000000000000000000000", "7b03627611678a997717578807a800e2" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffe0", "00000000000000000000000000000000", "cf78618f74f6f3696e0a4779b90b5a77" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffff0", "00000000000000000000000000000000", "03720371a04962eaea0a852e69972858" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffff8", "00000000000000000000000000000000", "1f8a8133aa8ccf70e2bd3285831ca6b7" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffc", "00000000000000000000000000000000", "27936bd27fb1468fc8b48bc483321725" },
			{ "fffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffe", "00000000000000000000000000000000", "b07d4f3e2cd2ef2eb545980754dfea0f" },
			{ "ffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffffff", "00000000000000000000000000000000", "4bf85f1b5d54adbc307b0a048389adcb" },
		};
		for (auto& Test : VarKeyCases)
		{
			RunTest(Test.Key, Test.PlainText, Test.CipherText);
		}
	}

	// 6.4 Monte Carlo Test, from ECBMCT256.rsp
	SECTION("Monte Carlo")
	{
		static const AesTestCase MonteCarloCases[] =
		{
			{ "f9e8389f5b80712e3886cc1fa2d28a3b8c9cd88a2d4a54c6aa86ce0fef944be0", "b379777f9050e2a818f2940cbbd9aba4", "6893ebaf0a1fccc704326529fdfb60db" },
			{ "db9ea5a2284fa17fb63e13bf891c8e42e40f332527559801aeb4ab26126f2b3b", "6893ebaf0a1fccc704326529fdfb60db", "f3c78a5e85e5439bf26d5818718157d6" },
			{ "7099ed88e82744228a5303ae2ef6c0d017c8b97ba2b0db9a5cd9f33e63ee7ced", "f3c78a5e85e5439bf26d5818718157d6", "2326b958b00b3050697eedb08cc20504" },
			{ "5e9e65ea96e78dd4fb78ea1184f6ebde34ee002312bbebca35a71e8eef2c79e9", "2326b958b00b3050697eedb08cc20504", "ec4332d5e3cebd3e0f5fc51452f4560d" },
			{ "33acf1cafc822646dc869e905bd26f9ad8ad32f6f17556f43af8db9abdd82fe4", "ec4332d5e3cebd3e0f5fc51452f4560d", "5da58b5ef2076340d555f861c3449a77" },
			{ "eb0ae85c1b44d5db4729d268f49be2a08508b9a8037235b4efad23fb7e9cb593", "5da58b5ef2076340d555f861c3449a77", "307d50c18a0b6a08402ff131d72cb7ec" },
			{ "fac93b561a9b6a0e809d71ecdb980afab575e96989795fbcaf82d2caa9b0027f", "307d50c18a0b6a08402ff131d72cb7ec", "92c34165a2963e77e05e2d6fc2d931d5" },
			{ "a0559e41d58af36174a67246df87541b27b6a80c2bef61cb4fdcffa56b6933aa", "92c34165a2963e77e05e2d6fc2d931d5", "cb33d519a1fdb1d5fbb185c47870c1ed" },
			{ "e48824d6c2251d3a27f38fb543c31fc1ec857d158a12d01eb46d7a611319f247", "cb33d519a1fdb1d5fbb185c47870c1ed", "78fb452f384c8f870e572890588f3728" },
			{ "7a33440ad7c69d583355c745e5c88c47947e383ab25e5f99ba3a52f14b96c56f", "78fb452f384c8f870e572890588f3728", "12375e02a8bbc84b00feaab54a66db43" },
			{ "0e6877c7fdc234efb9afcd96b4ebdb83864966381ae597d2bac4f84401f01e2c", "12375e02a8bbc84b00feaab54a66db43", "eab1606610b55c857f2b4bf1cf3feba0" },
			{ "ec91d3550c79ab4914a26987725ab1396cf8065e0a50cb57c5efb3b5cecff58c", "eab1606610b55c857f2b4bf1cf3feba0", "6c73381147de97961cc26ad26602a45a" },
			{ "83caed5a49579b3a55a71e5ece5966e5008b3e4f4d8e5cc1d92dd967a8cd51d6", "6c73381147de97961cc26ad26602a45a", "e76c08fd29bf015352003c636fee5ff9" },
			{ "91b9b50908968361dcd8f4ba236fa199e7e736b264315d928b2de504c7230e2f", "e76c08fd29bf015352003c636fee5ff9", "6f26b8191a2b059dcdeb3dbabc437c29" },
			{ "94fc46213c870f7965b88773afe93b1388c18eab7e1a580f46c6d8be7b607206", "6f26b8191a2b059dcdeb3dbabc437c29", "e91e2fcef14dd4251caec97c45223fef" },
			{ "b0deff009aff61f65763b0b9fdd39a9061dfa1658f578c2a5a6811c23e424de9", "e91e2fcef14dd4251caec97c45223fef", "98fb2122912360f07916e4802c0ea1e5" },
			{ "968f3c88d27f1be8decb00c4d464d369f92480471e74ecda237ef542124cec0c", "98fb2122912360f07916e4802c0ea1e5", "b4a15c59976b39da50da8ed393f27a62" },
			{ "35fe06a60309581565a97232140668464d85dc1e891fd50073a47b9181be966e", "b4a15c59976b39da50da8ed393f27a62", "d8b57d7a72ef92409c51d40bb8c4cbc5" },
			{ "4739043d7750bcf4a6f269a3d54083ca9530a164fbf04740eff5af9a397a5dab", "d8b57d7a72ef92409c51d40bb8c4cbc5", "548cc893e80caf5a601c2381517f8c5b" },
			{ "4b065b5195f2ddf6f5d0aed72ff7a1e3c1bc69f713fce81a8fe98c1b6805d1f0", "548cc893e80caf5a601c2381517f8c5b", "1c2238c560d678d40b48cc8034add0c4" },
			{ "1a949129e14c5963d997c86a6352ea53dd9e5132732a90ce84a1409b5ca80134", "1c2238c560d678d40b48cc8034add0c4", "2771ff806e061df8ad4aa877717bf309" },
			{ "91cd3e48f4b42c432bed9848583e6dc7faefaeb21d2c8d3629ebe8ec2dd3f23d", "2771ff806e061df8ad4aa877717bf309", "065593fa1fcdb481bb27f334505543f5" },
			{ "42d6f7b585a0a0d356c59be3d07d4d41fcba3d4802e139b792cc1bd87d86b1c8", "065593fa1fcdb481bb27f334505543f5", "d3f309c0039b15d14eb8b739a94b94fa" },
			{ "72900f0ecbf90fa058805deb430815072f493488017a2c66dc74ace1d4cd2532", "d3f309c0039b15d14eb8b739a94b94fa", "4e441a3fb277d6fbe0ed7c6e080d9a9f" },
			{ "12ef8789b91a8e35fd0ac79457a906f0610d2eb7b30dfa9d3c99d08fdcc0bfad", "4e441a3fb277d6fbe0ed7c6e080d9a9f", "cb9241bc964cbc9823531f68e3a03b7c" },
			{ "2cbe5980ad54c5dbf2ee3db1e9875733aa9f6f0b254146051fcacfe73f6084d1", "cb9241bc964cbc9823531f68e3a03b7c", "70b17c3e869aca6076617a2cf75e9f4a" },
			{ "dc4da3812736a2a603bcc6390763c5c5da2e1335a3db8c6569abb5cbc83e1b9b", "70b17c3e869aca6076617a2cf75e9f4a", "f4b8a6ed6d6d72aff59484314f210bb9" },
			{ "2e4e21a6eb77bb39a1cb5cd20cc5fd3b2e96b5d8ceb6feca9c3f31fa871f1022", "f4b8a6ed6d6d72aff59484314f210bb9", "f56a4597beaeafc0c14ee73988bcbee8" },
			{ "54d5c037e61fd09e5cf57fd34c5e0192dbfcf04f7018510a5d71d6c30fa3aeca", "f56a4597beaeafc0c14ee73988bcbee8", "04333079d5352236e1c79213f3f38dbe" },
			{ "3a5da203e03fa399caeb1fac63679b56dfcfc036a52d733cbcb644d0fc502374", "04333079d5352236e1c79213f3f38dbe", "22c97ecdf4af830f94b11951f41e4d29" },
			{ "5c84147d6ab2051b56b0993a7cbfa306fd06befb5182f03328075d81084e6e5d", "22c97ecdf4af830f94b11951f41e4d29", "def46a3b39c8048431d2491d97daa6ea" },
			{ "7af77aa155a33f658283ebc3e9eb708923f2d4c0684af4b719d5149c9f94c8b7", "def46a3b39c8048431d2491d97daa6ea", "4c6367ad8a2190366c3d730fe5eeb6ee" },
			{ "edabba9dc9d87357bb91da6931c743e16f91b36de26b648175e867937a7a7e59", "4c6367ad8a2190366c3d730fe5eeb6ee", "90bd09b4eb0f7d8397c0026cefea8fb3" },
			{ "ab7ef1b4a1e43771de88e158ad26a419ff2cbad909641902e22865ff9590f1ea", "90bd09b4eb0f7d8397c0026cefea8fb3", "25aefcfa6ec98fae81b93afad7761711" },
			{ "6b0145c6cbeaae320be86c2909c8d643da82462367ad96ac63915f0542e6e6fb", "25aefcfa6ec98fae81b93afad7761711", "607b26f4eb3585e3e886e75c0f3a01cf" },
			{ "b46d28c20a614b6d986e92795258631cbaf960d78c98134f8b17b8594ddce734", "607b26f4eb3585e3e886e75c0f3a01cf", "fe35fe16a3290042c28c021ee9ede73f" },
			{ "93c5c8403410764717e547e3d07b0ec344cc9ec12fb1130d499bba47a431000b", "fe35fe16a3290042c28c021ee9ede73f", "0141871c357a00ad37ae65597830cac8" },
			{ "cf85ea60e0c94e611fedba48e19e9693458d19dd1acb13a07e35df1edc01cac3", "0141871c357a00ad37ae65597830cac8", "df8bc7782ae10eb2c1bdfdc8887e4907" },
			{ "d40735932ea269aeb51fa3caf0c176d89a06dea5302a1d12bf8822d6547f83c4", "df8bc7782ae10eb2c1bdfdc8887e4907", "1ac65c65b6cd8ab751f1d908ebd962ab" },
			{ "9a7120341819fe54fc8a750d17ecf20f80c082c086e797a5ee79fbdebfa6e16f", "1ac65c65b6cd8ab751f1d908ebd962ab", "9f10b53fb6adf7189e277d04e351aa9d" },
			{ "8c19642c172cdb804059f751b3f25cff1fd037ff304a60bd705e86da5cf74bf2", "9f10b53fb6adf7189e277d04e351aa9d", "c94ecc943ddc8d1eeafadd173cf73fdc" },
			{ "e1cc9575e51d6a8240e6f3cd8958e1a6d69efb6b0d96eda39aa45bcd6000742e", "c94ecc943ddc8d1eeafadd173cf73fdc", "5dd513d74b72f9d46b06b4b96a94c3bd" },
			{ "ce41ff2d70b2169fad6a5f9ead4471e88b4be8bc46e41477f1a2ef740a94b793", "5dd513d74b72f9d46b06b4b96a94c3bd", "1594a31cc22cdbaf2011a9a317538608" },
			{ "7b467b87180772677903a043bc63c3ef9edf4ba084c8cfd8d1b346d71dc7319b", "1594a31cc22cdbaf2011a9a317538608", "4d5e8d6fa3e2ea343b5afb77124529aa" },
			{ "ca068b3afc717d1f6e8dc7e4eca0f56fd381c6cf272a25eceae9bda00f821831", "4d5e8d6fa3e2ea343b5afb77124529aa", "1bbf651cf61c295c96e73a210483d7a1" },
			{ "d04d80998a971ec52d5390fa7eaf78eac83ea3d3d1360cb07c0e87810b01cf90", "1bbf651cf61c295c96e73a210483d7a1", "84b42f75c3b62bbf21707ba66cca399f" },
			{ "2cfb1c71e8cac872e6a6dc90b0195acf4c8a8ca61280270f5d7efc2767cbf60f", "84b42f75c3b62bbf21707ba66cca399f", "a0c514e1e6a9c659f605eff4cf4951a0" },
			{ "405bc450a19e6dae76847b8eac858c8bec4f9847f429e156ab7b13d3a882a7af", "a0c514e1e6a9c659f605eff4cf4951a0", "1dcc38307c6b3c31e25f868b279b3711" },
			{ "35755fe97d9aeb1e67c1f4ca5a40ce1ff183a0778842dd67492495588f1990be", "1dcc38307c6b3c31e25f868b279b3711", "cdd744574664be7b221d7a2921b4f0b2" },
			{ "7e9d8b4bbd2aad70d229247d5880a0cb3c54e420ce26631c6b39ef71aead600c", "cdd744574664be7b221d7a2921b4f0b2", "e602ef85184ad4a86ed339e9403f541c" },
			{ "73be0bb5b74a8f6324f8a5f0acbf5faada560ba5d66cb7b405ead698ee923410", "e602ef85184ad4a86ed339e9403f541c", "7c227f03e605ff14c42fb9f8e8786e84" },
			{ "c82e22faeca51a38045f2a3a04a0e6e2a67474a6306948a0c1c56f6006ea5a94", "7c227f03e605ff14c42fb9f8e8786e84", "94c0466b27aa4d361b19c250329c14a0" },
			{ "7c1dd42bdff414e3733b846cbb00c43432b432cd17c30596dadcad3034764e34", "94c0466b27aa4d361b19c250329c14a0", "fd6b281c93c1fd8e9b83b69ecf722f3b" },
			{ "aa7ec003efbca86fe5ad028d0571cf3acfdf1ad18402f818415f1baefb04610f", "fd6b281c93c1fd8e9b83b69ecf722f3b", "a75b28be92dc53de9d44714918e3c541" },
			{ "14ba26bafb6c8410104d70ec23876db56884326f16deabc6dc1b6ae7e3e7a44e", "a75b28be92dc53de9d44714918e3c541", "48693025ba68f3fa3515e7112b6e32aa" },
			{ "33b40738b39ebe39118938c6461ebd7820ed024aacb6583ce90e8df6c88996e4", "48693025ba68f3fa3515e7112b6e32aa", "6bfa37cc5ed6e5d4ddf40b1301957f73" },
			{ "0d16eab1b56e2a468bf1ba1035129d574b173586f260bde834fa86e5c91ce997", "6bfa37cc5ed6e5d4ddf40b1301957f73", "ca18b89c3225877e02e21042af9836a8" },
			{ "44b55514a5b4a5a4294f3cf8e75ff73c810f8d1ac0453a96361896a76684df3f", "ca18b89c3225877e02e21042af9836a8", "c3ebc08368be58f8f514cfbd5f98ed52" },
			{ "265f1a7a53151bc51526fe476e8c63ad42e44d99a8fb626ec30c591a391c326d", "c3ebc08368be58f8f514cfbd5f98ed52", "788552397ddf2d5841dbc0e73dd11150" },
			{ "23fa4630937015585ac065e6a7bd93023a611fa0d5244f3682d799fd04cd233d", "788552397ddf2d5841dbc0e73dd11150", "e9d728bc19b50809bc1245552091d0b2" },
			{ "9dd3a2408e5694fced28964c7f442e6ed3b6371ccc91473f3ec5dca8245cf38f", "e9d728bc19b50809bc1245552091d0b2", "ce08d78b58179b7957eeab6f74ddb10f" },
			{ "055d53ef574a63f77c0b3a88e7e7cdb01dbee0979486dc46692b77c750814280", "ce08d78b58179b7957eeab6f74ddb10f", "33ce33e80fcc262e9e862f5ac50c14df" },
			{ "c1a90cdef5d5daaf7b182d6b409940e62e70d37f9b4afa68f7ad589d958d565f", "33ce33e80fcc262e9e862f5ac50c14df", "5c4d21e6045f11f9f91a8cef130304e0" },
			{ "6a257a4f65bdd7d882ff45ffbb9541e5723df2999f15eb910eb7d472868e52bf", "5c4d21e6045f11f9f91a8cef130304e0", "4f50679c2dbfa63b5e55d031afacd44b" },
			{ "5843bf4d6afc64273d328e1baf7821b03d6d9505b2aa4daa50e20443292286f4", "4f50679c2dbfa63b5e55d031afacd44b", "29f64e79de48f3d3d5dff92aa0b1783b" },
			{ "8915060a33758a300c053d365c304d4e149bdb7c6ce2be79853dfd698993fecf", "29f64e79de48f3d3d5dff92aa0b1783b", "8c5bf92515a92449846593a3bdbe521b" },
			{ "be2b2a41b3491511c181508b66ec96f798c02259794b9a3001586eca342dacd4", "8c5bf92515a92449846593a3bdbe521b", "5511fef7a2238277aaa9e25f1d097c19" },
			{ "4d6f38935317ffa0cb29bdfa8a43cd19cdd1dcaedb681847abf18c952924d0cd", "5511fef7a2238277aaa9e25f1d097c19", "2ecd75c6a5eace1c541e12db150a2143" },
			{ "1549ea3d167152c85223b0a603d1fa61e31ca9687e82d65bffef9e4e3c2ef18e", "2ecd75c6a5eace1c541e12db150a2143", "6a4cfcb9e8dae9610df99509daac7be0" },
			{ "ba593f6731f70edab83f5cdf02527436895055d196583f3af2160b47e6828a6e", "6a4cfcb9e8dae9610df99509daac7be0", "185772deff51807147ac5350249b3e1a" },
			{ "7126a05712f987f1c9249bf9a7c17b4f9107270f6909bf4bb5ba5817c219b474", "185772deff51807147ac5350249b3e1a", "bfb1ce7df706bc972e1b8306d44aa135" },
			{ "3d0fa3983ebc8a3f64c4135d7cd3195e2eb6e9729e0f03dc9ba1db1116531541", "bfb1ce7df706bc972e1b8306d44aa135", "f21df1e99a781dba4a68ff3491848f99" },
			{ "88c88ec4d63eb481cd8d6e8e8d2e2715dcab189b04771e66d1c9242587d79ad8", "f21df1e99a781dba4a68ff3491848f99", "e4e8c9e4963a44f5cf27767e4e42fa61" },
			{ "989b9545625a4025f7725d63847213883843d17f924d5a931eee525bc99560b9", "e4e8c9e4963a44f5cf27767e4e42fa61", "27475121d0e3367d0e0c2d9fb39cfe95" },
			{ "f49b61f190f982aa866d5c8f2e5749781f04805e42ae6cee10e27fc47a099e2c", "27475121d0e3367d0e0c2d9fb39cfe95", "c2990626cb34c5d7fafe0430650ed907" },
			{ "101537d5f633ad8dc7e8cc058ce7fe20dd9d8678899aa939ea1c7bf41f07472b", "c2990626cb34c5d7fafe0430650ed907", "1c5d0cca2845b66c371c1760f81e024a" },
			{ "1f6acaa66674733d28dfec92c43c7e76c1c08ab2a1df1f55dd006c94e7194561", "1c5d0cca2845b66c371c1760f81e024a", "6f5a53addd93c9a136401a804a710419" },
			{ "b55bdefe370646699012062df748b6aaae9ad91f7c4cd6f4eb407614ad684178", "6f5a53addd93c9a136401a804a710419", "8e0c070be7109ead0e8cc0aa0bf95e61" },
			{ "de09c34a85d56a2748559c309f04eb722096de149b5c4859e5ccb6bea6911f19", "8e0c070be7109ead0e8cc0aa0bf95e61", "db53b5306561db899a635a56c56e7239" },
			{ "2650ecd133a5df3825072a4df2d95d15fbc56b24fe3d93d07fafece863ff6d20", "db53b5306561db899a635a56c56e7239", "72e5ac05990d3f5508295f95f8973313" },
			{ "a8599f0edd6446b4bb9371e380bf33488920c7216730ac857786b37d9b685e33", "72e5ac05990d3f5508295f95f8973313", "86a8332f16f997bc02af5271c64e7e0f" },
			{ "672e7b0d497458e260084662c566394e0f88f40e71c93b397529e10c5d26203c", "86a8332f16f997bc02af5271c64e7e0f", "8e53858ce7ad0d25410f886eeeca1e0e" },
			{ "73ea41a50cdd5a98644e6f8d974af21381db71829664361c34266962b3ec3e32", "8e53858ce7ad0d25410f886eeeca1e0e", "c6a1d493d607d2cef6aef29ee878c434" },
			{ "14969b3ff78ccdd66c53b8027a79563e477aa5114063e4d2c2889bfc5b94fa06", "c6a1d493d607d2cef6aef29ee878c434", "a4973e0cfaf8d6ac6758615fc81e06df" },
			{ "d09e2741b9480c2166c9c1832654763de3ed9b1dba9b327ea5d0faa3938afcd9", "a4973e0cfaf8d6ac6758615fc81e06df", "67e3243d8bc81887517708a29a5d9ae5" },
			{ "22c65ead303a0395cc9c065ada930ddd840ebf2031532af9f4a7f20109d7663c", "67e3243d8bc81887517708a29a5d9ae5", "de2f35df51644cf5d79984be6c17c14c" },
			{ "8679ef0357516245ec489386419646bc5a218aff6037660c233e76bf65c0a770", "de2f35df51644cf5d79984be6c17c14c", "9eaedabb3bd046ec27a2bdafabdbc861" },
			{ "6043902c115f107639e7c92ba4952301c48f50445be720e0049ccb10ce1b6f11", "9eaedabb3bd046ec27a2bdafabdbc861", "0cde7e6c1aecbbe996865bf30b178de7" },
			{ "aca443a9e31033ea73b1eeda6e7d6ca1c8512e28410b9b09921a90e3c50ce2f6", "0cde7e6c1aecbbe996865bf30b178de7", "ba25bf5be1e4c099b9e45eedd7369cb5" },
			{ "60362e2b0bf998c875c087c370c4ca4772749173a0ef5b902bfece0e123a7e43", "ba25bf5be1e4c099b9e45eedd7369cb5", "805be62789549ce6af74966467f41135" },
			{ "6d7f0f7584162a1fa4dd6764548f355af22f775429bbc776848a586a75ce6f76", "805be62789549ce6af74966467f41135", "ab6001c6c4c56e8ca393c5fd173505ba" },
			{ "d8ecc39ac1d00c53216f6e64e826a7a9594f7692ed7ea9fa27199d9762fb6acc", "ab6001c6c4c56e8ca393c5fd173505ba", "3ba3673f4f495dd1541d47c22b7921c5" },
			{ "493108f91caedf714652149a2b2030fe62ec11ada237f42b7304da5549824b09", "3ba3673f4f495dd1541d47c22b7921c5", "b24fe17cdc5c8cfa4260c38691b57bfa" },
			{ "446af6dd5f58755aeaa0a1226d8c584fd0a3f0d17e6b78d1316419d3d83730f3", "b24fe17cdc5c8cfa4260c38691b57bfa", "86d999a63b96f6c9d9aaf3be6202977b" },
			{ "65f92d4e1723d5e58aeb350c79df28de567a697745fd8e18e8ceea6dba35a788", "86d999a63b96f6c9d9aaf3be6202977b", "c4712aa733f9737f91e4ed61609e02f1" },
			{ "915cee6af4ea95623f7122acda5e9040920b43d07604fd67792a070cdaaba579", "c4712aa733f9737f91e4ed61609e02f1", "0e8c1a77b280f4c753682768fd6f3b23" },
			{ "8e9bb2887fe60d42db4d827f00ba68ff9c8759a7c48409a02a42206427c49e5a", "0e8c1a77b280f4c753682768fd6f3b23", "dccb684d47c480cc1317dcaa451234c0" },
			{ "cbf85a6645469e5df882fe840776b6aa404c31ea8340896c3955fcce62d6aa9a", "dccb684d47c480cc1317dcaa451234c0", "9a2c4f07489c14265e33ac031d02b3d8" },
			{ "3ea3c33d7439ab3c478c01907f13cda7da607eedcbdc9d4a676650cd7fd41942", "9a2c4f07489c14265e33ac031d02b3d8", "5c8e622ddbd32ee79c17572e8b3ee61c" },
			{ "312c5b43263c1af8d1e35c0f24d1004386ee1cc0100fb3adfb7107e3f4eaff5e", "5c8e622ddbd32ee79c17572e8b3ee61c", "c5d2cb3d5b7ff0e23e308967ee074825" },
			{ "2b09ba39b834062b9e93f48373b8dd018dedf1e5ba1b8af831ebbacbc92a2643", "1f9b9b213f1884fa98b62dd6639fd33b", "89649bd0115f30bd878567610223a59d" },
			{ "58ac71619fdc3ac73a17f285319e1cd492766ac485030e02a95d971daab5f578", "aecd334ef8fb0c51b6896ae065d8be28", "1f9b9b213f1884fa98b62dd6639fd33b" },
			{ "f6e3cca2cd628c10625c62cf08b385743cbb598a7df802531fd4fdfdcf6d4b50", "f1938dd245c055e9c380336ff8450d9d", "aecd334ef8fb0c51b6896ae065d8be28" },
			{ "a9ae3bd7d454f19d69289875ff009d16cd28d458383857badc54ce92372846cd", "42d5c4a13b748800ebfe0f67781dcff1", "f1938dd245c055e9c380336ff8450d9d" },
			{ "e3bc6232865f0476149be7162ef8fc4b8ffd10f9034cdfba37aac1f54f35893c", "469968a00226f0aae7acfda02b2ce0ae", "42d5c4a13b748800ebfe0f67781dcff1" },
			{ "451fbfba7f09b625540822fac8c8e30bc9647859016a2f10d0063c5564196992", "eab144d6f80ccfd2fae95d16784718ac", "469968a00226f0aae7acfda02b2ce0ae" },
			{ "f53e21e46a0ec97c980d49d6f4b81ec423d53c8ff966e0c22aef61431c5e713e", "e04b91c3f084d733d3d0c1c7c152695a", "eab144d6f80ccfd2fae95d16784718ac" },
			{ "5fb3a2cbdbe6971fcd345961bdcba5f6c39ead4c09e237f1f93fa084dd0c1864", "e47db9b8c7fcc9459fa0a7fc84047b5b", "e04b91c3f084d733d3d0c1c7c152695a" },
			{ "6b145436c5ae0dcb2477f94f5bdb037927e314f4ce1efeb4669f07785908633f", "f617d26b55da999d65b2d236358b2e60", "e47db9b8c7fcc9459fa0a7fc84047b5b" },
			{ "126752da3d7f9a0ffc1a46ccaa8a3925d1f4c69f9bc46729032dd54e6c834d5f", "066ffea799ad5f09d03cb868deb1591e", "f617d26b55da999d65b2d236358b2e60" },
			{ "aa3daafbcd647f0a9eaa027146ab7aded79b383802693820d3116d26b2321441", "6d28dbb55fb10c1c3226609997a4fb38", "066ffea799ad5f09d03cb868deb1591e" },
			{ "fed7b5c1f84d3dc185222da3fffac7a1bab3e38d5dd8343ce1370dbf2596ef79", "9dd78987562c9c099d67ccc927fcec13", "6d28dbb55fb10c1c3226609997a4fb38" },
			{ "c36db4af0bee11dad6ea59679cfb988c27646a0a0bf4a8357c50c176026a036a", "b5cc780ec75b8d8c5128c4d26fa520ea", "9dd78987562c9c099d67ccc927fcec13" },
			{ "7b5aebe66f96867504ee9efd2bd56bda92a81204ccaf25b92d7805a46dcf2380", "c715bd2f23b98c66bf56e5680c6037f7", "b5cc780ec75b8d8c5128c4d26fa520ea" },
			{ "81a73bfcacc994f47aeec0d655e0485055bdaf2bef16a9df922ee0cc61af1477", "e00984f9d93551e95ae85cfe4a829b82", "c715bd2f23b98c66bf56e5680c6037f7" },
			{ "2678e4ccda2eb6ad7f0e2843020da2d8b5b42bd23623f836c8c6bc322b2d8ff5", "bfa487fcb2331afe3ad82e59064b4aae", "e00984f9d93551e95ae85cfe4a829b82" },
			{ "03565af5c267a62e0d2500ead06bc7d90a10ac2e8410e2c8f21e926b2d66c55b", "4c278ca418fd1c5718883da0ba4a066f", "bfa487fcb2331afe3ad82e59064b4aae" },
			{ "ec719a4ed851c36f806fbc78183026cc4637208a9cedfe9fea96afcb972cc334", "ceed1c8b96d84951cadf809d3ea1b397", "4c278ca418fd1c5718883da0ba4a066f" },
			{ "f138ad912201beecd15ebe2049f980e588da3c010a35b7ce20492f56a98d70a3", "f88aed84bca42a06647d6504a5c48379", "ceed1c8b96d84951cadf809d3ea1b397" },
			{ "8da22644e1267212742200dbf6b0ba947050d185b6919dc844344a520c49f3da", "c451b75375d0fec1cc1825eefbcc1372", "f88aed84bca42a06647d6504a5c48379" },
			{ "5104213aabaccafe44129632f67cf2dab40166d6c3416309882c6fbcf785e0a8", "9a54dfd56bff41ea17bcf08d3761e95b", "c451b75375d0fec1cc1825eefbcc1372" },
			{ "4c7c62be3f5a9d09934940284565e9c92e55b903a8be22e39f909f31c0e409f3", "76e074c587f112f3ffd17b390fa8cffc", "9a54dfd56bff41ea17bcf08d3761e95b" },
			{ "9871765871b1dae363fca3961224414a58b5cdc62f4f30106041e408cf4cc60f", "5b14fd4c40a127733c20cd84648ec287", "76e074c587f112f3ffd17b390fa8cffc" },
			{ "58e044758e5e4f300eb08f8b6ccd967203a1308a6fee17635c61298cabc20488", "01c234e2392f42831b1172352f322286", "5b14fd4c40a127733c20cd84648ec287" },
			{ "ffbb7104e110fdfe689f68ffa48b79590263046856c155e047705bb984f0260e", "ab142165cb093648938660fd800d8896", "01c234e2392f42831b1172352f322286" },
			{ "91aefec30a79b2524b19a611c0c4952da977250d9dc863a8d4f63b4404fdae98", "5c28ad9b5721edadc30e09b4541fd724", "ab142165cb093648938660fd800d8896" },
			{ "00d0f69c36671a9efeb7b8637e22af38f55f8896cae98e0517f832f050e279bc", "5a9ad24fe456f4f766e839a8101373c3", "5c28ad9b5721edadc30e09b4541fd724" },
			{ "b22571c62ee4e0b4ebd7d776703c1345afc55ad92ebf7af271100b5840f10a7f", "a30e9a7730628b560c71967bdb35efce", "5a9ad24fe456f4f766e839a8101373c3" },
			{ "1431d01006510179dd13e2a83e1b8b7b0ccbc0ae1eddf1a47d619d239bc4e5b1", "7118cc8a057df7ea9b7fb096decf05d9", "a30e9a7730628b560c71967bdb35efce" },
			{ "06104531e2d000fff0cc8336b62e9bb87dd30c241ba0064ee61e2db5450be068", "c550ff93a36fbb8e9a505682e07fb981", "7118cc8a057df7ea9b7fb096decf05d9" },
			{ "8e5ebd5c9bf858f3379bfb88c7c75319b883f3b7b8cfbdc07c4e7b37a57459e9", "1375e7b4164adc861bbb77c50aba445c", "c550ff93a36fbb8e9a505682e07fb981" },
			{ "ccc9445c3aab1c03ec8027593d370664abf61403ae85614667f50cf2afce1db5", "5bc8b411c0fae92bad54b4e5c3a8826e", "1375e7b4164adc861bbb77c50aba445c" },
			{ "b804994228d618a9b7a3e0c3d8d2d373f03ea0126e7f886dcaa1b8176c669fdb", "36c9de7c01298ff06c5e3ecf12507cd6", "5bc8b411c0fae92bad54b4e5c3a8826e" },
			{ "8a784f32cdd7e2ed19e85faf04de2015c6f77e6e6f56079da6ff86d87e36e30d", "35d814c0f6c07ae72bbe7cd1751d7527", "36c9de7c01298ff06c5e3ecf12507cd6" },
			{ "71e3cc0f60a2eafd83c3a455eefc4c9cf32f6aae99967d7a8d41fa090b2b962a", "73c39ce5ea1b5eab18c9c15221e258d1", "35d814c0f6c07ae72bbe7cd1751d7527" },
			{ "71bef3d2beffa17d37c6c3b28ef2928280ecf64b738d23d195883b5b2ac9cefb", "ac67508a8bcab69f96ead29121718cd7", "73c39ce5ea1b5eab18c9c15221e258d1" },
			{ "9aa1ffa9e175f03a6bcc71fc2aea3a522c8ba6c1f847954e0362e9ca0bb8422c", "49d2cac2ca39a3b66302e5f3650e71d7", "ac67508a8bcab69f96ead29121718cd7" },
			{ "4e480b73ab5c659538354b2474c9367b65596c03327e36f860600c396eb633fb", "e54638674ea0b3e837772a2866723753", "49d2cac2ca39a3b66302e5f3650e71d7" },
			{ "aeecd55c38bef76367a21469d08721d2801f54647cde85105717261108c404a8", "074d933018e9e0558b416e39548994ff", "e54638674ea0b3e837772a2866723753" },
			{ "e36c6a30f734a679ab7af2f9ff767cfe8752c75464376545dc5648285c4d9057", "04825a9ba20646acadb2cbdbd23530c0", "074d933018e9e0558b416e39548994ff" },
			{ "124806708fc5fc73a1e50c61e5a76b8783d09dcfc63123e971e483f38e78a097", "25b98d2837580f9f3da8f2593f7b1d72", "04825a9ba20646acadb2cbdbd23530c0" },
			{ "560afb5529a8b433738348a296408cada66910e7f1692c764c4c71aab103bde5", "048b220c70d875cad6901e40f45c5b03", "25b98d2837580f9f3da8f2593f7b1d72" },
			{ "8b7c61dd007d9d0d1937cfc69b63c114a2e232eb81b159bc9adc6fea455fe6e6", "34e2a33412c0e682b29b26d5409d015f", "048b220c70d875cad6901e40f45c5b03" },
			{ "30560ca0a31d8145228d34fbf8415683960091df9371bf3e2847493f05c2e7b9", "942c057914f3b88fa0a40bb41d02b051", "34e2a33412c0e682b29b26d5409d015f" },
			{ "20c7e6e45c08ba0756c379b6cd4d6886022c94a6878207b188e3428b18c057e8", "6f4e5c40e4135abfdf294b4d8c9304b3", "942c057914f3b88fa0a40bb41d02b051" },
			{ "2a6c928bf8c2b32ce82ed0be1bc954b96d62c8e663915d0e57ca09c69453535b", "cac032ddc2b27443d70d0f2dfb1a1d87", "6f4e5c40e4135abfdf294b4d8c9304b3" },
			{ "74994d46526b2e9d5470a0a687f726f2a7a2fa3ba123294d80c706eb6f494edc", "6aec33b2c1ae39237b13e86555722d1a", "cac032ddc2b27443d70d0f2dfb1a1d87" },
			{ "5aaec28d079eef5f1c42df377c7e80accd4ec989608d106efbd4ee8e3a3b63c6", "2df34173bbff331488b23687c755f617", "6aec33b2c1ae39237b13e86555722d1a" },
			{ "9bfa3c08b8f8b8758f0a1f4b0ef86be4e0bd88fadb72237a7366d809fd6e95d1", "07649dd5c1c7b38810cbecbad9c037ed", "2df34173bbff331488b23687c755f617" },
			{ "188e3a027b75fede978c76e70a853f95e7d9152f1ab590f263ad34b324aea23c", "ef3d9092321649d9bbb88f6448e3847b", "07649dd5c1c7b38810cbecbad9c037ed" },
			{ "aae2c6363be3dbb71c21f3a2d9f123f108e485bd28a3d92bd815bbd76c4d2647", "1e5058b3e986d4aad80a11f909af1cca", "ef3d9092321649d9bbb88f6448e3847b" },
			{ "85a51c6d62ba97564ac955a5409c362916b4dd0ec1250d81001faa2e65e23a8d", "9bf81052596b786b50b3f3506c009169", "1e5058b3e986d4aad80a11f909af1cca" },
			{ "0d09600f108a86cfd58f3c211a40c38d8d4ccd5c984e75ea50ac597e09e2abe4", "81bdaf86cf4255e5826294a51d2f9992", "9bf81052596b786b50b3f3506c009169" },
			{ "14ab478866e1d160311aa8820df354fb0cf162da570c200fd2cecddb14cd3276", "1fdc5fee9eb640ff547a98d9e1601b52", "81bdaf86cf4255e5826294a51d2f9992" },
			{ "39f18405f0b1da647f99479e87d4d153132d3d34c9ba60f086b45502f5ad2924", "936609876cd6eebf31db5a6af7839844", "1fdc5fee9eb640ff547a98d9e1601b52" },
			{ "85a900bc1563b16e01481e45d1125734804b34b3a56c8e4fb76f0f68022eb160", "9ea8b6942287f470e9866a48f4dc661a", "936609876cd6eebf31db5a6af7839844" },
			{ "c43d677fc0a1becbb0214221b866f0be1ee3822787eb7a3f5ee96520f6f2d77a", "1dfc2cf82bfaa2732eb525916ef9ef13", "9ea8b6942287f470e9866a48f4dc661a" },
			{ "584f10659c2d3c41f6a1926de4595eb2031faedfac11d84c705c40b1980b3869", "a5d8d1d60281a152b9b76142e3e5a512", "1dfc2cf82bfaa2732eb525916ef9ef13" },
			{ "8eba336927e18b13eb3858a89709768ba6c77f09ae90791ec9eb21f37bee9d7b", "8ac0278f2ab100a5964173d76934d732", "a5d8d1d60281a152b9b76142e3e5a512" },
			{ "13f1558dcae09062bfdccdd0037e5c432c075886842179bb5faa522412da4a49", "30233b70c4d6fd31b23cc77bc73559ea", "8ac0278f2ab100a5964173d76934d732" },
			{ "d736669842aa88f43f1de421d38f391f1c2463f640f7848aed96955fd5ef13a3", "766630dfd41e4146a827f55cba745b6a", "30233b70c4d6fd31b23cc77bc73559ea" },
			{ "0ca8353df90e73606588a11675bd37a76a42532994e9c5cc45b160036f9b48c9", "177f22d98f009c214a44b08cf40b790d", "766630dfd41e4146a827f55cba745b6a" },
			{ "136ec3a1a65ee7bb0fd09af7364b66607d3d71f01be959ed0ff5d08f9b9031c4", "e9c916a21dec9662bdd91a0f865f8b47", "177f22d98f009c214a44b08cf40b790d" },
			{ "ba5c2bd413daca7866de11c11db2926694f467520605cf8fb22cca801dcfba83", "92b3ab3e28f6c606508706e0c3d76d28", "e9c916a21dec9662bdd91a0f865f8b47" },
			{ "0fab4768fa7f104a3579f534508182920647cc6c2ef30989e2abcc60de18d7ab", "3652053dcba591f101944fb1a5eb2b99", "92b3ab3e28f6c606508706e0c3d76d28" },
			{ "e766aa23c075deaa6fae68384f5b205e3015c951e5569878e33f83d17bf3fc32", "242260d2387c6f1a415834b34e300a7e", "3652053dcba591f101944fb1a5eb2b99" },
			{ "9f1941442336bfffa1d6b27dce7845971437a983dd2af762a267b76235c3f64c", "a2a58ed050231b0d15f8cd99f5e171fb", "242260d2387c6f1a415834b34e300a7e" },
			{ "429a3cdc98124fafe33156a66b4e8c2ab69227538d09ec6fb79f7afbc02287b7", "ec6631b17e5c97193fd9651b64feffe7", "a2a58ed050231b0d15f8cd99f5e171fb" },
			{ "92764a89e0de63823b9d9a5a001868035af416e2f3557b7688461fe0a4dc7850", "1eef8afa419a9341085a2d611185e42c", "ec6631b17e5c97193fd9651b64feffe7" },
			{ "0c0bf13d79d1297e137e6f5995cb0a2c441b9c18b2cfe837801c3281b5599c7c", "d4eb4e591cf856b816e2207264e426e4", "1eef8afa419a9341085a2d611185e42c" },
			{ "de31f995867bb45813d369349cb7434290f0d241ae37be8f96fe12f3d1bdba98", "401e14b126348e1aedd13662a19c23fb", "d4eb4e591cf856b816e2207264e426e4" },
			{ "3b1a02b5600444db09118e99765fa61dd0eec6f0880330957b2f249170219963", "38bfea7e1eece741eb6a9601605129dd", "401e14b126348e1aedd13662a19c23fb" },
			{ "1fbcf4ffb36f76b88323f745b3152947e8512c8e96efd7d49045b2901070b0be", "71cd66b3b3553b6aa632c338a307e11a", "38bfea7e1eece741eb6a9601605129dd" },
			{ "b963bfe1c513f0dba7e24b5cfa1535a3999c4a3d25baecbe367771a8b37751a4", "34c2972d43e59eb0c44b7b69a540e7de", "71cd66b3b3553b6aa632c338a307e11a" },
			{ "23458e2a4332ce00e7967a438cd86448ad5edd10665f720ef23c0ac11637b67a", "8988b878d0c6a84b6262351e033fb1b7", "34c2972d43e59eb0c44b7b69a540e7de" },
			{ "e552b8a0f474a8da502e749e3f0639b124d66568b699da45905e3fdf150807cd", "010e24d01c37edca127484c8d8137772", "8988b878d0c6a84b6262351e033fb1b7" },
			{ "65a0645dd971bdb3b0d3a079ef96f3a925d841b8aaae378f822abb17cd1b70bf", "6c7b2e83630a8c8b6219839297d7c647", "010e24d01c37edca127484c8d8137772" },
			{ "9867d55ae9f56a03c435f4b25f2eee2c49a36f3bc9a4bb04e03338855accb6f8", "ccba81435ecf162f70d30b3163cb3858", "6c7b2e83630a8c8b6219839297d7c647" },
			{ "e4a3addb683bbe85ed84b3ce044bcc538519ee78976bad2b90e033b439078ea0", "a0575bc9b8931f34cd6072e2015e7331", "ccba81435ecf162f70d30b3163cb3858" },
			{ "6f543088bf24577d55a685e3637d85e8254eb5b12ff8b21f5d8041563859fd91", "a44046a87574e6d3e6c6446fb3c324bc", "a0575bc9b8931f34cd6072e2015e7331" },
			{ "5fff671ca24c3b241ddb92761f02dfe3810ef3195a8c54ccbb4605398b9ad92d", "ef6abb8ae007b730be45aa85e39f362d", "a44046a87574e6d3e6c6446fb3c324bc" },
			{ "db897d48a317f66412a8c52111c3f3b06e644893ba8be3fc0503afbc6805ef00", "e5bd2edfb7e563f6cd86da757c4c370f", "ef6abb8ae007b730be45aa85e39f362d" },
			{ "78aec1da8e8e79007af12c05d800a18d8bd9664c0d6e800ac88575c91449d80f", "33caa267a64fb3cac210271bb8f25369", "e5bd2edfb7e563f6cd86da757c4c370f" },
			{ "2c3e5387a51addb0b7789ef0289109b3b813c42bab2133c00a9552d2acbb8b66", "27bb26990ca5e12d387bd659cb7e7860", "33caa267a64fb3cac210271bb8f25369" },
			{ "85685fe6996d514e5928d7196f41b5d29fa8e2b2a784d2ed32ee848b67c5f306", "d866ace71916e00c1559bba4149380a0", "27bb26990ca5e12d387bd659cb7e7860" },
			{ "82abcc9a00bae55087cf5d87413f095547ce4e55be9232e127b73f2f735673a6", "8a4d6a0af8488a82a0c31332db3e36de", "d866ace71916e00c1559bba4149380a0" },
			{ "81d983ce59413e84d7af4558d2e3941ccd83245f46dab86387742c1da8684578", "05f6ff378b837cd375cddfae79ec2a9d", "8a4d6a0af8488a82a0c31332db3e36de" },
			{ "bbe6d0d13a6b146bb23db6885ba66487c875db68cd59c4b0f2b9f3b3d1846fe5", "b706bdb4019d604cc35faa07197c318d", "05f6ff378b837cd375cddfae79ec2a9d" },
			{ "8092df6673b2b77783f2ba73c22e8d3c7f7366dcccc4a4fc31e659b4c8f85e68", "b26354bc77d3ce16909ca505d2381e4c", "b706bdb4019d604cc35faa07197c318d" },
			{ "3118fc57b93677af8e14c1b5961b76f3cd103260bb176aeaa17afcb11ac04024", "1515ef6a25cf3943eadadc36a029194a", "b26354bc77d3ce16909ca505d2381e4c" },
			{ "c4bde4a0c5f80329be6dd515f5bf6dabd805dd0a9ed853a94ba02087bae9596e", "7b4ad946dfb59c80b1dc5cedb2fa87a0", "1515ef6a25cf3943eadadc36a029194a" },
			{ "80b47f9126b0b1f5ab408bb9ea5f349aa34f044c416dcf29fa7c7c6a0813dece", "fa10a53e14cde5ef1e9a8a5692847aa5", "7b4ad946dfb59c80b1dc5cedb2fa87a0" },
			{ "86a43939c8cd243f7ff2a9658524373b595fa17255a02ac6e4e6f63c9a97a46b", "4c9bed07308abcd0b09265b5efbe301c", "fa10a53e14cde5ef1e9a8a5692847aa5" },
			{ "74a8f02a15285153e0ae160fbddcf6bf15c44c75652a96165474938975299477", "75b7496c42809a3a592acbd8e069269e", "4c9bed07308abcd0b09265b5efbe301c" },
			{ "293b40a2f70cc2c4262554663fb6f0986073051927aa0c2c0d5e58519540b2e9", "2dbc01185af7e084d90578468b6b10ef", "75b7496c42809a3a592acbd8e069269e" },
			{ "b04d4f4eb708eef86c5b6b8e08a665084dcf04017d5deca8d45b20171e2ba206", "c95b3a9f689c9dce4995c24f72dd5162", "2dbc01185af7e084d90578468b6b10ef" },
			{ "7cb9f7211815e0cb48b71286f84f80a184943e9e15c171669dcee2586cf6f364", "616e55c3bf113e2c18cae3c61b7eb7d1", "c95b3a9f689c9dce4995c24f72dd5162" },
			{ "b3cdb46cf92aa0b96e87212bc650d5e5e5fa6b5daad04f4a8504019e778844b5", "87e8b80767ebbdbad75cb94f4cb54f3b", "616e55c3bf113e2c18cae3c61b7eb7d1" },
			{ "512c2a3821eb53af613141c71e1076656212d35acd3bf2f05258b8d13b3d0b8e", "c83e20e18f2b1457788954b49fd84307", "87e8b80767ebbdbad75cb94f4cb54f3b" },
			{ "9977c985745bc33954a2ce898bc8febdaa2cf3bb4210e6a72ad1ec65a4e54889", "e3d3868f578caf34e36445bf14cefc68", "c83e20e18f2b1457788954b49fd84307" },
		};
		for (auto& Test : MonteCarloCases)
		{
			RunTest(Test.Key, Test.PlainText, Test.CipherText, 1000);
		}
	}

	// 6.3 Multi-block Message Test, from ECBMMT256.rsp
	SECTION("Multi-block Message")
	{
		static const AesTestCase MultiBlockCases[] =
		{
			{
				"cc22da787f375711c76302bef0979d8eddf842829c2b99ef3dd04e23e54cc24b",
				"ccc62c6b0a09a671d64456818db29a4d",
				"df8634ca02b13a125b786e1dce90658b",
			},
			{
				"7a52e4d342aa07255a7e7c34266cf7302abe2d4dd7ec4468a46187ee61825ffa",
				"7e771c6ee4b26db89050e982ba7e9803c8da34606434dd85d2910e538076d001",
				"a91d8b2ddf37520bc469470ad0dd6394923143ce55386beb1f9c4bd51584658e",
			},
			{
				"605c4139c961b496ca5148f1bdb1bb1901f2101943a0ec10fcdc403d3b0c285a",
				"68c9885ba2be03181f65f1e04e83d6ba6880467550bcf099be26dc9d9c0af15ab02abac07c116ac862a41da90cfa604f",
				"a7603d29bbba4c77208bf2f3df9f5ec85204adce012299f2cce7b326ce78f5cf8040343dd291e8cf9f3645726368dc20",
			},
			{
				"f984b0f534fc0ae2c0a8593e16ab8365f25fcc9c5947f9a2db45b588160d35c3",
				"351fee099122e371c4830f409c6c4411186d22176f7138b054f16b3c79679c2f520685651ba8e4b61c08dccb2c31982f743631a97524d2ca4d351ac23546c178",
				"8b9c9e692c16e7059818e285e85d8fa5433dee2aff9fec61d6a0a781e24b24f64902fbd18cef7461ad7760cfb2442fb74ffd9be108a386545f2a216430ef16fb",
			},
			{
				"ba42b760bb5a5de21acb9aba214c9783cd71ea841ada018580abc4e1be3b76dd",
				"4b4b12d6ee6fc0bf987eaafe2634aad464781ff4c83d3f8a61a6af7c0a6d51f0e3855d0e02feb307652a6f562bfebe4604baf1b4e7cdd01603f231bcf7a0c95645a141b704008cd8d62979201a4c84e2",
				"fa18d25e37ea0ce94f0949efc0edecc6a40fada8f007fd8e760afed0a83ebb350c82b03baaa6ee19f791bb9bd1b44d27a76fc6eb0e1c0017d68776ed69a541851a732e46ef328def064baf6a0a755588",
			},
			{
				"1d29079cc34ab5a3bc713f416a129f9d26ada15fca458cc2731404ea857d2f79",
				"b36809198132518d815aa37f32f40dd7a952ec8bd63355700837b450b396b33c72125e23482a84a42b916021c3de78156f85c6a78906167feb64afd8b1d935d641c8ce1a89f3849588eeeb9910d40336ca385fc37a5e87bb84ab9ccbb05b3a28",
				"7ae803b14914d156439f580c02c592bf9a41b7b80c20168129a33fae2290403f01a6fabe4fea7ac770ffbc6c421f8e013b9e831674ef17eb27d446dded3baf50686b809c18b6bbd588cd3c7423b64987e9164b7e1e66987dc9319790ae27b3e8",
			},
			{
				"2e39c585ce4900d323ce29713bebe73a1be08a0cb22e9f1310fcc14ad4b9b23e",
				"e199899e1e12cffcb28909aec51b36c2f96fab49ef32b9650cc38aa37d2f4c8b785f9176c590f6a07e04037e13f7535290d5f5fc23aa1113d9dacf34a812749ab27ecfefc83dd3622d1285fa9d5c192a8e4857a5b6454473cdb8ff4594f1e906165e08b22effae6b491a55ca6d30ce73",
				"2c4998428e72f6d96e982a316f73bf2a7da81730909b65403489ab92ada6de11882d08742f90f0f109d3420b00b8abe6873f4fdd1474923da2c5bdea2e4523ffca2132015ecf7c9cac9de2f956b112c4ba8e4c8e4b354a3df6e4652d6a77ae982a24d15cff71b979538d49994fd38761",
			},
			{
				"85405c4f0ebbe8f29228f02f1ff184e2f5e7857e8933c2a1d08f61ecb9b68111",
				"0f5321db6fd9d816d88e28183a739d90974e76095caa9f12f11fe49c8fd35fa3524132118f397cdf6728857d9c9a3c74a4fde44afca80aa5bf1cbab47789f2cb3394574dda5727cfbea96f7a74a07eb1e45599f49ce7f0056ac3d14929221c70dbd3f759f83ac22f06994ed96a8e49917eddfdd2e3703b78199c91234ca6c3dc",
				"2ac6de212da0434bea9cdd7332637307131d31e8c4b0c1fd02298e249bfa9f64f34aae45faadf79d971ae82b033d903f6b18adec171ec83ccd147b44d05dec5cedb574534e8901385534c3f1dae4aebee0eb2164975e8b4e85a7b5a766ffd8247885c2b6429eb659b92c8d953af92b54517933566654d804665112bec17ff3a4",
			},
			{
				"f157285db00e64c2791668a54493966e3039a19426605056b95b7eac5106667d",
				"3637f71f60a430322980349ad414fcfdc14f87e9915d210e8b7be5aa3e09814468e0399d17e72fe40ee1e1296a89f31486e12fd71bc7ca61acc9e8d4213a633ab285c87406c2a3729c87fdaa6b0122c2c543a89081ddac4559b15fe59102c2fbdacead8a755e16469b1b90041da31270481cfafe0bc951235cda51c4d78924efa862efc9ace20fc7d344a321c9984a84",
				"ca0a683e759c1312928fe01198f625bba044bb9003e82b9279b6812feefe54e30ccc0ca51fb858eba9ed4667e85c146a424071e7bf603f1f538ead57a3e29c583549c08e279bb078cc51e42eeb3d2443da965192cd0478e8d7ea1343cf90196f520933e3aab6db2dcd9a76534e05483eea2d373deae95d6213b91db2b96aa6adce727cdf7e43bd01978d07e0241cf1c1",
			},
			{
				"44a2b5a7453e49f38261904f21ac797641d1bcd8ddedd293f319449fe63b2948",
				"c91b8a7b9c511784b6a37f73b290516bb9ef1e8df68d89bf49169eac4039650c4307b6260e9c4e93650223440252f5c7d31c26c56209cbd095bf035b9705880a1628832daf9da587a6e77353dbbce189f963235df160c008a753e8ccea1e0732aa469a97659c42e6e31c16a723153e39958abe5b8ad88ff2e89af40622ca0b0d6729a26c1ae04d3b8367b548c4a6335f0e5a9ec914bb6113c05cd0112552bc21",
				"05d51af0e2b61e2c06cb1e843fee3172825e63b5d1ce8183b7e1db6268db5aa726521f46e948028aa443af9ebd8b7c6baf958067ab0d4a8ac530ecbb68cdfc3eb93034a428eb7e8f6a3813cea6189068dfecfa268b7ecd5987f8cb2732c6882bbec8f716bac254d72269230aec5dc7f5a6b866fd305242552d400f5b0404f19cbfe7291fab690ecfe6018c4309fc639d1b65fcb65e643edb0ad1f09cfe9cee4a",
			},
			{
				"a81fd6ca56683d0f5445659dde4d995dc65f4bce208963053e28d7f2df517ce4",
				"8b2b1b22f733ac09d1196d6be6a87a72",
				"4154c0be71072945d8156f5f046d198d",
			},
			{
				"bc1415aa119c29a9a27c0ea9d19ed50ace86c488b8e39d6a056424fb23cddb3e",
				"a8fab53790af3519cf21978e3cf03ba8e52b902be23311bf17f1ad2c5ff37c16",
				"0a5f3278d7d96632e0508391e813f06b35d8d754ddf5867240d3168dd69f4a66",
			},
			{
				"0d0ec1b61ebc5177c4513ef1d7d5bb97d06abaa2d33710a8eda6d3709acf0705",
				"f17733de8f763110ef4b305594936ca2bb75119ad65261be32ba919a2c3ef8b8f1c42f62b8474362e53ee7cc6c82a647",
				"c1c83fa3cd3d52524876e715bc28efe7c7c4256a139e9d2c874ea029bf56b792ba0606cdd39ddbbdf3b1874304d16d05",
			},
			{
				"ae67999eb240a9e5b6e3f0bd6b504530000be51343b8bc3b0ae8e0eed1335f98",
				"959b7d696e2267f3d465f6f77ff2453296cfa54843363981484853e67cc21d340b803d6d653213d1037b81d849c5acc361771a5a072c9c29d6bc5096c38c9c86",
				"254c015626baa3ed2d7f05f5d0a98c8cc2f29dd7a4410ea41d74db4e2c5d1be1bd0a327f7b4a4703e66cf9e3a54a4d5b1d87ebc842148b52adf9aacdf05281e1",
			},
			{
				"dde079371133ebd68df061b56f0efd3a14c137ced35a30e0eb68422cb924dc3d",
				"5889ad2c09a6307611e6115a78c13566de05b5892f7850fb917f83898e07486cc9ce746e891db102a0f570d7ad3c2804f40cdfe23bcc8b2c8a3bfba48632892d3df3bb7bbd1029b915cab2faf281c4e6",
				"ea1fd2f064548906ad10ce1240758868ed9fb32921dabe18681232a8308b955ad0d28e45c9f34af64bec1d7bfb626dbbf393cc090a8c64f8569b9870f008e801f7001578d8d286b0cc5eebaba2c920e1",
			},
			{
				"d68a345159178b9bd2e3bd7a13c9512ee9b397944eff81a8df28b44890a2df3b",
				"a56e000be19eced20479cbe8964fdb366c437ca3b5fd9d04c339c1a51bdb0aad46df1b78ef05b02e492464136155d76dd32c3b4aa4198c3926cb75d2578434ab4fff0894508b6f60ff39686a0fd151d1bffba7a786b1bc02acd2d23b56e45749",
				"59722586a56ed6c8207d6a0a9f7278588f5203782fb64e6ffd71f1486d732d10e1ed7d25d6b66db15beeaf71c8e16cbbac2dd8bf0728090dba7c09573b142a788d3c99b670033644d5e4a30b94a69bab618186bcfd824a59a93163dfe07036f2",
			},
			{
				"5a20ff7da47c7e853becca0ca5f32bc80e17de973337146f7e1f3c93725a850d",
				"34467d07c2e49c44e9003da836786169e7c0fb54f1e2f178387c2f759d50809ce84d6705fe6350907d5e945e262af378753163f5569be9c09c874d254dde2545898d4cbd2eaa1ade9c8a02f8d76d418565d02075e082999bbf6f2ba985f65f17c73cc0cc29ac04824db98b3cc2b77126",
				"bed9f2db31ebbfac57f3745e0dacf71087cb35c94326b9ba3b1b7dc35f30003942f798e714350a52d0425842b6b3da7dfc3790d6c1bfb76642e29a1a507fab6e02a4d277a7ae0527feb219cdaa1cd8dbe09653ec632c7a05b2ade91b7f5405d664a74895d2a0d9b24a0b6007cf2f18b1",
			},
			{
				"082d33edd0a1ad3de596762d711bae6f3188a12c7b6ced987fc7e8c9cd7a3cc9",
				"51c527c0985369234a599f476731e8b51ac7a44bfbe37ec71a641ad71ad464f9e45467d82a0d101f67043b87e2da34de183459295000cea9e0f0ccdd82270fc70b880fa80104e0b78b7a5b1620dee83da84702223b277a0966b10ddf44ef06b98e48788092c763895d95f13fd389fff5706e70876d5af8c197cdebb38a4d2ba6",
				"496553f24d47264ef74a584b893ccd4ea1cfc0b1044aac15997e6dfab9fca28da68557c058c2e1a0803508e4c7706cc1568521069cbd9d644b8dd40da4c5a182f50b68a56088fa16dde1b8bc8269fca20395b40575b050ab57cfdf76d208743840dfcfc9ec0139571835b0fe458f4498aee6c1d716898e91c32ed55acb2be458",
			},
			{
				"3e1934554952b87a07d6371721f4d78a0e9adadc42be7347a2fcd8f53c81990b",
				"5f967e10da68887567b514293149697212888efc9fd90604459f8a9c4fa1f4117f0214fa675b680571bd980cf941bfccd4826e1fe1420a0ba59554341b507e760d2d85c7219b3f5e261eef2d201bc134d2ec32d8b97157e3de91f1c312b26ff193e637bf78012850f23d0536715e51a68be730c55375774642345e0daaa4c40185985cab38df09dac9a688e6b597d327",
				"2b3c036be68fba7a625dc72a0e68b6677326f8751714e0e1a14df2b73d5b5df5424dc640130947d643e4b8c4fc702d59a195bfdab2e49077f027097ec23d66cfd82575900e589a21935c17524a968ad47060b7ee805b8871530652ee8c905a2633eb98c2a022bb459bec6d3ec181d7c22aac68194822d2b81212566f62fd42e49cc8b8409110b028a3bd40ef29da1c0a",
			},
			{
				"c4a71e055a7254dda360693fe1be49f10faa6731c36dbaa6590b05974e185c5b",
				"31fd5a307e279b2f34581e2c432379df8eccbaf79532938916711cd377540b9045373e47f2214b8f876040af733f6c9d8f03a7c58f8714d2fbb4c14af59c75b483adc718946ee907a18286cc4efd206789064b6f1b195f0d0d234468e4f00e6f1cad5cd3b9c0a643b3c0dd09280ff2e2a5929183409384dd72dc94e39687ea2b623d5d776700bd8b36e6130ffde966f134c4b1f35f29c5cc4a03297e1ccc9539",
				"2c487fa96f4090c56aa1b5be81918a934c9492878fb0cd686dcf8d17d86485454c51237bbd09205dcef1552f430dd098b9d827a694730c133a0222c77f540f9d5fc2d36af359583c9e3b49df884228a64de79b67f66207c8281360b99b214042ce61367ff97960e944453cd63679bb44708897d29bc5e70f9fc8f1f715143fbb00f7f5c1b7b161ec26d8d41d36fab0fa8a85c3ee6ce4d37007eb7a89d6753590",
			},
		};
		for (auto& Test : MultiBlockCases)
		{
			RunTest(Test.Key, Test.PlainText, Test.CipherText);
		}
	}
}

#endif //WITH_TESTS
