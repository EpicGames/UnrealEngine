// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_TESTS

#include "Containers/StringView.h"
#include "Misc/AsciiSet.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(TAsciiSetTest, "System::Core::Misc::AsciiSet", "[ApplicationContextMask][SmokeFilter]")
{
	constexpr FAsciiSet Whitespaces(" \v\f\t\r\n");
	CHECK_MESSAGE(TEXT("Contains"), Whitespaces.Contains(' '));
	CHECK_MESSAGE(TEXT("Contains"), Whitespaces.Contains('\n'));
	CHECK_FALSE_MESSAGE(TEXT("Contains"), Whitespaces.Contains('a'));
	CHECK_FALSE_MESSAGE(TEXT("Contains no extended ASCII"), Whitespaces.Contains('\x80'));
	CHECK_FALSE_MESSAGE(TEXT("Contains no extended ASCII"), Whitespaces.Contains('\xA0'));
	CHECK_FALSE_MESSAGE(TEXT("Contains no extended ASCII"), Whitespaces.Contains('\xFF'));
	
	constexpr FAsciiSet Aa("Aa");
	uint32 ANum = 0;
	for (char32_t C = 0; C < 512; ++C)
	{
		ANum += Aa.Contains(C);
	}
	CHECK_EQUALS(TEXT("Contains no wide"), ANum, 2u);

	constexpr FAsciiSet NonWhitespaces = ~Whitespaces;
	uint32 WhitespaceNum = 0;
	for (uint8 Char = 0; Char < 128; ++Char)
	{
		WhitespaceNum += !!Whitespaces.Test(Char);
		CHECK_EQUALS(TEXT("Inverse"), !!Whitespaces.Test(Char), !NonWhitespaces.Test(Char));
	}
	CHECK_EQUALS(TEXT("Num"), WhitespaceNum, 6);

	CHECK_EQUALS(TEXT("Skip"), FAsciiSet::Skip(TEXT("  \t\tHello world!"), Whitespaces), TEXT("Hello world!"));
	CHECK_EQUALS(TEXT("Skip"), FAsciiSet::Skip(TEXT("Hello world!"), Whitespaces), TEXT("Hello world!"));
	CHECK_EQUALS(TEXT("Skip to extended ASCII"), FAsciiSet::Skip(" " "\xA0" " abc", Whitespaces), ANSITEXTVIEW("\xA0" " abc"));
	CHECK_EQUALS(TEXT("Skip to wide"), FAsciiSet::Skip(TEXT(" 变 abc"), Whitespaces), TEXT("变 abc"));
	CHECK_EQUALS(TEXT("AdvanceToFirst"),	*FAsciiSet::FindFirstOrEnd("NonWhitespace\t \nNonWhitespace", Whitespaces), '\t');
	CHECK_EQUALS(TEXT("AdvanceToLast"),	*FAsciiSet::FindLastOrEnd("NonWhitespace\t \nNonWhitespace", Whitespaces), '\n');
	CHECK_EQUALS(TEXT("AdvanceToLast"),	*FAsciiSet::FindLastOrEnd("NonWhitespace\t NonWhitespace\n", Whitespaces), '\n');
	CHECK_EQUALS(TEXT("AdvanceToFirst"),	*FAsciiSet::FindFirstOrEnd("NonWhitespaceNonWhitespace", Whitespaces), '\0');
	CHECK_EQUALS(TEXT("AdvanceToLast"),	*FAsciiSet::FindLastOrEnd("NonWhitespaceNonWhitespace", Whitespaces), '\0');

	constexpr FAsciiSet Lowercase("abcdefghijklmnopqrstuvwxyz");
	CHECK_EQUALS(TEXT("TrimPrefixWithout"),		FAsciiSet::TrimPrefixWithout(ANSITEXTVIEW("ABcdEF"), Lowercase), ANSITEXTVIEW("cdEF"));
	CHECK_EQUALS(TEXT("FindPrefixWithout"),		FAsciiSet::FindPrefixWithout(ANSITEXTVIEW("ABcdEF"), Lowercase), ANSITEXTVIEW("AB"));
	CHECK_EQUALS(TEXT("TrimSuffixWithout"),		FAsciiSet::TrimSuffixWithout(ANSITEXTVIEW("ABcdEF"), Lowercase), ANSITEXTVIEW("ABcd"));
	CHECK_EQUALS(TEXT("FindSuffixWithout"),		FAsciiSet::FindSuffixWithout(ANSITEXTVIEW("ABcdEF"), Lowercase), ANSITEXTVIEW("EF"));
	CHECK_EQUALS(TEXT("TrimPrefixWithout none"),	FAsciiSet::TrimPrefixWithout(ANSITEXTVIEW("same"), Lowercase), ANSITEXTVIEW("same"));
	CHECK_EQUALS(TEXT("FindPrefixWithout none"),	FAsciiSet::FindPrefixWithout(ANSITEXTVIEW("same"), Lowercase), ANSITEXTVIEW(""));
	CHECK_EQUALS(TEXT("TrimSuffixWithout none"),	FAsciiSet::TrimSuffixWithout(ANSITEXTVIEW("same"), Lowercase), ANSITEXTVIEW("same"));
	CHECK_EQUALS(TEXT("FindSuffixWithout none"),	FAsciiSet::FindSuffixWithout(ANSITEXTVIEW("same"), Lowercase), ANSITEXTVIEW(""));
	CHECK_EQUALS(TEXT("TrimPrefixWithout empty"),	FAsciiSet::TrimPrefixWithout(ANSITEXTVIEW(""), Lowercase), ANSITEXTVIEW(""));
	CHECK_EQUALS(TEXT("FindPrefixWithout empty"),	FAsciiSet::FindPrefixWithout(ANSITEXTVIEW(""), Lowercase), ANSITEXTVIEW(""));
	CHECK_EQUALS(TEXT("TrimSuffixWithout empty"),	FAsciiSet::TrimSuffixWithout(ANSITEXTVIEW(""), Lowercase), ANSITEXTVIEW(""));
	CHECK_EQUALS(TEXT("FindSuffixWithout empty"),	FAsciiSet::FindSuffixWithout(ANSITEXTVIEW(""), Lowercase), ANSITEXTVIEW(""));


	auto TestHasFunctions = [](auto MakeString)
	{
		FAsciiSet XmlEscapeChars("&<>\"'");
		CHECK_MESSAGE(TEXT("None"),	FAsciiSet::HasNone(MakeString("No escape chars"), XmlEscapeChars));
		CHECK_FALSE_MESSAGE(TEXT("Any"),	FAsciiSet::HasAny(MakeString("No escape chars"), XmlEscapeChars));
		CHECK_FALSE_MESSAGE(TEXT("Only"), FAsciiSet::HasOnly(MakeString("No escape chars"), XmlEscapeChars));

		CHECK_MESSAGE(TEXT("None"),	FAsciiSet::HasNone(MakeString(""), XmlEscapeChars));
		CHECK_FALSE_MESSAGE(TEXT("Any"),	FAsciiSet::HasAny(MakeString(""), XmlEscapeChars));
		CHECK_MESSAGE(TEXT("Only"),	FAsciiSet::HasOnly(MakeString(""), XmlEscapeChars));

		CHECK_FALSE_MESSAGE(TEXT("None"), FAsciiSet::HasNone(MakeString("&<>\"'"), XmlEscapeChars));
		CHECK_MESSAGE(TEXT("Any"),	FAsciiSet::HasAny(MakeString("&<>\"'"), XmlEscapeChars));
		CHECK_MESSAGE(TEXT("Only"),	FAsciiSet::HasOnly(MakeString("&<>\"'"), XmlEscapeChars));

		CHECK_FALSE_MESSAGE(TEXT("None"), FAsciiSet::HasNone(MakeString("&<>\"' and more"), XmlEscapeChars));
		CHECK_MESSAGE(TEXT("Any"),	FAsciiSet::HasAny(MakeString("&<>\"' and more"), XmlEscapeChars));
		CHECK_FALSE_MESSAGE(TEXT("Only"), FAsciiSet::HasOnly(MakeString("&<>\"' and more"), XmlEscapeChars));
	};
	TestHasFunctions([] (const char* Str) { return Str; });
	TestHasFunctions([] (const char* Str) { return FAnsiStringView(Str); });
	TestHasFunctions([] (const char* Str) { return FString(Str); });


	constexpr FAsciiSet Abc("abc");
	constexpr FAsciiSet Abcd = Abc + 'd';
	CHECK_MESSAGE(TEXT("Add"), Abcd.Contains('a'));
	CHECK_MESSAGE(TEXT("Add"), Abcd.Contains('b'));
	CHECK_MESSAGE(TEXT("Add"), Abcd.Contains('c'));
	CHECK_MESSAGE(TEXT("Add"), Abcd.Contains('d'));
	CHECK_FALSE_MESSAGE(TEXT("Add"), Abcd.Contains('e'));
}

#endif //WITH_TESTS