// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Build.h" 

#if WITH_LOW_LEVEL_TESTS

#include "Misc/AssertionMacros.h"
#include "GenericPlatform/GenericPlatformMisc.h"

#include "TestHarness.h"
#include "TestMacros/Assertions.h"

//since ensure uses a static bool to suppress multiple calls
//these small functions are needed to scope that bool
static void EnsureFailed()
{
	ensure(false);
}

static void EnsureFailAlways()
{
	ensureAlways(false);
}

static int EnsureContinues(int Value)
{
	if (ensure(Value == 1))
	{
		return Value + 1;
	}
	return Value + 2;
}

static int EnsureMultipleFailures(int Value)
{
	ensure(Value == 1);
	ensure(Value == 5);
	ensure(Value == 2);
	return Value + 1;
}

static int EnsureForCheck(int Value)
{
	if (ensure(Value == 1))
	{
		return Value + 1;
	}
	return Value + 2;
}

static int EnsureAlwaysMsg(int Value)
{
	if (ensureAlwaysMsgf(Value == 1, TEXT("expected value of 1")))
	{
		return Value + 1;
	}
	return Value + 2;
}

/**
 * @brief Ensure test case, tests Catch2-style macros that check or require whether at least one ensure was triggered or not.
 * Note that each different ensure expression is counted once, if the same ensure fails twice it's only counted the first time it fails.
 */
TEST_CASE("Core::Misc::TestEnsure", "[Core][Misc][AssertionMacros][Ensure]")
{
#if !UE_BUILD_SHIPPING
	//ignore the debugger as all the breaks make for annoying debugging experience
	TGuardValue<bool> IgnoreDebugger(GIgnoreDebugger, true);
#endif
	{
		REQUIRE_ENSURE(EnsureFailed());
		EnsureFailed(); //ensure already fired should not fire again
	}

	{
		//validate that ensure continue on failure
		//validate that non failure works
		int result = 0;
		result = EnsureContinues(1);
		REQUIRE(result == 2);

		REQUIRE_ENSURE(result = EnsureContinues(0));
		REQUIRE(result == 2);

		//second failure should not cause an error
		result = EnsureContinues(2);
		REQUIRE(result == 4);
	}

	{
		//validate that ensureAlways raises errors
		REQUIRE_ENSURE(EnsureFailAlways());
		REQUIRE_ENSURE(EnsureFailAlways());
	}

	{
		//not much of test of CHECK_ENSURE as its hard to tell if the continue part is working
		int result = 0;
		CHECK_ENSURE(result = EnsureForCheck(0));
		REQUIRE(result == 2);
	}
	
	{
		//multiple failed ensure should continue because its wrapped in a scope
		int result = 0;
		REQUIRE_ENSURE(result = EnsureMultipleFailures(7));
		REQUIRE(result == 8);
	}

	{
		int result = 0;
		REQUIRE_ENSURE_MSG("expected value of 1", result = EnsureAlwaysMsg(3));
		REQUIRE(result == 5);
		CHECK_ENSURE_MSG("expected value of 1", result = EnsureAlwaysMsg(4));
		REQUIRE(result == 6);
	}
}

TEST_CASE("Core::Misc::TestCheck", "[Core][Misc][AssertionMacros][Check]")
{
	REQUIRE_CHECK([]{ check(1 == 2); }());
	REQUIRE_CHECK_MSG("1 == 2", []{ checkf(1 == 2, TEXT("Error Message")); }());
	REQUIRE_CHECK_MSG("Error Message", []{ checkf(1 == 2, TEXT("Error Message")); }());
}

TEST_CASE("Core::Misc::TestCheck", "[Core][Misc][AssertionMacros][CheckSlow]")
{
	REQUIRE_CHECK_SLOW([]{ checkSlow(1 == 2); }());
	REQUIRE_CHECK_SLOW_MSG("1 == 2", []{ checkfSlow(1 == 2, TEXT("Error Message")); }());
	REQUIRE_CHECK_SLOW_MSG("Error Message", []{ checkfSlow(1 == 2, TEXT("Error Message")); }());
}

#endif
