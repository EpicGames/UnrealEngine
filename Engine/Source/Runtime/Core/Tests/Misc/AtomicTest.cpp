// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Atomic.h"
#include "Templates/EnableIf.h"
#include "Templates/UnrealTemplate.h"
#include "Traits/IntType.h"
#include "Containers/UnrealString.h"

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

namespace
{
	// Returns an element type filled with a repeated byte
	template <typename ElementType>
	FORCEINLINE constexpr typename TEnableIf<sizeof(ElementType) != 1, ElementType>::Type GetByteFilledElement(uint8 Byte)
	{
		TUnsignedIntType_T<sizeof(ElementType)> Result = 0;

		for (int32 i = 0; i != sizeof(ElementType); ++i)
		{
			Result <<= 8;
			Result |= (TUnsignedIntType_T<sizeof(ElementType)>)Byte;
		}

		return (const ElementType&)Result;
	}

	template <typename ElementType>
	FORCEINLINE constexpr typename TEnableIf<sizeof(ElementType) == 1, ElementType>::Type GetByteFilledElement(uint8 Byte)
	{
		return (const ElementType&)Byte;
	}

	// Returns an element type filled with random bytes
	template <typename ElementType>
	FORCEINLINE typename TEnableIf<sizeof(ElementType) != 1, ElementType>::Type GetRandomFilledElement()
	{
		TUnsignedIntType_T<sizeof(ElementType)> Result = 0;

		for (int32 i = 0; i != sizeof(ElementType); ++i)
		{
			Result <<= 8;
			Result |= (TUnsignedIntType_T<sizeof(ElementType)>)(uint8)FMath::Rand();
		}

		return (const ElementType&)Result;
	}

	template <typename ElementType>
	FORCEINLINE typename TEnableIf<sizeof(ElementType) == 1, ElementType>::Type GetRandomFilledElement()
	{
		return (const ElementType&)(uint8)FMath::Rand();
	}

	template <typename ElementType>
	struct TAtomicTestWrapper
	{
		static_assert(sizeof(ElementType) == sizeof(TAtomic<ElementType>), "Atomic should be the same size as the underlying type");

		explicit TAtomicTestWrapper(ElementType Init)
		{
			ElementType DebugCode = GetByteFilledElement<ElementType>(0xCD);

			NativeArray[0] = DebugCode;
			NativeArray[1] = Init;
			NativeArray[2] = DebugCode;

			AtomicArray[0] = DebugCode;
			AtomicArray[1] = Init;
			AtomicArray[2] = DebugCode;
		}

		void Check() const
		{
			ElementType DebugCode = GetByteFilledElement<ElementType>(0xCD);

			check(NativeArray[0]        == DebugCode);
			check(NativeArray[2]        == DebugCode);
			check(AtomicArray[0].Load() == DebugCode);
			check(AtomicArray[2].Load() == DebugCode);

			check(NativeArray[1] == AtomicArray[1].Load());
		}

		ElementType& GetNative()
		{
			return NativeArray[1];
		}

		TAtomic<ElementType>& GetAtomic()
		{
			return AtomicArray[1];
		}

	private:
		// Arrays with some run-off, where we're modifying the middle element, and we can check the
		// surrounding elements to see we're not accidentally over-/under-running.
		ElementType          NativeArray[3];
		TAtomic<ElementType> AtomicArray[3];
	};

	template <typename ElementType>
	void RunBasicAtomicTests()
	{
		ElementType Val = GetRandomFilledElement<ElementType>();

		TAtomicTestWrapper<ElementType> Data(Val);
		Data.Check();

		Val = GetRandomFilledElement<ElementType>();
		Data.GetNative() = Val;
		Data.GetAtomic().Store(Val, EMemoryOrder::SequentiallyConsistent);
		Data.Check();

		Val = GetRandomFilledElement<ElementType>();
		Data.GetNative() = Val;
		Data.GetAtomic().Store(Val, EMemoryOrder::Relaxed);
		Data.Check();

		Val = GetRandomFilledElement<ElementType>();
		ElementType Old = Data.GetAtomic().Exchange(Val);
		check(Data.GetNative() == Old);
		Data.GetNative() = Val;
		Data.Check();
	}

	template <typename ElementType>
	void RunArithmeticAtomicTests(ElementType Init)
	{
		typename TRemovePointer<ElementType>::Type Array[100] = {};
		TAtomicTestWrapper<ElementType> Data(Init);
		Data.Check();

		ElementType NativeVal = Data.GetNative() += 4;
		ElementType AtomicVal = Data.GetAtomic() += 4;
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative() += -7;
		AtomicVal = Data.GetAtomic() += -7;
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative() -= 3;
		AtomicVal = Data.GetAtomic() -= 3;
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative() -= -8;
		AtomicVal = Data.GetAtomic() -= -8;
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = --Data.GetNative();
		AtomicVal = --Data.GetAtomic();
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = ++Data.GetNative();
		AtomicVal = ++Data.GetAtomic();
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative()--;
		AtomicVal = Data.GetAtomic()--;
		check(NativeVal == AtomicVal);
		NativeVal = Data.GetNative();
		AtomicVal = Data.GetAtomic();
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative()++;
		AtomicVal = Data.GetAtomic()++;
		check(NativeVal == AtomicVal);
		NativeVal = Data.GetNative();
		AtomicVal = Data.GetAtomic();
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative()--;
		AtomicVal = Data.GetAtomic().DecrementExchange();
		check(NativeVal == AtomicVal);
		NativeVal = Data.GetNative();
		AtomicVal = Data.GetAtomic();
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative()++;
		AtomicVal = Data.GetAtomic().IncrementExchange();
		check(NativeVal == AtomicVal);
		NativeVal = Data.GetNative();
		AtomicVal = Data.GetAtomic();
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative();
		AtomicVal = Data.GetAtomic().AddExchange(47);
		check(NativeVal == AtomicVal);
		NativeVal = Data.GetNative() += 47;
		AtomicVal = Data.GetAtomic();
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative();
		AtomicVal = Data.GetAtomic().AddExchange(-11);
		check(NativeVal == AtomicVal);
		NativeVal = Data.GetNative() += -11;
		AtomicVal = Data.GetAtomic();
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative();
		AtomicVal = Data.GetAtomic().SubExchange(2);
		check(NativeVal == AtomicVal);
		NativeVal = Data.GetNative() -= 2;
		AtomicVal = Data.GetAtomic();
		check(NativeVal == AtomicVal);
		Data.Check();

		NativeVal = Data.GetNative();
		AtomicVal = Data.GetAtomic().SubExchange(-9);
		check(NativeVal == AtomicVal);
		NativeVal = Data.GetNative() -= -9;
		AtomicVal = Data.GetAtomic();
		check(NativeVal == AtomicVal);
		Data.Check();
	}

	template <typename ElementType>
	void RunBitwiseOperationsAtomicTests()
	{
		ElementType NativeVal;
		ElementType AtomicVal;

		// And
		{
			constexpr ElementType Init = GetByteFilledElement<ElementType>(0x30);
			constexpr ElementType AndValues[] = {GetByteFilledElement<ElementType>(0x66), GetByteFilledElement<ElementType>(0xFF), ElementType(0), };
			constexpr ElementType ExpectedValues[] = {GetByteFilledElement<ElementType>(0x20), Init, ElementType(0), };

			// operator &=
			for (SIZE_T TestIt = 0; TestIt < UE_ARRAY_COUNT(AndValues); ++TestIt)
			{
				TAtomicTestWrapper<ElementType> Data(Init);

				AtomicVal = Data.GetAtomic() &= AndValues[TestIt];
				NativeVal = Data.GetNative() &= AndValues[TestIt];
				check(NativeVal == AtomicVal);

				check(AtomicVal == ExpectedValues[TestIt]);
				Data.Check();
			}

			// AndExchange
			for (SIZE_T TestIt = 0; TestIt < UE_ARRAY_COUNT(AndValues); ++TestIt)
			{
				TAtomicTestWrapper<ElementType> Data(Init);

				AtomicVal = Data.GetAtomic().AndExchange(AndValues[TestIt]);
				NativeVal = Data.GetNative();
				check(NativeVal == AtomicVal);
				NativeVal = Data.GetNative() &= AndValues[TestIt];
				AtomicVal = Data.GetAtomic().Load();
				check(NativeVal == AtomicVal);

				check(AtomicVal == ExpectedValues[TestIt]);
				Data.Check();
			}
		}

		// Or
		{
			constexpr ElementType Init = GetByteFilledElement<ElementType>(0x30);
			constexpr ElementType OrValues[] = {GetByteFilledElement<ElementType>(0x66), GetByteFilledElement<ElementType>(0xFF), ElementType(0), };
			constexpr ElementType ExpectedValues[] = {GetByteFilledElement<ElementType>(0x76), GetByteFilledElement<ElementType>(0xFF), Init, };

			// operator |=
			for (SIZE_T TestIt = 0; TestIt < UE_ARRAY_COUNT(OrValues); ++TestIt)
			{
				TAtomicTestWrapper<ElementType> Data(Init);

				AtomicVal = Data.GetAtomic() |= OrValues[TestIt];
				NativeVal = Data.GetNative() |= OrValues[TestIt];
				check(NativeVal == AtomicVal);

				check(AtomicVal == ExpectedValues[TestIt]);
				Data.Check();
			}

			// OrExchange
			for (SIZE_T TestIt = 0; TestIt < UE_ARRAY_COUNT(OrValues); ++TestIt)
			{
				TAtomicTestWrapper<ElementType> Data(Init);

				AtomicVal = Data.GetAtomic().OrExchange(OrValues[TestIt]);
				NativeVal = Data.GetNative();
				check(NativeVal == AtomicVal);
				NativeVal = Data.GetNative() |= OrValues[TestIt];
				AtomicVal = Data.GetAtomic().Load();
				check(NativeVal == AtomicVal);

				check(AtomicVal == ExpectedValues[TestIt]);
				Data.Check();
			}
		}

		// Xor
		{
			constexpr ElementType Init = GetByteFilledElement<ElementType>(0x30);
			constexpr ElementType XorValues[] = {GetByteFilledElement<ElementType>(0x66), GetByteFilledElement<ElementType>(0xFF), ElementType(0), };
			constexpr ElementType ExpectedValues[] = {GetByteFilledElement<ElementType>(0x56), GetByteFilledElement<ElementType>(~0x30), Init, };

			// operator ^=
			for (SIZE_T TestIt = 0; TestIt < UE_ARRAY_COUNT(XorValues); ++TestIt)
			{
				TAtomicTestWrapper<ElementType> Data(Init);

				AtomicVal = Data.GetAtomic() ^= XorValues[TestIt];
				NativeVal = Data.GetNative() ^= XorValues[TestIt];
				check(NativeVal == AtomicVal);

				check(AtomicVal == ExpectedValues[TestIt]);
				Data.Check();
			}

			// XorExchange
			for (SIZE_T TestIt = 0; TestIt < UE_ARRAY_COUNT(XorValues); ++TestIt)
			{
				TAtomicTestWrapper<ElementType> Data(Init);

				AtomicVal = Data.GetAtomic().XorExchange(XorValues[TestIt]);
				NativeVal = Data.GetNative();
				check(NativeVal == AtomicVal);
				NativeVal = Data.GetNative() ^= XorValues[TestIt];
				AtomicVal = Data.GetAtomic().Load();
				check(NativeVal == AtomicVal);

				check(AtomicVal == ExpectedValues[TestIt]);
				Data.Check();
			}
		}
	}

	template <typename ElementType>
	void RunNumericAtomicTests()
	{
		RunBasicAtomicTests<ElementType>();

		RunArithmeticAtomicTests<ElementType>(50);

		RunBitwiseOperationsAtomicTests<ElementType>();
	}

	template <typename ElementType>
	void RunPointerAtomicTests()
	{
		RunBasicAtomicTests<ElementType>();

		typename TRemovePointer<ElementType>::Type Array[100] = {};
		RunArithmeticAtomicTests<ElementType>(&Array[50]);
	}
}

TEST_CASE_NAMED(FAtomicSmokeTest, "System::Core::Misc::Atomic", "[ApplicationContextMask][SmokeFilter]")
{
	RunNumericAtomicTests<int8>  ();
	RunNumericAtomicTests<uint8> ();
	RunNumericAtomicTests<int16> ();
	RunNumericAtomicTests<uint16>();
	RunNumericAtomicTests<int32> ();
	RunNumericAtomicTests<uint32>();
	RunNumericAtomicTests<int64> ();
	RunNumericAtomicTests<uint64>();

	// Don't run the pointer test on void*, because you can't do arithmetic on void*s.
	RunBasicAtomicTests<               void*>();
	RunBasicAtomicTests<const          void*>();
	RunBasicAtomicTests<      volatile void*>();
	RunBasicAtomicTests<const volatile void*>();

	RunPointerAtomicTests<int8*>();
	RunPointerAtomicTests<int16*>();
	RunPointerAtomicTests<int32*>();
	RunPointerAtomicTests<int64*>();
	RunPointerAtomicTests<FString*>();

	RunPointerAtomicTests<const int8*>();
	RunPointerAtomicTests<const int16*>();
	RunPointerAtomicTests<const int32*>();
	RunPointerAtomicTests<const int64*>();
	RunPointerAtomicTests<const FString*>();

	RunPointerAtomicTests<volatile int8*>();
	RunPointerAtomicTests<volatile int16*>();
	RunPointerAtomicTests<volatile int32*>();
	RunPointerAtomicTests<volatile int64*>();
	RunPointerAtomicTests<volatile FString*>();

	RunPointerAtomicTests<const volatile int8*>();
	RunPointerAtomicTests<const volatile int16*>();
	RunPointerAtomicTests<const volatile int32*>();
	RunPointerAtomicTests<const volatile int64*>();
	RunPointerAtomicTests<const volatile FString*>();
}

#endif // WITH_TESTS
