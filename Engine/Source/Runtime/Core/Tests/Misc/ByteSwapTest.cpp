// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/ByteSwap.h"

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

/**
 * Test byte swapping algorithms.
 */
TEST_CASE_NAMED(FByteSwapTest, "System::Core::Misc::ByteSwap", "[ApplicationContextMask][SmokeFilter]")
{
	int16  ValS16 = static_cast<int16>(0x1122);
	uint16 ValU16 = static_cast<uint16>(0x1122);
	int32  ValS32 = 0xFFEE1122L;
	uint32 ValU32 = 0xFFEE1122UL;
	int64  ValS64 = 0xFFEEDDCC11223344ll;
	uint64 ValU64 = 0xFFEEDDCC11223344ull;
	float  ValF   = *reinterpret_cast<const float*>(&ValU32);
	double ValD   = *reinterpret_cast<const double*>(&ValU64);
	uint32 ExpectedF = ByteSwap(ValU32);
	uint64 ExpectedD = ByteSwap(ValU64);
	char16_t ValCh16 = static_cast<char16_t>(0x2233);

	CHECK_MESSAGE("Swapping singned int16 value",  ByteSwap(ValS16) == 0x2211);
	CHECK_MESSAGE("Swapping unsigned int16 value", ByteSwap(ValU16) == 0x2211);
	CHECK_MESSAGE("Swapping singned int32 value",  ByteSwap(ValS32) == 0x2211EEFF);
	CHECK_MESSAGE("Swapping unsigned int32 value", ByteSwap(ValU32) == 0x2211EEFF);
	CHECK_MESSAGE("Swapping singned int64 value",  ByteSwap(ValS64) == 0x44332211CCDDEEFFll);
	CHECK_MESSAGE("Swapping unsigned int64 value", ByteSwap(ValU64) == 0x44332211CCDDEEFFull);
	CHECK_MESSAGE("Swapping float value",          ByteSwap(ValF)   == *reinterpret_cast<const float*>(&ExpectedF));
	CHECK_MESSAGE("Swapping double value",         ByteSwap(ValD)   == *reinterpret_cast<const double*>(&ExpectedD));
	CHECK_MESSAGE("Swapping char16_t value",       ByteSwap(ValCh16)== 0x3322);
}

// The byte swap benchmarking test are useful to compare the intrinsic implementation vs the generic implementation. Normally, the intrinsic is expected to be faster, but in some
// cases, the intrinsic vs generic speed is within margin of error in optimized build. So the test result are not always consistent. The code remains there in case a new
// implementation needs to be testes. Here some observations:
//     - VC++ 2019 (16.4.3): The intrinsic versions are consistently (and significantly) faster than the generic version. The compiler poorly optimize the generic version.
//     - Apple Clang (11.0.0): No performance change observed in release. Clang generates the same assembly for the intrinsic and the generic functions.
//     - Linux Clang (8.0.1): No performance change observed in release. Clang generates the same assembly for the intrinsic and the generic functions.
#define UE_BYTE_SWAP_BENCHMARK_ENABLED 0
#if UE_BYTE_SWAP_BENCHMARK_ENABLED

/**
 * Compare the performance of swapping bytes using the intrinsic vs generic implementation. See comment above UE_BYTE_SWAP_BENCHMARK_ENABLED.
 */

template<typename T, typename ByteSwapFn>
TPair<FTimespan, T> BenchmarkByteswapping(T InitialValue, uint64 LoopCount, ByteSwapFn InByteSwapFn)
{
	static_assert(TIsIntegral<T>::Value, "The test only works with integers");
	static_assert(!TIsSigned<T>::Value, "The test must support overflowing T when making the sum, value cannot be signed.");

	T Sum = 0; // The sum value is not very relevant, but it is an observable result the compiler cannot ignore (prevent to optimize away the entire loop).
	FDateTime StartTime = FDateTime::UtcNow();
	for (uint64 I = 0; I < LoopCount; ++I)
	{
		Sum += InByteSwapFn(InitialValue + static_cast<T>(I)); // Expect InByteSwapFn to be inlined because the compiler knows the exact function to call.
	}
	FTimespan Duration = FDateTime::UtcNow() - StartTime;
	return MakeTuple(Duration, Sum);
}

TEST_CASE_NAMED(FByteSwapPerformanceTest, "System::Core::Misc::ByteSwapPerf", "[ApplicationContextMask][PerfFilter]")
{
#ifdef UE_BYTESWAP_INTRINSIC_PRIVATE_16
	{
		uint16 InitialValue = 0xF0F0;
		TPair<FTimespan, uint16> Intrinsic = BenchmarkByteswapping(InitialValue, 1000000000, [](uint16 Val){ return ByteSwap(Val); });
		TPair<FTimespan, uint16> Generic   = BenchmarkByteswapping(InitialValue, 1000000000, [](uint16 Val){ return Internal::ByteSwapGeneric16(Val); });
		CHECK_MESSAGE("Swapping uint16 bytes is faster uing the compiler intrinsic than the generic implementation", Intrinsic.Get<0>() <= Generic.Get<0>());
		CHECK_MESSAGE("Swapping uint16 bytes using intrinsic and generic algorithm produce the same values", Intrinsic.Get<1>() == Generic.Get<1>());
		GLog->Logf(TEXT("Swapping 2 bytes using intrinsic is %s faster than generic version"), *LexToString(Generic.Get<0>().GetTotalMicroseconds() / Intrinsic.Get<0>().GetTotalMicroseconds()));
	}
#endif

#ifdef UE_BYTESWAP_INTRINSIC_PRIVATE_32
	{
		uint32 InitialValue = 0xFF00FF00ul;
		TPair<FTimespan, uint32> Intrinsic = BenchmarkByteswapping(InitialValue, 1000000000, [](uint32 Val){ return ByteSwap(Val); });
		TPair<FTimespan, uint32> Generic   = BenchmarkByteswapping(InitialValue, 1000000000, [](uint32 Val){ return Internal::ByteSwapGeneric32(Val); });
		CHECK_MESSAGE("Swapping uint32 bytes is faster uing the compiler intrinsic than the generic implementation", Intrinsic.Get<0>() <= Generic.Get<0>());
		CHECK_MESSAGE("Swapping uint32 bytes using intrinsic and generic algorithm produce the same values", Intrinsic.Get<1>() == Generic.Get<1>());
		GLog->Logf(TEXT("Swapping 4 bytes using intrinsic is %s faster than generic version"), *LexToString(Generic.Get<0>().GetTotalMicroseconds() / Intrinsic.Get<0>().GetTotalMicroseconds()));
	}
#endif

#ifdef UE_BYTESWAP_INTRINSIC_PRIVATE_64
	{
		uint64 InitialValue = 0xFF00FF00FF00FF00ull;
		TPair<FTimespan, uint64> Intrinsic = BenchmarkByteswapping(InitialValue, 1000000000, [](uint64 Val){ return ByteSwap(Val); });
		TPair<FTimespan, uint64> Generic   = BenchmarkByteswapping(InitialValue, 1000000000, [](uint64 Val){ return Internal::ByteSwapGeneric64(Val); });
		CHECK_MESSAGE("Swapping uint64 bytes is faster uing the compiler intrinsic than the generic implementation", Intrinsic.Get<0>() <= Generic.Get<0>());
		CHECK_MESSAGE("Swapping uint64 bytes using intrinsic and generic algorithm produce the same values", Intrinsic.Get<1>() == Generic.Get<1>());
		GLog->Logf(TEXT("Swapping 8 bytes using intrinsic is %s faster than generic version"), *LexToString(Generic.Get<0>().GetTotalMicroseconds() / Intrinsic.Get<0>().GetTotalMicroseconds()));
	}
#endif

}

#endif //UE_BYTE_SWAP_BENCHMARK_ENABLED

#endif // WITH_TESTS
