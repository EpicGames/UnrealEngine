// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS 

#include "Misc/Char.h"
#include <locale.h>
#include <ctype.h>
#include <wctype.h>
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

namespace crt
{
	int tolower(ANSICHAR c) { return ::tolower(c); }
	int toupper(ANSICHAR c) { return ::toupper(c); }

	int tolower(WIDECHAR c) { return ::towlower(c); }
	int toupper(WIDECHAR c) { return ::towupper(c); }
}

template<typename CharType>
void RunCharTests(uint32 MaxChar)
{
	for (uint32 I = 0; I < MaxChar; ++I)
	{
		CharType C = (CharType)I;
		CHECK_EQUALS(TEXT("TChar::ToLower()"), TChar<CharType>::ToLower(C), crt::tolower(C));
		CHECK_EQUALS(TEXT("TChar::ToUpper()"), TChar<CharType>::ToUpper(C), crt::toupper(C));
	}
}

TEST_CASE_NAMED(TCharTest, "System::Core::Misc::Char", "[ApplicationContextMask][SmokeFilter][AndroidSkip]")
{
	const char* CurrentLocale = setlocale(LC_CTYPE, nullptr);
	if (CurrentLocale == nullptr)
	{
		FAIL_CHECK(FString::Printf(TEXT("Locale is null but should be \"C\". Did something call setlocale()?")));
	}
	else if (strcmp("C", CurrentLocale))
	{
		FAIL_CHECK(FString::Printf(TEXT("Locale is \"%s\" but should be \"C\". Did something call setlocale()?"), ANSI_TO_TCHAR(CurrentLocale)));
	}
	else
	{
		RunCharTests<ANSICHAR>(128);
		RunCharTests<WIDECHAR>(65536);
	}

}

#endif //WITH_TESTS