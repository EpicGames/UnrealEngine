// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Containers/CircularBuffer.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(FCircularBufferTest, "System::Core::Misc::CircularBuffer", "[ApplicationContextMask][SmokeFilter]")
{
	// buffer capacity
	TCircularBuffer<int32> b1_1(127);
	TCircularBuffer<int32> b1_2(128);
	TCircularBuffer<int32> b1_3(129);


	CHECK_EQUALS(TEXT("Buffer capacity of 127 must be rounded up to 128"), b1_1.Capacity(), 128u);
	CHECK_EQUALS(TEXT("Buffer capacity of 128 must not change"), b1_2.Capacity(), 128u);
	CHECK_EQUALS(TEXT("Buffer capacity of 129 must be rounded up to 256"), b1_3.Capacity(), 256u);

	// initial values
	TCircularBuffer<int32> b2_1(64, 666);

	for (uint32 Index = 0; Index < b2_1.Capacity(); ++Index)
	{
		CHECK_EQUALS(FString::Printf(TEXT("Initial value must be correct (%i)"), Index), b2_1[Index], 666);
	}

	// indexing
	TCircularBuffer<int32> b3_1(64, 0);

	CHECK_EQUALS(TEXT("Next index from 0 must be 1"), b3_1.GetNextIndex(0), 1u);
	CHECK_EQUALS(TEXT("Next index from 63 must be 0"), b3_1.GetNextIndex(63), 0u);
	CHECK_EQUALS(TEXT("Next index from 64 must be 1"), b3_1.GetNextIndex(64), 1u);

	b3_1[0] = 42;
	b3_1[65] = 42;

	CHECK_EQUALS(TEXT("Index 0 must be written and read correctly"), b3_1[0], 42);
	CHECK_EQUALS(TEXT("Index 1 must be written and read correctly"), b3_1[1], 42);
	CHECK_EQUALS(TEXT("Index 65 must be written and read correctly"), b3_1[65], 42);

}

#endif //WITH_TESTS
