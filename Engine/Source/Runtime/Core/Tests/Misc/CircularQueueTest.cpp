// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Containers/CircularQueue.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

#pragma warning(disable : 6011)

TEST_CASE_NAMED(FCircularQueueTest, "System::Core::Misc::CircularQueue", "[ApplicationContextMask][SmokeFilter]")
{
	const uint32 QueueSize = 8;

	// empty queue
	{
		TCircularQueue<int32> Queue(QueueSize);

		CHECK_EQUALS(TEXT("Newly created queues must have zero elements"), Queue.Count(), 0u);
		CHECK_MESSAGE(TEXT("Newly created queues must be empty"), Queue.IsEmpty());
		CHECK_FALSE_MESSAGE(TEXT("Newly created queues must not be full"), Queue.IsFull());

		int32 Value;
		CHECK_FALSE_MESSAGE(TEXT("Peek must fail on an empty queue"), Queue.Peek(Value));
		CHECK_MESSAGE(TEXT("Peek must fail on an empty queue"), Queue.Peek() == nullptr);
	}

	// partially filled
	{
		TCircularQueue<int32> Queue(QueueSize);
		int32 Value = 0;

		CHECK_MESSAGE(TEXT("Adding to an empty queue must succeed"), Queue.Enqueue(666));
		CHECK_EQUALS(TEXT("After adding to an empty queue it must have one element"), Queue.Count(), 1u);
		CHECK_FALSE_MESSAGE(TEXT("Partially filled queues must not be empty"), Queue.IsEmpty());
		CHECK_FALSE_MESSAGE(TEXT("Partially filled queues must not be full"), Queue.IsFull());
		CHECK_MESSAGE(TEXT("Peeking at a partially filled queue must succeed"), Queue.Peek(Value));
		CHECK_EQUALS(TEXT("The peeked at value must be correct"), Value, 666);

		const int32* PeekValue = Queue.Peek();
		CHECK_MESSAGE(TEXT("Peeking at a partially filled queue must succeed"), PeekValue != nullptr);
		CHECK_EQUALS(TEXT("The peeked at value must be correct"), *PeekValue, 666);
	}

	// full queue
	for (uint32 PeekType = 0; PeekType < 2; PeekType++)
	{
		TCircularQueue<int32> Queue(QueueSize);

		for (int32 Index = 0; Index < QueueSize - 1; ++Index)
		{
			CHECK_MESSAGE(TEXT("Adding to non-full queue must succeed"), Queue.Enqueue(Index));
		}

		CHECK_FALSE_MESSAGE(TEXT("Full queues must not be empty"), Queue.IsEmpty());
		CHECK_MESSAGE(TEXT("Full queues must be full"), Queue.IsFull());
		CHECK_FALSE_MESSAGE(TEXT("Adding to full queue must fail"), Queue.Enqueue(666));

		int32 Value = 0;

		for (int32 Index = 0; Index < QueueSize - 1; ++Index)
		{
			if (PeekType == 0)
			{
				CHECK_MESSAGE(TEXT("Peeking at a non-empty queue must succeed"), Queue.Peek(Value));
				CHECK_EQUALS(TEXT("The peeked at value must be correct"), Value, Index);

				CHECK_MESSAGE(TEXT("Removing from a non-empty queue must succeed"), Queue.Dequeue(Value));
				CHECK_EQUALS(TEXT("The removed value must be correct"), Value, Index);
			}
			else
			{
				const int32* PeekValue = Queue.Peek();
				CHECK_MESSAGE(TEXT("Peeking at a non-empty queue must succeed"), PeekValue != nullptr);
				CHECK_EQUALS(TEXT("The peeked at value must be correct"), *PeekValue, Index);

				CHECK_MESSAGE(TEXT("Removing from a non-empty queue must succeed"), Queue.Dequeue());
			}
		}

		CHECK_MESSAGE(TEXT("A queue that had all items removed must be empty"), Queue.IsEmpty());
		CHECK_FALSE_MESSAGE(TEXT("A queue that had all items removed must not be full"), Queue.IsFull());
	}

	// queue with index wrapping around
	{
		TCircularQueue<int32> Queue(QueueSize);

		//Fill queue
		for (int32 Index = 0; Index < QueueSize - 1; ++Index)
		{
			CHECK_MESSAGE(TEXT("Adding to non-full queue must succeed"), Queue.Enqueue(Index));
		}

		int32 Value = 0;
		const int32 ExpectedSize = QueueSize - 1;
		for (int32 Index = 0; Index < QueueSize; ++Index)
		{
			CHECK_EQUALS(TEXT("Number of elements must be valid for all permutation of Tail and Head"), Queue.Count(), ExpectedSize);
			CHECK_MESSAGE(TEXT("Removing from a non-empty queue must succeed"), Queue.Dequeue(Value));
			CHECK_MESSAGE(TEXT("Adding to non-full queue must succeed"), Queue.Enqueue(Index));
		}
	}

	// Non-zero initialization - ensures the backing store constructs and destructs non-POD objects properly.
	{
		static uint32 Const;
		static uint32 Dest;
		static uint32 CopyConst;
		struct FNonPOD
		{
			FNonPOD() { Const++; }
			~FNonPOD() { Dest++; }
			FNonPOD(const FNonPOD&) { CopyConst++; }
		};

		// The current implementation of TCircularQueue doesn't call the held object constructors on but does call the
		// destructors. Verify that (somewhat surprising, if typically ok for POD) behavior first.
		Const = 0;
		Dest = 0;
		CopyConst = 0;
		{
			TCircularQueue<FNonPOD> Queue(QueueSize);

			CHECK_EQUALS(TEXT("The constructor should not run"), Const, 0);
			CHECK_EQUALS(TEXT("The destructor should not run"), Dest, 0);
			CHECK_EQUALS(TEXT("The copy constructor should not run"), CopyConst, 0);
		}
		CHECK_EQUALS(TEXT("The constructor should not run"), Const, 0);
		CHECK_EQUALS(TEXT("The destructor should run"), Dest, QueueSize);
		CHECK_EQUALS(TEXT("The copy constructor should not run"), CopyConst, 0);
	}

}

#pragma warning(default : 6011)

#endif //WITH_TESTS
