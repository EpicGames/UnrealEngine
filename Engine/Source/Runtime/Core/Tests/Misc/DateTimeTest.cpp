// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

#define TestUnixEquivalent(Desc, A, B) if ((A).ToUnixTimestamp() != (B)) FAIL_CHECK(FString::Printf(TEXT("%s - A=%" INT64_FMT " B=%" INT64_FMT), Desc, (A).ToUnixTimestamp(), (B)));
#define TestYear(Desc, A, B) if ((A.GetYear()) != (B)) FAIL_CHECK(FString::Printf(TEXT("%s - A=%d B=%d"), Desc, (A.GetYear()), (B)));
#define TestMonth(Desc, A, B) if ((A.GetMonth()) != (B)) FAIL_CHECK(FString::Printf(TEXT("%s - A=%d B=%d"), Desc, (A.GetMonth()), B));
#define TestMonthOfYear(Desc, A, B) if ((A.GetMonthOfYear()) != (B)) FAIL_CHECK(FString::Printf(TEXT("%s - A=%d B=%d"), Desc ,(static_cast<int32>(A.GetMonthOfYear())), static_cast<int32>(B)));
#define TestDay(Desc, A, B) if ((A.GetDay()) != (B)) FAIL_CHECK(FString::Printf(TEXT("%s - A=%d B=%d"), Desc, (A.GetDay()), (B)));
#define TestHour(Desc, A, B) if ((A.GetHour()) != (B)) FAIL_CHECK(FString::Printf(TEXT("%s - A=%d B=%d"), Desc, (A.GetHour()), (B)));
#define TestMinute(Desc, A, B) if ((A.GetMinute()) != (B)) FAIL_CHECK(FString::Printf(TEXT("%s - A=%d B=%d"), Desc, (A.GetMinute()), (B)));
#define TestSecond(Desc, A, B) if ((A.GetSecond()) != (B)) FAIL_CHECK(FString::Printf(TEXT("%s - A=%d B=%d"), Desc, (A.GetSecond()), (B)));
#define TestMillisecond(Desc, A, B) if ((A.GetMillisecond()) != (B)) FAIL_CHECK(FString::Printf(TEXT("%s - A=%d B=%d"), Desc, (A.GetMillisecond()) ,(B)));

TEST_CASE_NAMED(FDateTimeTest, "System::Core::Misc::DateTime", "[ApplicationContextMask][SmokeFilter]")
{
	const int64 UnixEpochTimestamp = 0;
	const int64 UnixBillenniumTimestamp = 1000000000;
	const int64 UnixOnesTimestamp = 1111111111;
	const int64 UnixDecimalSequenceTimestamp = 1234567890;

	const FDateTime UnixEpoch = FDateTime::FromUnixTimestamp(UnixEpochTimestamp);
	const FDateTime UnixBillennium = FDateTime::FromUnixTimestamp(UnixBillenniumTimestamp);
	const FDateTime UnixOnes = FDateTime::FromUnixTimestamp(UnixOnesTimestamp);
	const FDateTime UnixDecimalSequence = FDateTime::FromUnixTimestamp(UnixDecimalSequenceTimestamp);

	const FDateTime TestDateTime(2013, 8, 14, 12, 34, 56, 789);

	TestUnixEquivalent(TEXT("Testing Unix Epoch Ticks"), UnixEpoch, UnixEpochTimestamp);
	TestUnixEquivalent(TEXT("Testing Unix Billennium Ticks"), UnixBillennium, UnixBillenniumTimestamp);
	TestUnixEquivalent(TEXT("Testing Unix Ones Ticks"), UnixOnes, UnixOnesTimestamp);
	TestUnixEquivalent(TEXT("Testing Unix Decimal Sequence Ticks"), UnixDecimalSequence, UnixDecimalSequenceTimestamp);

	TestYear(TEXT("Testing Unix Epoch Year"), UnixEpoch, 1970);
	TestMonth(TEXT("Testing Unix Epoch Month"), UnixEpoch, 1);
	TestMonthOfYear(TEXT("Testing Unix Epoch MonthOfYear"), UnixEpoch, EMonthOfYear::January);
	TestDay(TEXT("Testing Unix Epoch Day"), UnixEpoch, 1);
	TestHour(TEXT("Testing Unix Epoch Hour"), UnixEpoch, 0);
	TestMinute(TEXT("Testing Unix Epoch Minute"), UnixEpoch, 0);
	TestSecond(TEXT("Testing Unix Epoch Second"), UnixEpoch, 0);
	TestMillisecond(TEXT("Testing Unix Epoch Millisecond"), UnixEpoch, 0);

	TestYear(TEXT("Testing Unix Billennium Year"), UnixBillennium, 2001);
	TestMonth(TEXT("Testing Unix Billennium MonthOfYear"), UnixBillennium, 9);
	TestMonthOfYear(TEXT("Testing Unix Billennium Month"), UnixBillennium, EMonthOfYear::September);
	TestDay(TEXT("Testing Unix Billennium Day"), UnixBillennium, 9);
	TestHour(TEXT("Testing Unix Billennium Hour"), UnixBillennium, 1);
	TestMinute(TEXT("Testing Unix Billennium Minute"), UnixBillennium, 46);
	TestSecond(TEXT("Testing Unix Billennium Second"), UnixBillennium, 40);
	TestMillisecond(TEXT("Testing Unix Billennium Millisecond"), UnixBillennium, 0);

	TestYear(TEXT("Testing Unix Ones Year"), UnixOnes, 2005);
	TestMonth(TEXT("Testing Unix Ones Month"), UnixOnes, 3);
	TestMonthOfYear(TEXT("Testing Unix Ones MonthOfYear"), UnixOnes, EMonthOfYear::March);
	TestDay(TEXT("Testing Unix Ones Day"), UnixOnes, 18);
	TestHour(TEXT("Testing Unix Ones Hour"), UnixOnes, 1);
	TestMinute(TEXT("Testing Unix Ones Minute"), UnixOnes, 58);
	TestSecond(TEXT("Testing Unix Ones Second"), UnixOnes, 31);
	TestMillisecond(TEXT("Testing Unix Ones Millisecond"), UnixOnes, 0);

	TestYear(TEXT("Testing Unix Decimal Sequence Year"), UnixDecimalSequence, 2009);
	TestMonth(TEXT("Testing Unix Decimal Sequence Month"), UnixDecimalSequence, 2);
	TestMonthOfYear(TEXT("Testing Unix Decimal Sequence MonthOfYear"), UnixDecimalSequence, EMonthOfYear::February);
	TestDay(TEXT("Testing Unix Decimal Sequence Day"), UnixDecimalSequence, 13);
	TestHour(TEXT("Testing Unix Decimal Sequence Hour"), UnixDecimalSequence, 23);
	TestMinute(TEXT("Testing Unix Decimal Sequence Minute"), UnixDecimalSequence, 31);
	TestSecond(TEXT("Testing Unix Decimal Sequence Second"), UnixDecimalSequence, 30);
	TestMillisecond(TEXT("Testing Unix Decimal Sequence Millisecond"), UnixDecimalSequence, 0);

	TestYear(TEXT("Testing Test Date Time Year"), TestDateTime, 2013);
	TestMonth(TEXT("Testing Test Date Time Month"), TestDateTime, 8);
	TestMonthOfYear(TEXT("Testing Test Date Time MonthOfYear"), TestDateTime, EMonthOfYear::August);
	TestDay(TEXT("Testing Test Date Time Day"), TestDateTime, 14);
	TestHour(TEXT("Testing Test Date Time Hour"), TestDateTime, 12);
	TestMinute(TEXT("Testing Test Date Time Minute"), TestDateTime, 34);
	TestSecond(TEXT("Testing Test Date Time Second"), TestDateTime, 56);
	TestMillisecond(TEXT("Testing Test Date Time Millisecond"), TestDateTime, 789);

	FDateTime ParsedDateTime;

	CHECK_FALSE_MESSAGE(TEXT("Parsing an empty ISO string must fail"), FDateTime::ParseIso8601(TEXT(""), ParsedDateTime));

	FDateTime::ParseIso8601(TEXT("2019-05-22"), ParsedDateTime);
	CHECK_EQUALS(TEXT("Testing ISO 8601 date"), ParsedDateTime, (FDateTime{ 2019, 5, 22 }));

	FDateTime::ParseIso8601(TEXT("2019-05-20T19:41:38+01:30"), ParsedDateTime);
	CHECK_EQUALS(TEXT("Testing ISO 8601 with +hh:mm timezone info"), ParsedDateTime, (FDateTime{ 2019, 5, 20, 18, 11, 38 }));
	FDateTime::ParseIso8601(TEXT("2019-05-20T19:41:38-01:30"), ParsedDateTime);
	CHECK_EQUALS(TEXT("Testing ISO 8601 with -hh:mm timezone info"), ParsedDateTime, (FDateTime{ 2019, 5, 20, 21, 11, 38 }));
	FDateTime::ParseIso8601(TEXT("2019-05-20T19:41:38+0030"), ParsedDateTime);
	CHECK_EQUALS(TEXT("Testing ISO 8601 with +hhmm timezone info"), ParsedDateTime, (FDateTime{ 2019, 5, 20, 19, 11, 38 }));
	FDateTime::ParseIso8601(TEXT("2019-05-20T19:41:38-01"), ParsedDateTime);
	CHECK_EQUALS(TEXT("Testing ISO 8601 with -hh timezone info"), ParsedDateTime, (FDateTime{ 2019, 5, 20, 20, 41, 38 }));

	for (double JulianDay : {0., 1000., 1721425.5, 1721425.0, FDateTime::MinValue().GetJulianDay(), FDateTime::MaxValue().GetJulianDay()})
	{
		CHECK_EQUALS(TEXT("convertion from/to JulianDay is stable"), FDateTime::FromJulianDay(JulianDay).GetJulianDay(), JulianDay);
	}

	FDateTime Date{ 2019, 5, 20, 18, 11, 38 };
	FDateTime DateMidnight = Date.GetDate();
	CHECK_EQUALS(TEXT("GetDate returns a FDateTime at midnight"), DateMidnight.GetHour(), 0);

	FDateTime DateNoon = DateMidnight + FTimespan::FromHours(12);
	CHECK_EQUALS(TEXT("GetDate returns a FDateTime at midnight"), DateNoon.GetHour(), 12);

	double JulianDayMidnight = DateMidnight.GetJulianDay();
	double JulianDayNoon = DateNoon.GetJulianDay();
	CHECK_EQUALS(TEXT("At midnight, fractionnal part of the JulianDay value should be 0.5"), JulianDayMidnight - (int)JulianDayMidnight, 0.5);
	CHECK_EQUALS(TEXT("A 12h timespan adds half a julianday"), JulianDayNoon - JulianDayMidnight, 0.5);

	double OffsetDayCount = 12345.;
	FDateTime OffsetDate = Date + FTimespan::FromDays(OffsetDayCount);
	CHECK_EQUALS(TEXT("An offset by a given numer of days leads to a similar JulianDay offset"), OffsetDate.GetJulianDay() - Date.GetJulianDay(), OffsetDayCount);

	int32 Year, Month, Day;
	DateMidnight.GetDate(Year, Month, Day);
	FDateTime PreviousDay(DateMidnight.GetTicks() - 1);
	int32 YearPrev, MonthPrev, DayPrev;
	PreviousDay.GetDate(YearPrev, MonthPrev, DayPrev);
	CHECK_NOT_EQUALS(TEXT("One tick before a date at midnight leads to the previous date"), Day, DayPrev);
}


#undef TestUnixEquivalent
#undef TestYear
#undef TestMonth
#undef TestDay
#undef TestHour
#undef TestMinute
#undef TestSecond
#undef TestMillisecond


#endif //WITH_TESTS
