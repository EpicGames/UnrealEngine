// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS 

#include "UObject/UnrealNames.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(TDisplayNameTest, "System::Core::Misc::DisplayName", "[ApplicationContextMask][SmokeFilter]")
{
	CHECK_EQUALS(TEXT("Boolean"), FName::NameToDisplayString(TEXT("bTest"), true), TEXT("Test"));
	CHECK_EQUALS(TEXT("Boolean Lower"), FName::NameToDisplayString(TEXT("bTwoWords"), true), TEXT("Two Words"));
	CHECK_EQUALS(TEXT("Lower Boolean"), FName::NameToDisplayString(TEXT("boolean"), true), TEXT("Boolean"));
	CHECK_EQUALS(TEXT("Almost Boolean"), FName::NameToDisplayString(TEXT("bNotBoolean"), false), TEXT("B Not Boolean"));
	CHECK_EQUALS(TEXT("Boolean No Prefix"), FName::NameToDisplayString(TEXT("NonprefixBoolean"), true), TEXT("Nonprefix Boolean"));
	CHECK_EQUALS(TEXT("Lower Boolean No Prefix"), FName::NameToDisplayString(TEXT("lowerNonprefixBoolean"), true), TEXT("Lower Nonprefix Boolean"));
	CHECK_EQUALS(TEXT("Lower Camel Case"), FName::NameToDisplayString(TEXT("lowerCase"), false), TEXT("Lower Case"));
	CHECK_EQUALS(TEXT("With Underscores"), FName::NameToDisplayString(TEXT("With_Underscores"), false), TEXT("With Underscores"));
	CHECK_EQUALS(TEXT("Lower Underscores"), FName::NameToDisplayString(TEXT("lower_underscores"), false), TEXT("Lower Underscores"));
	CHECK_EQUALS(TEXT("Mixed Underscores"), FName::NameToDisplayString(TEXT("mixed_Underscores"), false), TEXT("Mixed Underscores"));
	CHECK_EQUALS(TEXT("Mixed Underscores"), FName::NameToDisplayString(TEXT("Mixed_underscores"), false), TEXT("Mixed Underscores"));
	CHECK_EQUALS(TEXT("Article in String"), FName::NameToDisplayString(TEXT("ArticleInString"), false), TEXT("Article in String"));
	CHECK_EQUALS(TEXT("One or Two"), FName::NameToDisplayString(TEXT("OneOrTwo"), false), TEXT("One or Two"));
	CHECK_EQUALS(TEXT("One and Two"), FName::NameToDisplayString(TEXT("OneAndTwo"), false), TEXT("One and Two"));
	CHECK_EQUALS(TEXT("-1.5"), FName::NameToDisplayString(TEXT("-1.5"), false), TEXT("-1.5"));
	CHECK_EQUALS(TEXT("1234"), FName::NameToDisplayString(TEXT("1234"), false), TEXT("1234"));
	CHECK_EQUALS(TEXT("1234.5"), FName::NameToDisplayString(TEXT("1234.5"), false), TEXT("1234.5"));
	CHECK_EQUALS(TEXT("-1234.5"), FName::NameToDisplayString(TEXT("-1234.5"), false), TEXT("-1234.5"));
	CHECK_EQUALS(TEXT("Text (In Parens)"), FName::NameToDisplayString(TEXT("Text (in parens)"), false), TEXT("Text (In Parens)"));

	CHECK_EQUALS(TEXT("Text 3D"), FName::NameToDisplayString(TEXT("Text3D"), false), TEXT("Text 3D"));
	CHECK_EQUALS(TEXT("Plural CAPs"), FName::NameToDisplayString(TEXT("PluralCAPs"), false), TEXT("Plural CAPs"));
	CHECK_EQUALS(TEXT("FBXEditor"), FName::NameToDisplayString(TEXT("FBXEditor"), false), TEXT("FBXEditor"));
	CHECK_EQUALS(TEXT("FBX_Editor"), FName::NameToDisplayString(TEXT("FBX_Editor"), false), TEXT("FBX Editor"));
}

#endif //WITH_TESTS