// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Misc/ScopeExit.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

// These file tests are designed to ensure expected file writing behavior, as well as cross-platform consistency

TEST_CASE_NAMED(FFileTruncateTest, "System::Core::Misc::FileTruncate", "[.][EditorContext][CriticalPriority][EngineFilter]")
{
	const FString TempFilename = FPaths::CreateTempFilename(*FPaths::ProjectIntermediateDir());
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	ON_SCOPE_EXIT
	{
		// Delete temp file
		PlatformFile.DeleteFile(*TempFilename);
	};

	// Open a test file
	if (TUniquePtr<IFileHandle> TestFile = TUniquePtr<IFileHandle>(PlatformFile.OpenWrite(*TempFilename, /*bAppend*/false, /*bAllowRead*/true)))
	{
		// Append 4 int32 values of incrementing value to this file
		int32 Val = 1;
		TestFile->Write((const uint8*)&Val, sizeof(Val));

		++Val;
		TestFile->Write((const uint8*)&Val, sizeof(Val));

		// Tell here, so we can move back and truncate after writing
		const int64 ExpectedTruncatePos = TestFile->Tell();
		++Val;
		TestFile->Write((const uint8*)&Val, sizeof(Val));

		// Tell here, so we can attempt to read here after truncation
		const int64 TestReadPos = TestFile->Tell();
		++Val;
		TestFile->Write((const uint8*)&Val, sizeof(Val));

		// Validate that the Tell position is at the end of the file, and that the size is reported correctly
		{
			const int64 ActualEOFPos = TestFile->Tell();
			const int64 ExpectedEOFPos = (sizeof(int32) * 4);
			REQUIRE_MESSAGE(FString::Printf(TEXT("File was not the expected size (got %" INT64_FMT ", expected %" INT64_FMT "): %s"), ActualEOFPos, ExpectedEOFPos, *TempFilename), ActualEOFPos == ExpectedEOFPos);

			const int64 ActualFileSize = TestFile->Size();
			REQUIRE_MESSAGE(FString::Printf(TEXT("File was not the expected size (got %" INT64_FMT ", expected %" INT64_FMT "): %s"), ActualFileSize, ExpectedEOFPos, *TempFilename), ActualFileSize == ExpectedEOFPos);
		}

		// Truncate the file at our test pos
		REQUIRE_MESSAGE(FString::Printf(TEXT("File truncation request failed: %s"), *TempFilename), TestFile->Truncate(ExpectedTruncatePos));

		// Validate that the size is reported correctly
		{
			const int64 ActualFileSize = TestFile->Size();
			REQUIRE_MESSAGE(FString::Printf(TEXT("File was not the expected size after truncation (got %" INT64_FMT ", expected %" INT64_FMT "): %s"), ActualFileSize, ExpectedTruncatePos, *TempFilename), ActualFileSize == ExpectedTruncatePos);
		}

		// Validate that we can't read past the truncation point
		{
			int32 Dummy = 0;
			REQUIRE_MESSAGE(FString::Printf(TEXT("File read seek outside the truncated range: %s"), *TempFilename), !(TestFile->Seek(TestReadPos) && TestFile->Read((uint8*)&Dummy, sizeof(Dummy))));
		}
	}
	else
	{
		REQUIRE_MESSAGE(FString::Printf(TEXT("File failed to open: %s"), *TempFilename), false);
	}
}

TEST_CASE_NAMED(FFileAppendTest, "System::Core::Misc::FileAppend", "[.][EditorContext][CriticalPriority][EngineFilter]")
{
	const FString TempFilename = FPaths::CreateTempFilename(*FPaths::ProjectIntermediateDir());
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	ON_SCOPE_EXIT
	{
		// Delete temp file
		PlatformFile.DeleteFile(*TempFilename);
	};

	// Scratch data for testing
	uint8 One = 1;
	TArray<uint8> TestData;

	// Check a new file can be created
	if (TUniquePtr<IFileHandle> TestFile = TUniquePtr<IFileHandle>(PlatformFile.OpenWrite(*TempFilename, /*bAppend*/false, /*bAllowRead*/true)))
	{
		TestData.AddZeroed(64);

		TestFile->Write(TestData.GetData(), TestData.Num());
	}
	else
	{
		REQUIRE_MESSAGE(FString::Printf(TEXT("File failed to open when new: %s"), *TempFilename), false);
	}

	// Confirm same data
	{
		TArray<uint8> ReadData;
		REQUIRE_MESSAGE(FString::Printf(TEXT("File failed to load after writing: %s"), *TempFilename), FFileHelper::LoadFileToArray(ReadData, *TempFilename));
		REQUIRE_MESSAGE(FString::Printf(TEXT("File data was incorrect after writing: %s"), *TempFilename), ReadData == TestData);
	}

	// Using append flag should open the file, and writing data immediately should append to the end.
	// We should also be capable of seeking writing.
	if (TUniquePtr<IFileHandle> TestFile = TUniquePtr<IFileHandle>(PlatformFile.OpenWrite(*TempFilename, /*bAppend*/true, /*bAllowRead*/true)))
	{
		// Validate the file actually opened in append mode correctly
		{
			const int64 ActualEOFPos = TestFile->Tell();
			const int64 ExpectedEOFPos = TestFile->Size();
			REQUIRE_MESSAGE(FString::Printf(TEXT("File did not seek to the end when opening (got %" INT64_FMT ", expected %" INT64_FMT "): %s"), ActualEOFPos, ExpectedEOFPos, *TempFilename), ActualEOFPos == ExpectedEOFPos);
		}

		TestData.Add(One);
		TestData[10] = One;

		TestFile->Write(&One, 1);
		TestFile->Seek(10);
		TestFile->Write(&One, 1);
	}
	else
	{
		REQUIRE_MESSAGE(FString::Printf(TEXT("File failed to open when appending: %s"), *TempFilename), false);
	}

	// Confirm same data
	{
		TArray<uint8> ReadData;
		REQUIRE_MESSAGE(FString::Printf(TEXT("File failed to load after appending: %s"), *TempFilename), FFileHelper::LoadFileToArray(ReadData, *TempFilename));
		REQUIRE_MESSAGE(FString::Printf(TEXT("File data was incorrect after appending: %s"), *TempFilename), ReadData == TestData);
	}

	// No append should clobber existing file
	if (TUniquePtr<IFileHandle> TestFile = TUniquePtr<IFileHandle>(PlatformFile.OpenWrite(*TempFilename, /*bAppend*/false, /*bAllowRead*/true)))
	{
		TestData.Reset();
		TestData.Add(One);

		TestFile->Write(&One, 1);
	}
	else
	{
		REQUIRE_MESSAGE(FString::Printf(TEXT("File failed to open when clobbering: %s"), *TempFilename), false);
	}

	// Confirm same data
	{
		TArray<uint8> ReadData;
		REQUIRE_MESSAGE(FString::Printf(TEXT("File failed to load after clobbering: %s"), *TempFilename), FFileHelper::LoadFileToArray(ReadData, *TempFilename));
		REQUIRE_MESSAGE(FString::Printf(TEXT("File data was incorrect after clobbering: %s"), *TempFilename), ReadData == TestData);
	}
}

TEST_CASE_NAMED(FFileShrinkBuffersTest, "System::Core::Misc::FileShrinkBuffers", "[.][EditorContext][CriticalPriority][EngineFilter]")
{
	const FString TempFilename = FPaths::CreateTempFilename(*FPaths::ProjectIntermediateDir());
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	ON_SCOPE_EXIT
	{
		// Delete temp file
		PlatformFile.DeleteFile(*TempFilename);
	};

	// Scratch data for testing
	uint8 One = 1;
	TArray<uint8> TestData;

	// Check a new file can be created
	if (TUniquePtr<IFileHandle> TestFile = TUniquePtr<IFileHandle>(PlatformFile.OpenWrite(*TempFilename, /*bAppend*/false, /*bAllowRead*/true)))
	{
		for (uint8 i = 0; i < 64; ++i)
		{
			TestData.Add(i);
		}

		TestFile->Write(TestData.GetData(), TestData.Num());
	}
	else
	{
		REQUIRE_MESSAGE(FString::Printf(TEXT("File failed to open when new: %s"), *TempFilename), false);
	}

	// Confirm same data
	{
		TArray<uint8> ReadData;
		REQUIRE_MESSAGE(FString::Printf(TEXT("File failed to load after writing: %s"), *TempFilename), FFileHelper::LoadFileToArray(ReadData, *TempFilename));
		REQUIRE_MESSAGE(FString::Printf(TEXT("File data was incorrect after writing: %s"), *TempFilename), ReadData == TestData);
	}

	// Using ShrinkBuffers should not disrupt our read position in the file
	if (TUniquePtr<IFileHandle> TestFile = TUniquePtr<IFileHandle>(PlatformFile.OpenRead(*TempFilename, /*bAllowWrite*/false)))
	{
		// Validate the file actually opened and is of the right size
		CHECK_EQUALS(TEXT("File not of expected size at time of ShrinkBuffers read test"), TestFile->Size(), static_cast<decltype(TestFile->Size())>(TestData.Num()));

		const int32 FirstHalfSize = TestData.Num() / 2;
		const int32 SecondHalfSize = TestData.Num() - FirstHalfSize;

		TArray<uint8> FirstHalfReadData;
		FirstHalfReadData.AddUninitialized(FirstHalfSize);
		CHECK_MESSAGE(TEXT("Failed to read first half of test file"), TestFile->Read(FirstHalfReadData.GetData(), FirstHalfReadData.Num()));

		for (int32 i = 0; i < FirstHalfSize; ++i)
		{
			CHECK_EQUALS(TEXT("Mismatch in data before ShrinkBuffers was called"), FirstHalfReadData[i], TestData[i]);
		}

		TestFile->ShrinkBuffers();

		TArray<uint8> SecondHalfReadData;
		SecondHalfReadData.AddUninitialized(SecondHalfSize);
		CHECK_MESSAGE(TEXT("Failed to read second half of test file"), TestFile->Read(SecondHalfReadData.GetData(), SecondHalfReadData.Num()));

		for (int32 i = 0; i < SecondHalfSize; ++i)
		{
			CHECK_EQUALS(TEXT("Mismatch in data after ShrinkBuffers was called"), SecondHalfReadData[i], TestData[FirstHalfSize + i]);
		}
	}
	else
	{
		REQUIRE_MESSAGE(FString::Printf(TEXT("File failed to open file for reading: %s"), *TempFilename), false);
	}
}

#endif //WITH_TESTS
