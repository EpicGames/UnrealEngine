// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/Guid.h"
#include "Tests/TestHarnessAdapter.h"
#include "Misc/SecureHash.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(FGuidTest, "System::Core::Misc::Guid", "[ApplicationContextMask][SmokeFilter]")
{
	// Base36		12SRDE1NOQERISMPEF0H6KUU9
	// Base64		EjRWeIdlQyESNFZ4h2VDIQ
	// Hex			0x12345678 0x87654321  0x12345678 0x87654321
	FGuid g = FGuid(305419896, 2271560481, 305419896, 2271560481);

	// string conversions
	CHECK_EQUALS(TEXT("String conversion (Default) must return EGuidFormats::Digits string"), g.ToString(), g.ToString(EGuidFormats::Digits));
	TestEqual<FString>(TEXT("String conversion (EGuidFormats::Digits)"), g.ToString(EGuidFormats::Digits), TEXT("12345678876543211234567887654321"));
	TestEqual<FString>(TEXT("String conversion (EGuidFormats::DigitsWithHyphens)"), g.ToString(EGuidFormats::DigitsWithHyphens), TEXT("12345678-8765-4321-1234-567887654321"));
	TestEqual<FString>(TEXT("String conversion (EGuidFormats::DigitsWithHyphensInBraces)"), g.ToString(EGuidFormats::DigitsWithHyphensInBraces), TEXT("{12345678-8765-4321-1234-567887654321}"));
	TestEqual<FString>(TEXT("String conversion (EGuidFormats::DigitsWithHyphensInParentheses)"), g.ToString(EGuidFormats::DigitsWithHyphensInParentheses), TEXT("(12345678-8765-4321-1234-567887654321)"));
	TestEqual<FString>(TEXT("String conversion (EGuidFormats::HexValuesInBraces)"), g.ToString(EGuidFormats::HexValuesInBraces), TEXT("{0x12345678,0x8765,0x4321,{0x12,0x34,0x56,0x78,0x87,0x65,0x43,0x21}}"));
	TestEqual<FString>(TEXT("String conversion (EGuidFormats::UniqueObjectGuid)"), g.ToString(EGuidFormats::UniqueObjectGuid), TEXT("12345678-87654321-12345678-87654321"));
	TestEqual<FString>(TEXT("String conversion (EGuidFormats::Short)"), g.ToString(EGuidFormats::Short), TEXT("EjRWeIdlQyESNFZ4h2VDIQ"));
	TestEqual<FString>(TEXT("String conversion (EGuidFormats::Base36Encoded)"), g.ToString(EGuidFormats::Base36Encoded), TEXT("12SRDE1NOQERISMPEF0H6KUU9"));

	// parsing valid strings (exact)
	FGuid g2_1;

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (EGuidFormats::Digits)"), FGuid::ParseExact(TEXT("12345678876543211234567887654321"), EGuidFormats::Digits, g2_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (EGuidFormats::Digits)"), g2_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (EGuidFormats::DigitsWithHyphens)"), FGuid::ParseExact(TEXT("12345678-8765-4321-1234-567887654321"), EGuidFormats::DigitsWithHyphens, g2_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (EGuidFormats::DigitsWithHyphens)"), g2_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (EGuidFormats::DigitsWithHyphensInBraces)"), FGuid::ParseExact(TEXT("{12345678-8765-4321-1234-567887654321}"), EGuidFormats::DigitsWithHyphensInBraces, g2_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (EGuidFormats::DigitsWithHyphensInBraces)"), g2_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (EGuidFormats::DigitsWithHyphensInParentheses)"), FGuid::ParseExact(TEXT("(12345678-8765-4321-1234-567887654321)"), EGuidFormats::DigitsWithHyphensInParentheses, g2_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (EGuidFormats::DigitsWithHyphensInParentheses)"), g2_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (EGuidFormats::HexValuesInBraces)"), FGuid::ParseExact(TEXT("{0x12345678,0x8765,0x4321,{0x12,0x34,0x56,0x78,0x87,0x65,0x43,0x21}}"), EGuidFormats::HexValuesInBraces, g2_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (EGuidFormats::HexValuesInBraces)"), g2_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (EGuidFormats::UniqueObjectGuid)"), FGuid::ParseExact(TEXT("12345678-87654321-12345678-87654321"), EGuidFormats::UniqueObjectGuid, g2_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (EGuidFormats::UniqueObjectGuid)"), g2_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (EGuidFormats::Short)"), FGuid::ParseExact(TEXT("EjRWeIdlQyESNFZ4h2VDIQ"), EGuidFormats::Short, g2_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (EGuidFormats::Short)"), g2_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (EGuidFormats::Base36Encoded)"), FGuid::ParseExact(TEXT("12SRDE1NOQERISMPEF0H6KUU9"), EGuidFormats::Base36Encoded, g2_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (EGuidFormats::Base36Encoded)"), g2_1, g);

	// parsing invalid strings (exact)


	// parsing valid strings (automatic)
	FGuid g3_1;

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (12345678876543211234567887654321)"), FGuid::Parse(TEXT("12345678876543211234567887654321"), g3_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (12345678876543211234567887654321)"), g3_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (12345678-8765-4321-1234-567887654321)"), FGuid::Parse(TEXT("12345678-8765-4321-1234-567887654321"), g3_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (12345678-8765-4321-1234-567887654321)"), g3_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed ({12345678-8765-4321-1234-567887654321})"), FGuid::Parse(TEXT("{12345678-8765-4321-1234-567887654321}"), g3_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed ({12345678-8765-4321-1234-567887654321})"), g3_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed ((12345678-8765-4321-1234-567887654321))"), FGuid::Parse(TEXT("(12345678-8765-4321-1234-567887654321)"), g3_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed ((12345678-8765-4321-1234-567887654321))"), g3_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed ({0x12345678,0x8765,0x4321,{0x12,0x34,0x56,0x78,0x87,0x65,0x43,0x21}})"), FGuid::Parse(TEXT("{0x12345678,0x8765,0x4321,{0x12,0x34,0x56,0x78,0x87,0x65,0x43,0x21}}"), g3_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed ({0x12345678,0x8765,0x4321,{0x12,0x34,0x56,0x78,0x87,0x65,0x43,0x21}})"), g3_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (12345678-87654321-12345678-87654321)"), FGuid::Parse(TEXT("12345678-87654321-12345678-87654321"), g3_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (12345678-87654321-12345678-87654321)"), g3_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (EjRWeIdlQyESNFZ4h2VDIQ)"), FGuid::Parse(TEXT("EjRWeIdlQyESNFZ4h2VDIQ"), g3_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (EjRWeIdlQyESNFZ4h2VDIQ)"), g3_1, g);

	CHECK_MESSAGE(TEXT("Parsing valid strings must succeed (12SRDE1NOQERISMPEF0H6KUU9)"), FGuid::Parse(TEXT("12SRDE1NOQERISMPEF0H6KUU9"), g3_1));
	CHECK_EQUALS(TEXT("Parsing valid strings must succeed (12SRDE1NOQERISMPEF0H6KUU9)"), g3_1, g);

	//parsing invalid strings (automatic)

	// GUID validation
	FGuid g4_1 = FGuid::NewGuid();

	CHECK_MESSAGE(TEXT("New GUIDs must be valid"), g4_1.IsValid());

	g4_1.Invalidate();

	CHECK_FALSE_MESSAGE(TEXT("Invalidated GUIDs must be invalid"), g4_1.IsValid());

	// GUID creation from MD5 hash
	FMD5 MD5;
	ANSICHAR String[] = "Construct Guid from MD5 hash";
	MD5.Update((uint8*)String, sizeof(String) - 1);
	FMD5Hash Hash;
	Hash.Set(MD5);
	CHECK_MESSAGE(TEXT("Expected hash must match"), LexToString(Hash) == TEXT("e500d989a5db77bc696e79ee785f4a69"));

	FGuid g5_1(MD5HashToGuid(Hash));
	CHECK_MESSAGE(TEXT("Constructing from MD5 hash must be valid"), g5_1.ToString() == TEXT("89d900e5bc77dba5ee796e69694a5f78"));

	// GUID combines
	const FGuid g6_0;
	const FGuid g6_1(TEXT("e500d989a5db77bc696e79ee785f4a69"));
	const FGuid g6_2(TEXT("89d900e5bc77dba5ee796e69694a5f78"));
	const FGuid g6_3(TEXT("43be1e3368afb993e98aaa574a6d550f"));

	CHECK_MESSAGE(TEXT("Combining 2 GUIDs"), FGuid::Combine(g6_1, g6_2) == g6_3);
	CHECK_MESSAGE(TEXT("Combining 2 invalid GUIDs"), !FGuid::CombineValid(g6_0, g6_0).IsValid());
	CHECK_MESSAGE(TEXT("Combining 1 valid GUID and 1 invalid GUID"), FGuid::CombineValid(g6_1, g6_0) == g6_1);
	CHECK_MESSAGE(TEXT("Combining 1 invalid GUID and 1 valid GUID"), FGuid::CombineValid(g6_0, g6_1) == g6_1);
	CHECK_MESSAGE(TEXT("Combining 2 valid GUIDs "), FGuid::CombineValid(g6_1, g6_2) == g6_3);
}

#endif //WITH_TESTS
