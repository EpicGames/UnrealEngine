// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Misc/MonotonicTime.h"

#include "Tests/TestHarnessAdapter.h"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE
{

TEST_CASE_NAMED(FMonotonicTimeSpanTest, "System::Core::Time::MonotonicTimeSpan", "[Core][Time][SmokeFilter]")
{
	SECTION("Constructors")
	{
		STATIC_CHECK(FMonotonicTimeSpan().ToSeconds() == 0.0);
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(0.0).ToSeconds() == 0.0);
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(123.0).ToSeconds() == 123.0);
		STATIC_CHECK(FMonotonicTimeSpan::Zero().ToSeconds() == 0.0);
	}

	SECTION("Comparison")
	{
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(0.0) == FMonotonicTimeSpan());
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(1.0) != FMonotonicTimeSpan());
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(0.0) <= FMonotonicTimeSpan());
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(0.0) >= FMonotonicTimeSpan());
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(0.0) < FMonotonicTimeSpan::FromSeconds(1.0));
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(1.0) > FMonotonicTimeSpan::FromSeconds(0.0));
	}

	SECTION("Infinity")
	{
		STATIC_CHECK(FMonotonicTimeSpan::Infinity().IsInfinity());
		STATIC_CHECK_FALSE(FMonotonicTimeSpan().IsInfinity());
		STATIC_CHECK_FALSE(FMonotonicTimeSpan::FromSeconds(123.0).IsInfinity());

		STATIC_CHECK(FMonotonicTimeSpan::Infinity() == FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimeSpan::Infinity() <= FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimeSpan::Infinity() >= FMonotonicTimeSpan::Infinity());

		STATIC_CHECK(FMonotonicTimeSpan::Infinity() >= FMonotonicTimeSpan());
		STATIC_CHECK(FMonotonicTimeSpan::Infinity() > FMonotonicTimeSpan());

		STATIC_CHECK(FMonotonicTimeSpan() != FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimeSpan() <= FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimeSpan() < FMonotonicTimeSpan::Infinity());
	}

	SECTION("Addition")
	{
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(234.0) + FMonotonicTimeSpan::FromSeconds(123.0) == FMonotonicTimeSpan::FromSeconds(357.0));
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(234.0) + (-FMonotonicTimeSpan::Infinity()) == -FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(234.0) + FMonotonicTimeSpan::Infinity() == FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimeSpan::Infinity() + FMonotonicTimeSpan::FromSeconds(1.0) == FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimeSpan::Infinity() + FMonotonicTimeSpan::Infinity() == FMonotonicTimeSpan::Infinity());
		STATIC_CHECK((-FMonotonicTimeSpan::Infinity()) + FMonotonicTimeSpan::FromSeconds(1.0) == -FMonotonicTimeSpan::Infinity());
		STATIC_CHECK((-FMonotonicTimeSpan::Infinity()) + (-FMonotonicTimeSpan::Infinity()) == -FMonotonicTimeSpan::Infinity());
		CHECK((FMonotonicTimeSpan::Infinity() + (-FMonotonicTimeSpan::Infinity())).IsNaN());
		CHECK(((-FMonotonicTimeSpan::Infinity()) + FMonotonicTimeSpan::Infinity()).IsNaN());
	}

	SECTION("Subtraction")
	{
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(234.0) - FMonotonicTimeSpan::FromSeconds(123.0) == FMonotonicTimeSpan::FromSeconds(111.0));
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(234.0) - (-FMonotonicTimeSpan::Infinity()) == FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimeSpan::FromSeconds(234.0) - FMonotonicTimeSpan::Infinity() == -FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimeSpan::Infinity() - FMonotonicTimeSpan::FromSeconds(1.0) == FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimeSpan::Infinity() - (-FMonotonicTimeSpan::Infinity()) == FMonotonicTimeSpan::Infinity());
		STATIC_CHECK((-FMonotonicTimeSpan::Infinity()) - FMonotonicTimeSpan::FromSeconds(1.0) == -FMonotonicTimeSpan::Infinity());
		STATIC_CHECK((-FMonotonicTimeSpan::Infinity()) - FMonotonicTimeSpan::Infinity() == -FMonotonicTimeSpan::Infinity());
		CHECK((FMonotonicTimeSpan::Infinity() - FMonotonicTimeSpan::Infinity()).IsNaN());
		CHECK(((-FMonotonicTimeSpan::Infinity()) - (-FMonotonicTimeSpan::Infinity())).IsNaN());
	}
}

TEST_CASE_NAMED(FMonotonicTimePointTest, "System::Core::Time::MonotonicTimePoint", "[Core][Time][SmokeFilter]")
{
	SECTION("Constructors")
	{
		STATIC_CHECK(FMonotonicTimePoint().ToSeconds() == 0.0);
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(0.0).ToSeconds() == 0.0);
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(123.0).ToSeconds() == 123.0);
	}

	SECTION("Comparison")
	{
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(0.0) == FMonotonicTimePoint());
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(1.0) != FMonotonicTimePoint());
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(0.0) <= FMonotonicTimePoint());
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(0.0) >= FMonotonicTimePoint());
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(0.0) < FMonotonicTimePoint::FromSeconds(1.0));
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(1.0) > FMonotonicTimePoint::FromSeconds(0.0));
	}

	SECTION("Infinity")
	{
		STATIC_CHECK(FMonotonicTimePoint::Infinity().IsInfinity());
		STATIC_CHECK_FALSE(FMonotonicTimePoint().IsInfinity());
		STATIC_CHECK_FALSE(FMonotonicTimePoint::FromSeconds(123.0).IsInfinity());

		STATIC_CHECK(FMonotonicTimePoint::Infinity() == FMonotonicTimePoint::Infinity());
		STATIC_CHECK(FMonotonicTimePoint::Infinity() <= FMonotonicTimePoint::Infinity());
		STATIC_CHECK(FMonotonicTimePoint::Infinity() >= FMonotonicTimePoint::Infinity());

		STATIC_CHECK(FMonotonicTimePoint::Infinity() >= FMonotonicTimePoint());
		STATIC_CHECK(FMonotonicTimePoint::Infinity() > FMonotonicTimePoint());

		STATIC_CHECK(FMonotonicTimePoint() != FMonotonicTimePoint::Infinity());
		STATIC_CHECK(FMonotonicTimePoint() <= FMonotonicTimePoint::Infinity());
		STATIC_CHECK(FMonotonicTimePoint() < FMonotonicTimePoint::Infinity());
	}

	SECTION("Addition")
	{
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(234.0) + FMonotonicTimeSpan::FromSeconds(123.0) == FMonotonicTimePoint::FromSeconds(357.0));
		STATIC_CHECK(FMonotonicTimePoint::Infinity() + FMonotonicTimeSpan::FromSeconds(1.0) == FMonotonicTimePoint::Infinity());
		STATIC_CHECK(FMonotonicTimePoint::Infinity() + FMonotonicTimeSpan::Infinity() == FMonotonicTimePoint::Infinity());
		CHECK((FMonotonicTimePoint::Infinity() + (-FMonotonicTimeSpan::Infinity())).IsNaN());
	}

	SECTION("Subtraction")
	{
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(234.0) - FMonotonicTimeSpan::FromSeconds(123.0) == FMonotonicTimePoint::FromSeconds(111.0));
		STATIC_CHECK(FMonotonicTimePoint::Infinity() - FMonotonicTimeSpan::FromSeconds(1.0) == FMonotonicTimePoint::Infinity());
		STATIC_CHECK(FMonotonicTimePoint::Infinity() - (-FMonotonicTimeSpan::Infinity()) == FMonotonicTimePoint::Infinity());
		CHECK((FMonotonicTimePoint::Infinity() - FMonotonicTimeSpan::Infinity()).IsNaN());
	}

	SECTION("Span")
	{
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(357.0) - FMonotonicTimePoint::FromSeconds(234.0) == FMonotonicTimeSpan::FromSeconds(123.0));
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(234.0) - FMonotonicTimePoint::FromSeconds(357.0) == FMonotonicTimeSpan::FromSeconds(-123.0));

		STATIC_CHECK(FMonotonicTimePoint::Infinity() - FMonotonicTimePoint::FromSeconds(123.0) == FMonotonicTimeSpan::Infinity());
		STATIC_CHECK(FMonotonicTimePoint::FromSeconds(123.0) - FMonotonicTimePoint::Infinity() == -FMonotonicTimeSpan::Infinity());
	}
}

} // namespace UE

UE_ENABLE_OPTIMIZATION_SHIP

#endif // WITH_TESTS
