// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"
#include <type_traits>
#if WITH_TESTS

#include "CoreTypes.h"
#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Math/Vector.h"
#include "Misc/LowLevelTestAdapter.h"
#include "Misc/NotNull.h"
#include "Misc/Optional.h"
#include "Templates/AlignmentTemplates.h"
#include "Tests/TestHarnessAdapter.h"

// Tests for traits and constexpr use of TOptional
namespace UE::Core::Private
{
	// TIsTOptional_V of optionals
	static_assert(TIsTOptional_V<      TOptional<float>>, "Expected TIsTOptional_V<TOptional> to be true");
	static_assert(TIsTOptional_V<const TOptional<float>>, "Expected TIsTOptional_V<const TOptional> to be true");

	// TIsTOptional_V of non-optionals (references are not optionals)
	static_assert(TIsTOptional_V<      TOptional<float>&>  == false, "Expected TIsTOptional_V<TOptional&> to be false");
	static_assert(TIsTOptional_V<const TOptional<float>&>  == false, "Expected TIsTOptional_V<const TOptional&> to be false");
	static_assert(TIsTOptional_V<      TOptional<float>&&> == false, "Expected TIsTOptional_V<TOptional&&> to be false");
	static_assert(TIsTOptional_V<      bool>               == false, "Expected TIsTOptional_V<non-TOptional> to be false");

	// Ensure that TOptional has a trivial destructor if the inner type does
	static_assert( std::is_trivially_destructible_v<TOptional<int>>);
	static_assert( std::is_trivially_destructible_v<TOptional<FVector>>);
	static_assert(!std::is_trivially_destructible_v<FString>);
	static_assert(!std::is_trivially_destructible_v<TOptional<FString>>);
	static_assert(!std::is_trivially_destructible_v<TOptional<FText>>);

	template<bool bIntrusive, bool bTrackDestruction>
	struct TTestValue
	{
		static inline int32 DestructorCount = 0;
		static constexpr inline bool bHasIntrusiveUnsetOptionalState = bIntrusive;
		using IntrusiveUnsetOptionalStateType = TTestValue<true, bTrackDestruction>;

		constexpr TTestValue(FIntrusiveUnsetOptionalState) requires bIntrusive
		: Value(0)
		{
		}
		constexpr TTestValue(int32 InValue)
		: Value(InValue)
		{
			static_assert(!bIntrusive || sizeof(TOptional<TTestValue>) == sizeof(TTestValue), "Expected intrusive optional to be the same size as the inner type");
			static_assert(bIntrusive || sizeof(TOptional<TTestValue>) > sizeof(TTestValue), "Expected non-intrusive optional to be larger than the inner type");
			static_assert(bIntrusive || sizeof(TOptional<TTestValue>) <= Align(sizeof(TTestValue) + 1, alignof(TTestValue)),
				"Expected non-intrusive optional to be larger than the inner type by at most the alignment of the inner type");
		}
		constexpr TTestValue(const TTestValue&) = default;
		constexpr TTestValue& operator=(const TTestValue&) = default;
		constexpr TTestValue(TTestValue&&) = default;
		constexpr TTestValue& operator=(TTestValue&&) = default;

		// Explicit destructor so not trivially destructible, but constexpr-destructible
		constexpr ~TTestValue()
		{
			if constexpr (bTrackDestruction)
			{
				++DestructorCount;
			}
		}

		[[nodiscard]] constexpr bool operator==(FIntrusiveUnsetOptionalState) const requires bIntrusive
		{
			return Value == 0;
		}

		[[nodiscard]] constexpr bool operator==(const TTestValue&) const = default;

		int32 Value;
	};

	template<bool bTrackDestruction = false>
	using TIntrusive = TTestValue<true, bTrackDestruction>;
	template<bool bTrackDestruction = false>
	using TNonIntrusive = TTestValue<false, bTrackDestruction>;
	struct FDerivedIntrusive : TIntrusive<> {};

	static_assert(HasIntrusiveUnsetOptionalState<TIntrusive<>>());
	static_assert(!HasIntrusiveUnsetOptionalState<FDerivedIntrusive>()); // a class which derives from another with an intrusive unset state should not be assumed to have an intrusive unset state itself.
	static_assert(!HasIntrusiveUnsetOptionalState<TNonIntrusive<>>());

	// Ensure TNotNull honors the intrusive unset state of the thing it's wrapping.
	static_assert(HasIntrusiveUnsetOptionalState<TNotNull<TIntrusive<>>>());
	static_assert(!HasIntrusiveUnsetOptionalState<TNotNull<FDerivedIntrusive>>());
	static_assert(!HasIntrusiveUnsetOptionalState<TNotNull<TNonIntrusive<>>>());

	// Tests for constexpr use of optional
	template<typename InnerType>
	constexpr bool TestConstexprOptionalUse()
	{
		InnerType Value{32};
		TOptional<InnerType> O1; // Default construction
		TOptional<InnerType> O2{NullOpt}; // Explicit empty construction;
		TOptional<InnerType> O3(Value); // Copying inner type
		TOptional<InnerType> O4(Forward<InnerType>(InnerType(30))); // Moving inner type
		TOptional<InnerType> O5(InPlace, 29); // In-place construction
		if constexpr (false) // At present, copy and move construction are not constexpr-safe
		{
			TOptional<InnerType> O6(O3); // Copy construction
			TOptional<InnerType> O7(MoveTemp(O4)); // Move construction
		}
		bool bTestResult = (O1 == O2) && (O4 != O5);
		bTestResult = !O1.IsSet() && O3.IsSet() && bTestResult;
		const TOptional<InnerType> C1(InPlace, 33);
		const InnerType& I1 = C1.GetValue();
		O5.GetValue().Value++;
		O5->Value = C1->Value;
		bTestResult = ((*O5) == (*C1)) && bTestResult;
		const InnerType& I2 = O1.Get(Value);
		*O4.GetPtrOrNull() = *C1.GetPtrOrNull();

		return bTestResult;
	}

	static_assert(TestConstexprOptionalUse<TIntrusive<>>(), "Constexpr tests for TOptional<TIntrusive> failed");
	static_assert(TestConstexprOptionalUse<TNonIntrusive<>>(), "Constexpr tests for TOptional<TNonIntrusive> failed");
};

TEST_CASE_NAMED(FTOptionalTest_NonIntrusive, "System::Core::TOptional::NonIntrusive", "[Core][TOptional][SmokeFilter]")
{
	using FNonIntrusiveDestructible = UE::Core::Private::TNonIntrusive<true>;
	FNonIntrusiveDestructible::DestructorCount = 0;
	{ 
		TOptional<FNonIntrusiveDestructible> Unset;
	}
	CHECK_MESSAGE("Unset optional is not destroyed", FNonIntrusiveDestructible::DestructorCount == 0);
}

TEST_CASE_NAMED(FTOptionalTest_Intrusive, "System::Core::TOptional::Intrusive", "[Core][TOptional][SmokeFilter]")
{
	using FIntrusiveDestructible = UE::Core::Private::TIntrusive<true>;
	FIntrusiveDestructible::DestructorCount = 0;
	{ 
		TOptional<FIntrusiveDestructible> Unset;
	}
	CHECK_MESSAGE("Unset optional is not destroyed", FIntrusiveDestructible::DestructorCount == 0);
}

// Test that trivial copyability is transitive
static_assert(std::is_trivially_copyable_v<TOptional<bool>>                      == std::is_trivially_copyable_v<bool>);
static_assert(std::is_trivially_copyable_v<TOptional<int32>>                     == std::is_trivially_copyable_v<int32>);
static_assert(std::is_trivially_copyable_v<TOptional<double>>                    == std::is_trivially_copyable_v<double>);
static_assert(std::is_trivially_copyable_v<TOptional<FName>>                     == std::is_trivially_copyable_v<FName>);
static_assert(std::is_trivially_copyable_v<TOptional<FString*>>                  == std::is_trivially_copyable_v<FString*>);
static_assert(std::is_trivially_copyable_v<TOptional<TSharedPtr<int32>>>         == std::is_trivially_copyable_v<TSharedPtr<int32>>);
static_assert(std::is_trivially_copyable_v<TOptional<FString>>                   == std::is_trivially_copyable_v<FString>);
static_assert(std::is_trivially_copyable_v<TOptional<FVector>>                   == std::is_trivially_copyable_v<FVector>);
static_assert(std::is_trivially_copyable_v<TOptional<TArray<int32>>>             == std::is_trivially_copyable_v<TArray<int32>>);
static_assert(std::is_trivially_copyable_v<TOptional<TStaticArray<int32, 20>>>   == std::is_trivially_copyable_v<TStaticArray<int32, 20>>);
static_assert(std::is_trivially_copyable_v<TOptional<TStaticArray<FString, 20>>> == std::is_trivially_copyable_v<TStaticArray<FString, 20>>);

// Test that copyability traits are as expected.

// An optional of a trivial type should have trivial operations
namespace TestTrivialOptionalType
{
	using OptionalType = int32;

	static_assert(std::is_trivially_copy_constructible_v<TOptional<OptionalType>>);
	static_assert(std::is_trivially_copy_assignable_v<TOptional<OptionalType>>);

	static_assert(std::is_trivially_move_constructible_v<TOptional<OptionalType>>);
	static_assert(std::is_trivially_move_assignable_v<TOptional<OptionalType>>);

	static_assert(std::is_trivially_destructible_v<TOptional<OptionalType>>);
}
// An optional of a movable and copyable type should be movable and copyable but not have have trivial operations.
namespace TestMovableAndCopyableOptionalType
{
	using OptionalType = FString;

	static_assert(std::is_copy_constructible_v<TOptional<OptionalType>>);
	static_assert(std::is_copy_assignable_v<TOptional<OptionalType>>);
	static_assert(!std::is_trivially_copy_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_trivially_copy_assignable_v<TOptional<OptionalType>>);

	static_assert(std::is_move_constructible_v<TOptional<OptionalType>>);
	static_assert(std::is_move_assignable_v<TOptional<OptionalType>>);
	static_assert(!std::is_trivially_move_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_trivially_move_assignable_v<TOptional<OptionalType>>);

	static_assert(!std::is_trivially_destructible_v<TOptional<OptionalType>>);
}
// An optional of a movable-only type should be movable but not copyable.
namespace TestMovableOnlyOptionalType
{
	using OptionalType = TUniquePtr<int>;

	static_assert(!std::is_copy_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_copy_assignable_v<TOptional<OptionalType>>);

	static_assert(std::is_move_constructible_v<TOptional<OptionalType>>);
	static_assert(std::is_move_assignable_v<TOptional<OptionalType>>);
	static_assert(!std::is_trivially_move_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_trivially_move_assignable_v<TOptional<OptionalType>>);

	static_assert(!std::is_trivially_destructible_v<TOptional<OptionalType>>);
}
// An optional of a type which is neither copyable nor movable, but trivially destructible should be neither copyable nor movable, but trivially destructible.
namespace TestNonMovableTriviallyDestructibleType
{
	struct OptionalType
	{
		OptionalType(OptionalType&&) = delete;
		OptionalType(const OptionalType&) = delete;
		OptionalType& operator=(OptionalType&&) = delete;
		OptionalType& operator=(const OptionalType&) = delete;
		~OptionalType() = default;
	};

	static_assert(!std::is_copy_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_copy_assignable_v<TOptional<OptionalType>>);

	static_assert(!std::is_move_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_move_assignable_v<TOptional<OptionalType>>);

	static_assert(std::is_trivially_destructible_v<TOptional<OptionalType>>);
}
// An optional of a type which is neither copyable, movable nor trivially destructible should be neither copyable, movable nor trivially destructible.
namespace TestNonMovableNonTriviallyDestructibleType
{
	struct OptionalType
	{
		OptionalType(OptionalType&&) = delete;
		OptionalType(const OptionalType&) = delete;
		OptionalType& operator=(OptionalType&&) = delete;
		OptionalType& operator=(const OptionalType&) = delete;
		~OptionalType();
	};

	static_assert(!std::is_copy_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_copy_assignable_v<TOptional<OptionalType>>);

	static_assert(!std::is_move_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_move_assignable_v<TOptional<OptionalType>>);

	static_assert(!std::is_trivially_move_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_trivially_move_assignable_v<TOptional<OptionalType>>);

	static_assert(!std::is_trivially_destructible_v<OptionalType>);
	static_assert(!std::is_trivially_destructible_v<TOptional<OptionalType>>);
}
// An optional of a type which is only copy constructible but not copy assignable should be copyable *and* move assignable, because TOptional assignment destructs and reconstructs.
// However, it is not trivially copy assignable even if the type is trivially copy constructible.
namespace TestTriviallyCopyConstructibleOnlyOptionalType
{
	struct OptionalType
	{
		OptionalType(OptionalType&&) = default;
		OptionalType(const OptionalType&) = default;
		OptionalType& operator=(OptionalType&&) = delete;
		OptionalType& operator=(const OptionalType&) = delete;
		~OptionalType() = default;
	};

	static_assert(std::is_copy_constructible_v<TOptional<OptionalType>>);
	static_assert(std::is_copy_assignable_v<TOptional<OptionalType>>);
	static_assert(std::is_trivially_copy_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_trivially_copy_assignable_v<TOptional<OptionalType>>);

	static_assert(std::is_move_constructible_v<TOptional<OptionalType>>);
	static_assert(std::is_move_assignable_v<TOptional<OptionalType>>);
	static_assert(std::is_trivially_move_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_trivially_move_assignable_v<TOptional<OptionalType>>);

	static_assert(std::is_trivially_destructible_v<TOptional<OptionalType>>);
}
// Same as above but only movable
namespace TestTriviallyMoveConstructibleOnlyAndDestructibleOptionalType
{
	struct OptionalType
	{
		OptionalType(OptionalType&&) = default;
		OptionalType(const OptionalType&) = delete;
		OptionalType& operator=(OptionalType&&) = delete;
		OptionalType& operator=(const OptionalType&) = delete;
		~OptionalType() = default;
	};

	static_assert(!std::is_copy_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_copy_assignable_v<TOptional<OptionalType>>);

	static_assert(std::is_move_constructible_v<TOptional<OptionalType>>);
	static_assert(std::is_move_assignable_v<TOptional<OptionalType>>);
	static_assert(std::is_trivially_move_constructible_v<TOptional<OptionalType>>);
	static_assert(!std::is_trivially_move_assignable_v<TOptional<OptionalType>>);

	static_assert(std::is_trivially_destructible_v<TOptional<OptionalType>>);
}

#endif // WITH_TESTS