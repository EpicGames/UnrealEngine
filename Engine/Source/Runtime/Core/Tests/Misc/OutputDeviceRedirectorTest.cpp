// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/OutputDeviceRedirector.h"

#if WITH_TESTS
#include "CoreGlobals.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FOutputDeviceRedirectorTest, "Core::Logging::OutputDeviceRedirector", "[Core][EngineFilter]")
{
	bool bHasThread = GLog->TryStartDedicatedPrimaryThread();

	// Test a fence with a dedicated logging thread when available.
	if (bHasThread)
	{
		FOutputDeviceFence Fence = GLog->CreateFence();
		Fence.Wait();
	}

	if (bHasThread)
	{
		GLog->SetCurrentThreadAsPrimaryThread();
	}

	// Test a fence without a dedicated logging thread.
	{
		FOutputDeviceFence Fence = GLog->CreateFence();
		Fence.Wait();
	}

	if (bHasThread)
	{
		GLog->TryStartDedicatedPrimaryThread();
	}
}

#endif // WITH_TESTS
