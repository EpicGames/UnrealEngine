// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Misc/Parse.h"

#include "Tests/TestHarnessAdapter.h"

#include <catch2/generators/catch_generators.hpp>
#include <catch2/catch_approx.hpp>

TEST_CASE("Parse::Value::ToBuffer", "[Parse][Smoke]")
{
	TCHAR Buffer[256];

	SECTION("Basic Usage") 
	{
		const TCHAR* Line = TEXT("a=a1 b=b2 c=c3");

		CHECK(FParse::Value(Line, TEXT("a="), Buffer, 256));
		CHECK(FCString::Strcmp(Buffer, TEXT("a1")) == 0);

		CHECK(FParse::Value(Line, TEXT("b="), Buffer, 256));
		CHECK(FCString::Strcmp(Buffer, TEXT("b2")) == 0);

		CHECK(FParse::Value(Line, TEXT("c="), Buffer, 256));
		CHECK(FCString::Strcmp(Buffer, TEXT("c3")) == 0);

		CHECK(false == FParse::Value(Line, TEXT("not_there="), Buffer, 256));
		CHECK(Buffer[0] == TCHAR(0));
	}

	SECTION("Quoted Values")
	{
		CHECK(FParse::Value(TEXT("a=a1 b=\"value with a space, and commas\" c=c3"), TEXT("b="), Buffer, 256));
		CHECK(FCString::Strcmp(Buffer, TEXT("value with a space, and commas")) == 0);
	}

	SECTION("Value may (not)? have a delimiter")
	{
		const TCHAR* Line = TEXT("a=a1,a2");

		CHECK(FParse::Value(Line, TEXT("a="), Buffer, 256, true));
		CHECK(FCString::Strcmp(Buffer, TEXT("a1")) == 0);

		CHECK(FParse::Value(Line, TEXT("a="), Buffer, 256, false)); // false = don't stop on , or )
		CHECK(FCString::Strcmp(Buffer, TEXT("a1,a2")) == 0);
	}

	SECTION("Value may have spaces on its left")
	{
		CHECK(FParse::Value(TEXT("a=   value"), TEXT("a="), Buffer, 256));
		CHECK(FCString::Strcmp(Buffer, TEXT("value")) == 0);
	}

	SECTION("Value could be a key value pair")
	{
		CHECK(FParse::Value(TEXT("a=  b=value"), TEXT("a="), Buffer, 256));
		CHECK(FCString::Strcmp(Buffer, TEXT("b=value")) == 0);

		CHECK(FParse::Value(TEXT("a=  b=  value"), TEXT("a="), Buffer, 256));
		CHECK(FCString::Strcmp(Buffer, TEXT("b=")) == 0);
		CHECK(FParse::Value(TEXT("a=  b=  value"), TEXT("b="), Buffer, 256));
		CHECK(FCString::Strcmp(Buffer, TEXT("value")) == 0);
	}

	SECTION("Key may appear mutiple times")
	{
		const TCHAR* Line = TEXT("rep=a1 rep=\"b2\" rep=c3");
		const TCHAR* ExpectedResults[] = { TEXT("a1"), TEXT("b2"), TEXT("c3") };

		const TCHAR* Cursor = Line;
		for (int Loop = 0; Loop < 4; ++Loop)
		{
			CHECK(Cursor != nullptr);

			bool bFound = FParse::Value(Cursor, TEXT("rep="), Buffer, 256, true, &Cursor);

			if (Loop < 3) 
			{
				CHECK(bFound);
				CHECK(FCString::Strcmp(Buffer, ExpectedResults[Loop]) == 0);
			}
			else
			{
				CHECK(!bFound);
				CHECK(Buffer[0] == TCHAR(0));
				CHECK(Cursor == nullptr);
			}
		}
	}
	
	SECTION("Key may have no value, It is found but Value is empty")
	{
		CHECK(FParse::Value(TEXT("a=   "), TEXT("a="), Buffer, 256));
		CHECK(Buffer[0] == TCHAR(0));
	}

	SECTION("Key with unbalanced quote, It is found but Value is empty")
	{
		for (TCHAR& C : Buffer)
		{
			C = TCHAR('*');
		}
		CHECK(FParse::Value(TEXT("a=\"   "), TEXT("a="), Buffer, 256));
		CHECK(FCString::Strchr(Buffer, TCHAR('*')) == nullptr);
	}

	SECTION("Key may have no value, It is found but Value is empty")
	{
		CHECK(FParse::Value(TEXT("a=   "), TEXT("a="), Buffer, 256));
		CHECK(Buffer[0] == TCHAR(0));
	}

	SECTION("Output var sanity")
	{
		CHECK(false == FParse::Value(TEXT("a=   "), TEXT("a="), Buffer, 0));
	}
}

TEST_CASE("Parse::InitFromString", "[Parse][Smoke]")
{
	SECTION("FVector2D")
	{
		FVector2D Value(0.0, 1.0);
		FString Expected("X=0.000 Y=1.000");
		CHECK(Value.ToString() == Expected);

		// Back-and-forth conversion should work :
		FVector2D NewValue;
		CHECK(NewValue.InitFromString(Value.ToString()));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));

		// Permissive formatting should work :
		CHECK(NewValue.InitFromString(TEXT("X=0     ,Y=1.000000.2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Out-of-order parameters :
		CHECK(NewValue.InitFromString(TEXT("Y=1 X=0")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Various formats/delimiters :
		CHECK(NewValue.InitFromString(TEXT("X=.0;Y=1.000000")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=0A Y=1.000000A")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Missing value == 0 :
		CHECK(NewValue.InitFromString(TEXT("X= Y=1")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=Y=1")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=A Y=1")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Invalid formats : 
		CHECK(!NewValue.InitFromString(TEXT("XA= Y=1")));
		CHECK(!NewValue.InitFromString(TEXT("X=0Y=1")));
		// Missing component should yield an error :
		CHECK(!NewValue.InitFromString(TEXT("X=0")));
		CHECK(!NewValue.InitFromString(TEXT("X=0 A=2")));
	}

	SECTION("FVector")
	{
		FVector Value(0.0, 1.0, 2.0);
		FString Expected("X=0.000 Y=1.000 Z=2.000");
		CHECK(Value.ToString() == Expected);

		// Back-and-forth conversion should work :
		FVector NewValue;
		CHECK(NewValue.InitFromString(Value.ToString()));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));

		// Permissive formatting should work :
		CHECK(NewValue.InitFromString(TEXT("X=0     ,Y= 1.000000.2:Z=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Out-of-order parameters :
		CHECK(NewValue.InitFromString(TEXT("Y=1 Z=2 X=0")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Various formats/delimiters :
		CHECK(NewValue.InitFromString(TEXT("X=.0;Y=1.000000|Z=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=0A Y=1.000000A Z=2.")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Missing value == 0 :
		CHECK(NewValue.InitFromString(TEXT("X= Y=1 Z=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=Y=1 Z=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=A Y=1 Z=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Invalid formats : 
		CHECK(!NewValue.InitFromString(TEXT("XA= Y=1 Z=2")));
		CHECK(!NewValue.InitFromString(TEXT("X =0 Y=1 Z=2")));
		CHECK(!NewValue.InitFromString(TEXT("X=0Y=1Z=2")));
		// Missing component should yield an error :
		CHECK(!NewValue.InitFromString(TEXT("X=0 Y=1")));
		CHECK(!NewValue.InitFromString(TEXT("X=0 Y=1 A=2")));
	}

	SECTION("FVector4")
	{
		FVector4 Value(0.0, 1.0, 2.0, 3.0);
		FString Expected("X=0.000 Y=1.000 Z=2.000 W=3.000");
		CHECK(Value.ToString() == Expected);

		// Back-and-forth conversion should work :
		FVector4 NewValue;
		CHECK(NewValue.InitFromString(Value.ToString()));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));

		// Permissive formatting should work :
		CHECK(NewValue.InitFromString(TEXT("X=0     ,Y= 1.000000.2:Z=2 W= 3.")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Out-of-order parameters :
		CHECK(NewValue.InitFromString(TEXT("Y=1 Z=2 W=3 X=0")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Various formats/delimiters :
		CHECK(NewValue.InitFromString(TEXT("X=.0;Y=1.000000|Z=2:W=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=0A Y=1.000000A Z=2. W= 3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Missing value == 0 :
		CHECK(NewValue.InitFromString(TEXT("X= Y=1 Z=2 W=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=Y=1 Z=2 W=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=A Y=1 Z=2 W=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Invalid formats : 
		CHECK(!NewValue.InitFromString(TEXT("XA= Y=1 Z=2 W=3")));
		CHECK(!NewValue.InitFromString(TEXT("X=0Y=1Z=2W=3")));
		// Missing component should yield an error :
		CHECK(NewValue.InitFromString(TEXT("X=0 Y=1 Z=2"))); // W is optional for FVector4
		CHECK(!NewValue.InitFromString(TEXT("X=0 Y=1 W=3")));
		CHECK(!NewValue.InitFromString(TEXT("X=0 Y=1 A=2 W=3")));
	}

	SECTION("FQuat")
	{
		FQuat Value(0.0, 1.0, 2.0, 3.0);
		FString Expected("X=0.000000000 Y=1.000000000 Z=2.000000000 W=3.000000000");
		CHECK(Value.ToString() == Expected);

		// Back-and-forth conversion should work :
		FQuat NewValue;
		CHECK(NewValue.InitFromString(Value.ToString()));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));

		// Permissive formatting should work :
		CHECK(NewValue.InitFromString(TEXT("X=0     ,Y= 1.000000.2:Z=2 W= 3.")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Out-of-order parameters :
		CHECK(NewValue.InitFromString(TEXT("Y=1 Z=2 W=3 X=0")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Various formats/delimiters :
		CHECK(NewValue.InitFromString(TEXT("X=.0;Y=1.000000|Z=2:W=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=0A Y=1.000000A Z=2. W= 3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Missing value == 0 :
		CHECK(NewValue.InitFromString(TEXT("X= Y=1 Z=2 W=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=Y=1 Z=2 W=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("X=A Y=1 Z=2 W=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Invalid formats : 
		CHECK(!NewValue.InitFromString(TEXT("XA= Y=1 Z=2 W=3")));
		CHECK(!NewValue.InitFromString(TEXT("X=0Y=1Z=2W=3")));
		// Missing component should yield an error :
		CHECK(!NewValue.InitFromString(TEXT("X=0 Y=1 Z=2"))); 
		CHECK(!NewValue.InitFromString(TEXT("X=0 Y=1 W=3")));
		CHECK(!NewValue.InitFromString(TEXT("X=0 Y=1 A=2 W=3")));
	}

	SECTION("FLinearColor")
	{
		FLinearColor Value(0.0, 1.0, 2.0, 3.0);
		FString Expected("(R=0.000000,G=1.000000,B=2.000000,A=3.000000)");
		CHECK(Value.ToString() == Expected);

		// Back-and-forth conversion should work :
		FLinearColor NewValue;
		CHECK(NewValue.InitFromString(Value.ToString()));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));

		// Permissive formatting should work :
		CHECK(NewValue.InitFromString(TEXT("R=0     ,G= 1.000000.2:B=2 A= 3.")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Out-of-order parameters :
		CHECK(NewValue.InitFromString(TEXT("G=1 B=2 A=3 R=0")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Various formats/delimiters :
		CHECK(NewValue.InitFromString(TEXT("R=.0;G=1.000000|B=2:A=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("R=0A G=1.000000A B=2. A= 3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Missing value == 0 :
		CHECK(NewValue.InitFromString(TEXT("R= G=1 B=2 A=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("R=G=1 B=2 A=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("R=A G=1 B=2 A=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Invalid formats : 
		CHECK(!NewValue.InitFromString(TEXT("RA= G=1 B=2 A=3")));
		CHECK(!NewValue.InitFromString(TEXT("R=0G=1B=2W=3")));
		CHECK(!NewValue.InitFromString(TEXT("R =0 G=1 B=2 W=3")));
		// Missing component should yield an error :
		CHECK(NewValue.InitFromString(TEXT("R=0 G=1 B=2"))); // A is optional for FLinearColor
		CHECK(!NewValue.InitFromString(TEXT("R=0 G=1 A=3")));
		CHECK(!NewValue.InitFromString(TEXT("R=0 G=1 A=2 A=3")));
	}

	SECTION("FColor")
	{
		FColor Value(0, 1, 2, 3);
		FString Expected("(R=0,G=1,B=2,A=3)");
		CHECK(Value.ToString() == Expected);

		// Back-and-forth conversion should work :
		FColor NewValue;
		CHECK(NewValue.InitFromString(Value.ToString()));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);

		// Permissive formatting should work :
		CHECK(NewValue.InitFromString(TEXT("R=0     ,G= 1.000000.2:B=2 A= 3.")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Out-of-order parameters :
		CHECK(NewValue.InitFromString(TEXT("G=1 B=2 A=3 R=0")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Various formats/delimiters :
		CHECK(NewValue.InitFromString(TEXT("R=0;G=1|B=2:A=3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		CHECK(NewValue.InitFromString(TEXT("R=0A G=1A B=2 A= 3")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Missing value is invalid for FColor :
		CHECK(!NewValue.InitFromString(TEXT("R= G=1 B=2 A=3")));
		CHECK(!NewValue.InitFromString(TEXT("R=G=1 B=2 A=3")));
		CHECK(!NewValue.InitFromString(TEXT("R=A G=1 B=2 A=3")));
		// Invalid formats : 
		CHECK(!NewValue.InitFromString(TEXT("RA= G=1 B=2 A=3")));
		CHECK(!NewValue.InitFromString(TEXT("R=0G=1B=2A=3")));
		CHECK(!NewValue.InitFromString(TEXT("R =0 G=1 B=2 A=3")));
		CHECK(!NewValue.InitFromString(TEXT("R=.0 G=1 B=2 A=3")));
		// Missing component should yield an error :
		CHECK(NewValue.InitFromString(TEXT("R=0 G=1 B=2"))); // A is optional for FColor
		CHECK(!NewValue.InitFromString(TEXT("R=0 G=1 A=3")));
		CHECK(!NewValue.InitFromString(TEXT("R=0 G=1 A=2 A=3")));
	}

	SECTION("FRotator")
	{
		FRotator Value(0.0, 1.0, 2.0);
		FString Expected("P=0.000000 Y=1.000000 R=2.000000");
		CHECK(Value.ToString() == Expected);

		// Back-and-forth conversion should work :
		FRotator NewValue;
		CHECK(NewValue.InitFromString(Value.ToString()));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));

		// Permissive formatting should work :
		CHECK(NewValue.InitFromString(TEXT("P=0     ,Y= 1.000000.2:R=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Out-of-order parameters :
		CHECK(NewValue.InitFromString(TEXT("Y=1 R=2 P=0")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Various formats/delimiters :
		CHECK(NewValue.InitFromString(TEXT("P=.0;Y=1.000000|R=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("P=0A Y=1.000000A R=2.")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Missing value == 0 :
		CHECK(NewValue.InitFromString(TEXT("P= Y=1 R=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("P=Y=1 R=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		CHECK(NewValue.InitFromString(TEXT("P=A Y=1 R=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue.Equals(Value));
		// Invalid formats : 
		CHECK(!NewValue.InitFromString(TEXT("PA= Y=1 R=2")));
		CHECK(!NewValue.InitFromString(TEXT("P=0Y=1R=2")));
		// Missing component should yield an error :
		CHECK(!NewValue.InitFromString(TEXT("P=0 Y=1")));
		CHECK(!NewValue.InitFromString(TEXT("P=0 Y=1 A=2")));
	}

	SECTION("FIntPoint")
	{
		FIntPoint Value(0, 1);
		FString Expected("X=0 Y=1");
		CHECK(Value.ToString() == Expected);

		// Back-and-forth conversion should work :
		FIntPoint NewValue;
		CHECK(NewValue.InitFromString(Value.ToString()));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);

		// Permissive formatting should work :
		CHECK(NewValue.InitFromString(TEXT("X=0     ,Y= 1.000000.2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Out-of-order parameters :
		CHECK(NewValue.InitFromString(TEXT("Y=1 X=0")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Various formats/delimiters :
		CHECK(NewValue.InitFromString(TEXT("X=.0;Y=1")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		CHECK(NewValue.InitFromString(TEXT("|X=0:Y=1")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		CHECK(NewValue.InitFromString(TEXT("X=0A Y= 1")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Missing value == 0 :
		CHECK(NewValue.InitFromString(TEXT("X= Y=1")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		CHECK(NewValue.InitFromString(TEXT("X=Y=1")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		CHECK(NewValue.InitFromString(TEXT("X=A Y=1")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Invalid formats : 
		CHECK(!NewValue.InitFromString(TEXT("RA= Y=1")));
		CHECK(!NewValue.InitFromString(TEXT("X=0Y=1")));
		CHECK(!NewValue.InitFromString(TEXT("X =0 Y=1")));
		// Missing component should yield an error :
		CHECK(!NewValue.InitFromString(TEXT("X=0")));
		CHECK(!NewValue.InitFromString(TEXT("X=0 A=1")));
	}

	SECTION("FIntVector")
	{
		FIntVector Value(0, 1, 2);
		FString Expected("X=0 Y=1 Z=2");
		CHECK(Value.ToString() == Expected);

		// Back-and-forth conversion should work :
		FIntVector NewValue;
		CHECK(NewValue.InitFromString(Value.ToString()));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);

		// Permissive formatting should work :
		CHECK(NewValue.InitFromString(TEXT("X=0     ,Y= 1.000000.2:Z=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Out-of-order parameters :
		CHECK(NewValue.InitFromString(TEXT("Y=1 Z=2 X=0")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Various formats/delimiters :
		CHECK(NewValue.InitFromString(TEXT("X=.0;Y=1|Z=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		CHECK(NewValue.InitFromString(TEXT("X=0A Y=1A Z= 2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Missing value == 0:
		CHECK(NewValue.InitFromString(TEXT("X= Y=1 Z=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		CHECK(NewValue.InitFromString(TEXT("X=Y=1 Z=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		CHECK(NewValue.InitFromString(TEXT("X=A Y=1 Z=2")));
		CHECK_MESSAGE(*FString::Printf(TEXT("Value:%s, Expected:%s"), *NewValue.ToString(), *Value.ToString()), NewValue == Value);
		// Invalid formats : 
		CHECK(!NewValue.InitFromString(TEXT("XA= Y=1 Z=2")));
		CHECK(!NewValue.InitFromString(TEXT("X=0Y=1Z=2")));
		CHECK(!NewValue.InitFromString(TEXT("X =0 Y=1 Z=2")));
		// Missing component should yield an error :
		CHECK(!NewValue.InitFromString(TEXT("X=0 Y=1")));
		CHECK(!NewValue.InitFromString(TEXT("X=0 Y=1 A=2")));
	}
}

TEST_CASE("Parse::GrammaredCLIParse::Callback", "[Smoke]")
{
	struct StringKeyValue {
		const TCHAR* Key;
		const TCHAR* Value;
	};

	SECTION("ExpectedPass")
	{
		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, std::vector<StringKeyValue>>(
			{
				{ TEXT("basic_ident"),	{ {TEXT("basic_ident"), nullptr} } },
				{ TEXT("-one_dash"),	{ {TEXT("-one_dash"), nullptr} } },
				{ TEXT("--two_dash"),	{ {TEXT("--two_dash"), nullptr} } },
				{ TEXT("/slash"),		{ {TEXT("/slash"), nullptr} } },
				{ TEXT("key=value"),	{ {TEXT("key"), TEXT("value")} } },
				{ TEXT("key.with.dots=value"),	{ {TEXT("key.with.dots"), TEXT("value")} } },
				{ TEXT("-key=value"),	{ {TEXT("-key"), TEXT("value")} } },
				{ TEXT("-key=\"value\""), { {TEXT("-key"), TEXT("\"value\"")} } },
				{ TEXT("-key=111"),		{ {TEXT("-key"), TEXT("111")} } },
				{ TEXT("-key=111."),	{ {TEXT("-key"), TEXT("111.")} } },
				{ TEXT("-key=111.222"),	{ {TEXT("-key"), TEXT("111.222")} } },
				{ TEXT("-key=-111"),	{ {TEXT("-key"), TEXT("-111")} } },
				{ TEXT("-key=-111.22"),	{ {TEXT("-key"), TEXT("-111.22")} } },
				{ TEXT("-key=../../some+dir\\text-file.txt"),	{ {TEXT("-key"), TEXT("../../some+dir\\text-file.txt")} } },
				{ TEXT("-key=c:\\log.txt"),	{ {TEXT("-key"), TEXT("c:\\log.txt")} } },
				{ TEXT("-token=00aabbcc99"),	{ {TEXT("-token"), TEXT("00aabbcc99")} } },
				{ TEXT("-token=\"00aab bcc99\""),	{ {TEXT("-token"), TEXT("\"00aab bcc99\"")} } },
				{ TEXT("a -b --c d=e"),	{ {TEXT("a"), nullptr},
										  {TEXT("-b"), nullptr},
										  {TEXT("--c"), nullptr},
										  {TEXT("d"), TEXT("e")} } },
				{ TEXT("a \"-b --c\" d=e"),	{ {TEXT("a"), nullptr},
											  {TEXT("-b"), nullptr},
											  {TEXT("--c"), nullptr},
											  {TEXT("d"), TEXT("e")} } },
				{ TEXT("\"a -b --c d=e\""),	{ {TEXT("a"), nullptr},
											  {TEXT("-b"), nullptr},
											  {TEXT("--c"), nullptr},
											  {TEXT("d"), TEXT("e")} } },
				{ TEXT("    leading_space"), { {TEXT("leading_space"), nullptr} } },
				{ TEXT("trailing_space   "), { {TEXT("trailing_space"), nullptr} } },
			}));
		size_t CallbackCalledCount = 0;
		// NOTE: I'm making a Ref to a Structured binding var because CLANG
		// has issues when trying to use them in lambdas.
		// https://github.com/llvm/llvm-project/issues/48582
		const std::vector<StringKeyValue>& ExpectedResultsRef = ExpectedResults;
		auto CallBack = [&CallbackCalledCount, &ExpectedResultsRef](FStringView Key, FStringView Value)
		{
			REQUIRE(CallbackCalledCount < ExpectedResultsRef.size());
			CHECK(Key == FStringView{ ExpectedResultsRef[CallbackCalledCount].Key });
			CHECK(Value == FStringView{ ExpectedResultsRef[CallbackCalledCount].Value });
			++CallbackCalledCount;
		};

		INFO("ExpectedPass " << FStringView{ Input });
		FParse::FGrammarBasedParseResult Result = FParse::GrammarBasedCLIParse(Input, CallBack);

		CHECK(CallbackCalledCount == ExpectedResults.size());
		CHECK(Result.ErrorCode == FParse::EGrammarBasedParseErrorCode::Succeeded);
	}

	SECTION("Quoted commands may be dissallowed, if so gives an error code.")
	{
		auto [Input, ExpectedErrorCode, ExpectedErrorAt, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, FParse::EGrammarBasedParseErrorCode, size_t, std::vector<StringKeyValue>>(
			{
				{ TEXT("a \"-b --c\" d=e"), FParse::EGrammarBasedParseErrorCode::DisallowedQuotedCommand, 2, { {TEXT("a"), nullptr} } },
			}));
		size_t CallbackCalledCount = 0;
		const std::vector<StringKeyValue>& ExpectedResultsRef = ExpectedResults;
		auto CallBack = [&CallbackCalledCount, &ExpectedResultsRef](FStringView Key, FStringView Value)
		{
			REQUIRE(CallbackCalledCount < ExpectedResultsRef.size());
			CHECK(Key == FStringView{ ExpectedResultsRef[CallbackCalledCount].Key });
			CHECK(Value == FStringView{ ExpectedResultsRef[CallbackCalledCount].Value });
			++CallbackCalledCount;
		};

		FParse::FGrammarBasedParseResult Result = FParse::GrammarBasedCLIParse(Input, CallBack, FParse::EGrammarBasedParseFlags::None);

		CHECK(CallbackCalledCount == ExpectedResults.size());
		CHECK(Result.ErrorCode == ExpectedErrorCode);
		CHECK(Result.At == Input + ExpectedErrorAt);
	}

	SECTION("Expected Fail cases")
	{
		auto [Input, ExpectedErrorCode, ExpectedErrorAt, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, FParse::EGrammarBasedParseErrorCode, size_t, std::vector<StringKeyValue>>(
			{
					{ TEXT("-a \"-b"), FParse::EGrammarBasedParseErrorCode::UnBalancedQuote, 3, { {TEXT("-a"), nullptr},
																								{TEXT("-b"), nullptr}} },
					{ TEXT("-a=\"unbalanced_quote_value"), FParse::EGrammarBasedParseErrorCode::UnBalancedQuote, 3, { } },
			}));
		size_t CallbackCalledCount = 0;
		const std::vector<StringKeyValue>& ExpectedResultsRef = ExpectedResults;
		auto CallBack = [&CallbackCalledCount, &ExpectedResultsRef](FStringView Key, FStringView Value)
		{
			REQUIRE(CallbackCalledCount < ExpectedResultsRef.size());
			CHECK(Key == FStringView{ ExpectedResultsRef[CallbackCalledCount].Key });
			CHECK(Value == FStringView{ ExpectedResultsRef[CallbackCalledCount].Value });
			++CallbackCalledCount;
		};

		FParse::FGrammarBasedParseResult Result = FParse::GrammarBasedCLIParse(Input, CallBack);

		CHECK(CallbackCalledCount == ExpectedResults.size());
		CHECK(Result.ErrorCode == ExpectedErrorCode);
		CHECK(Result.At == Input + ExpectedErrorAt);
	}
}

TEST_CASE("Parse::Token", "[Parse][Token][Smoke]")
{
	const int32 BufferLen = 256;
	TCHAR Buffer[BufferLen];

	SECTION("Space Delimited")
	{
		const TCHAR* Line = TEXT("a=a1 b=b2 c=c3");
		FParse::Token(Line, Buffer, BufferLen, false);
		CHECK(FCString::Strcmp(Buffer, TEXT("a=a1"))==0);
		CHECK_MESSAGE(TEXT("FParse::Token unexpectedly consumed trailing whitespace"), Line[0] == TEXT(' '));
		FParse::Token(Line, Buffer, BufferLen, false);
		CHECK(FCString::Strcmp(Buffer, TEXT("b=b2")) == 0);
		CHECK_MESSAGE(TEXT("FParse::Token unexpectedly consumed trailing whitespace"), Line[0] == TEXT(' '));
	}

	SECTION("Custom Delimiter")
	{
		const TCHAR* Line = TEXT("-ini:EditorPerProjectUserSettings:[/Script/Project.Setting]:GameFeaturePluginActiveProfile=\"My Default\"");
		FParse::Token(Line, Buffer, BufferLen, false, TEXT(':'));
		CHECK(FCString::Strcmp(Buffer, TEXT("-ini")) == 0);
		FParse::Token(Line, Buffer, BufferLen, false, TEXT(':'));
		CHECK(FCString::Strcmp(Buffer, TEXT("EditorPerProjectUserSettings")) == 0);
		FParse::Token(Line, Buffer, BufferLen, false, TEXT(':'));
		CHECK(FCString::Strcmp(Buffer, TEXT("[/Script/Project.Setting]")) == 0);
		FParse::Token(Line, Buffer, BufferLen, false, TEXT(':'));
		CHECK(FCString::Strcmp(Buffer, TEXT("GameFeaturePluginActiveProfile=\"My Default\"")) == 0);
	}

	SECTION("Leading/Trailing Delimiters")
	{
		{
			const TCHAR* Line = TEXT(":::Foo::Bar::");
			FParse::Token(Line, Buffer, BufferLen, false, TEXT(':'));
			CHECK(FCString::Strcmp(Buffer, TEXT("Foo")) == 0);
			FParse::Token(Line, Buffer, BufferLen, false, TEXT(':'));
			CHECK(FCString::Strcmp(Buffer, TEXT("Bar")) == 0);
			CHECK(FParse::Token(Line, Buffer, BufferLen, false, TEXT(':')) == false);
		}
		{
			const TCHAR* Line = TEXT("   Foo  Bar  ");
			FParse::Token(Line, Buffer, BufferLen, false);
			CHECK(FCString::Strcmp(Buffer, TEXT("Foo")) == 0);
			FParse::Token(Line, Buffer, BufferLen, false);
			CHECK(FCString::Strcmp(Buffer, TEXT("Bar")) == 0);
			CHECK(FParse::Token(Line, Buffer, BufferLen, false) == false);
		}
	}
}


TEST_CASE("Parse::Value::Numbers", "[Smoke]")
{
	SECTION("Int8")
	{
		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, int8>(
			{
				// raw
				{ TEXT("a=0"), (int8)0 },
				{ TEXT("a=42"), (int8)42 },
				{ TEXT("a=127"), (int8)127 },
				{ TEXT("a=-1"), (int8)-1 },
				{ TEXT("a=-128"), (int8)-128 },
				// quoted
				{ TEXT("a=\"0\""), (int8)0 },
				{ TEXT("a=\"42\""), (int8)42 },
				{ TEXT("a=\"127\""), (int8)127 },
				{ TEXT("a=\"-1\""), (int8)-1 },
				{ TEXT("a=\"-128\""), (int8)-128 },
				// broken quotes takes the first number found
				{ TEXT("a=123\"456\""), (int8)123 },
				{ TEXT("a=\"123\"456"), (int8)123 },
			}));

		int8 Result;
		CHECK(FParse::Value(Input, TEXT("a="), Result));
		CHECK(Result == ExpectedResults);
	}

	SECTION("UInt8")
	{
		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, uint8>(
			{
				// raw
				{ TEXT("a=0"), (uint8)0 },
				{ TEXT("a=42"), (uint8)42 },
				{ TEXT("a=127"), (uint8)127 },
				{ TEXT("a=255"), (uint8)255 },
				// quoted
				{ TEXT("a=\"0\""), (uint8)0 },
				{ TEXT("a=\"42\""), (uint8)42 },
				{ TEXT("a=\"127\""), (uint8)127 },
				{ TEXT("a=\"255\""), (uint8)255 },
				// broken quotes takes the first number found
				{ TEXT("a=123\"456\""), (uint8)123 },
				{ TEXT("a=\"123\"456"), (uint8)123 },
			}));

		uint8 Result;
		CHECK(FParse::Value(Input, TEXT("a="), Result));
		CHECK(Result == ExpectedResults);
	}
	
	SECTION("Int16")
	{
		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, int16>(
			{
				// raw
				{ TEXT("a=0"), (int16)0 },
				{ TEXT("a=42"), (int16)42 },
				{ TEXT("a=32767"), (int16)32767 },
				{ TEXT("a=-1"), (int16)-1 },
				{ TEXT("a=-32768"), (int16)-32768 },
				// quoted
				{ TEXT("a=\"0\""), (int16)0 },
				{ TEXT("a=\"42\""), (int16)42 },
				{ TEXT("a=\"32767\""), (int16)32767 },
				{ TEXT("a=\"-1\""), (int16)-1 },
				{ TEXT("a=\"-32768\""), (int16)-32768 },
				// broken quotes takes the first number found
				{ TEXT("a=123\"456\""), (int16)123 },
				{ TEXT("a=\"123\"456"), (int16)123 },
			}));

		int16 Result;
		CHECK(FParse::Value(Input, TEXT("a="), Result));
		CHECK(Result == ExpectedResults);
	}

	SECTION("UInt16")
	{
		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, uint16>(
			{
				// raw
				{ TEXT("a=0"), (uint16)0 },
				{ TEXT("a=42"), (uint16)42 },
				{ TEXT("a=32767"), (uint16)32767 },
				{ TEXT("a=65535"), (uint16)65535 },
				// quoted
				{ TEXT("a=\"0\""), (uint16)0 },
				{ TEXT("a=\"42\""), (uint16)42 },
				{ TEXT("a=\"32767\""), (uint16)32767 },
				{ TEXT("a=\"65535\""), (uint16)65535 },
				// broken quotes takes the first number found
				{ TEXT("a=123\"456\""), (uint16)123 },
				{ TEXT("a=\"123\"456"), (uint16)123 },
			}));

		uint16 Result;
		CHECK(FParse::Value(Input, TEXT("a="), Result));
		CHECK(Result == ExpectedResults);
	}

	SECTION("Int32")
	{
		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, int32>(
			{
				// raw
				{ TEXT("a=0"), (int32)0 },
				{ TEXT("a=42"), (int32)42 },
				{ TEXT("a=2147483647"), (int32)2147483647 },
				{ TEXT("a=-1"), (int32)-1 },
				{ TEXT("a=-2147483648"), (int32)-2147483648 },
				// quoted
				{ TEXT("a=\"0\""), (int32)0 },
				{ TEXT("a=\"42\""), (int32)42 },
				{ TEXT("a=\"2147483647\""), (int32)2147483647 },
				{ TEXT("a=\"-1\""), (int32)-1 },
				{ TEXT("a=\"-2147483648\""), (int32)-2147483648 },
				// broken quotes takes the first number found
				{ TEXT("a=123\"456\""), (int32)123 },
				{ TEXT("a=\"123\"456"), (int32)123 },
			}));

		int32 Result;
		CHECK(FParse::Value(Input, TEXT("a="), Result));
		CHECK(Result == ExpectedResults);
	}

	SECTION("UInt32")
	{
		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, uint32>(
			{
				// raw
				{ TEXT("a=0"), (uint32)0 },
				{ TEXT("a=42"), (uint32)42 },
				{ TEXT("a=2147483647"), (uint32)2147483647 },
				{ TEXT("a=4294967295"), (uint32)4294967295 },
				// quoted
				{ TEXT("a=\"0\""), (uint32)0 },
				{ TEXT("a=\"42\""), (uint32)42 },
				{ TEXT("a=\"2147483647\""), (uint32)2147483647 },
				{ TEXT("a=\"4294967295\""), (uint32)4294967295 },
				// broken quotes takes the first number found
				{ TEXT("a=123\"456\""), (uint32)123 },
				{ TEXT("a=\"123\"456"), (uint32)123 },
			}));

		uint32 Result;
		CHECK(FParse::Value(Input, TEXT("a="), Result));
		CHECK(Result == ExpectedResults);
	}

	SECTION("Int64")
	{
		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, int64>(
			{
				// raw
				{ TEXT("a=0"), 0ll },
				{ TEXT("a=42"), 42ll },
				{ TEXT("a=9223372036854775807"), 9223372036854775807ll },
				{ TEXT("a=-1"), -1 },
				{ TEXT("a=-9223372036854775807"), -9223372036854775807ll },
				// quoted
				{ TEXT("a=\"0\""), 0ll },
				{ TEXT("a=\"42\""), 42ll },
				{ TEXT("a=\"9223372036854775807\""), 9223372036854775807ll },
				{ TEXT("a=\"-1\""), -1ll },
				{ TEXT("a=\"-9223372036854775807\""), -9223372036854775807ll },
				// broken quotes takes the first number found
				{ TEXT("a=123\"456\""), 123ll },
				{ TEXT("a=\"123\"456"), 123ll },
			}));

		int64 Result;
		CHECK(FParse::Value(Input, TEXT("a="), Result));
		CHECK(Result == ExpectedResults);
	}

	SECTION("UInt64")
	{
		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, uint64>(
			{
				// raw
				{ TEXT("a=0"), 0ull },
				{ TEXT("a=42"), 42ull },
				{ TEXT("a=9223372036854775807"),   9223372036854775807ull },
				{ TEXT("a=18446744073709551615"), 18446744073709551615ull },
				// quoted
				{ TEXT("a=\"0\""), 0ull },
				{ TEXT("a=\"42\""), 42ull },
				{ TEXT("a=\"9223372036854775807\""), 9223372036854775807ull },
				{ TEXT("a=\"18446744073709551615\""), 18446744073709551615ull },
				// broken quotes takes the first number found
				{ TEXT("a=123\"456\""), 123ull },
				{ TEXT("a=\"123\"456"), 123ull },
			}));

		uint64 Result;
		CHECK(FParse::Value(Input, TEXT("a="), Result));
		CHECK(Result == ExpectedResults);
	}

	SECTION("float")
	{
		using Catch::Approx;

		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, float>(
			{
				// raw
				{ TEXT("a=0.0"), 0.0f },
				{ TEXT("a=0.5"), 0.5f },
				{ TEXT("a=1.0"), 1.0f },
				{ TEXT("a=42"), 42.0f },
				{ TEXT("a=3.1415"), 3.1415f },
				{ TEXT("a=-3.1415"), -3.1415f },
				{ TEXT("a=340282346638528859811704183484516925440.0"), 340282346638528859811704183484516925440.0f },
				{ TEXT("a=-340282346638528859811704183484516925440.0"), -340282346638528859811704183484516925440.0f },
				// quoted
				{ TEXT("a=\"0.0\""), 0.0f },
				{ TEXT("a=\"0.5\""), 0.5f },
				{ TEXT("a=\"1.0\""), 1.0f },
				{ TEXT("a=\"42\""), 42.0f },
				{ TEXT("a=\"3.1415\""), 3.1415f },
				{ TEXT("a=\"-3.1415\""), -3.1415f },
				{ TEXT("a=\"340282346638528859811704183484516925440.0\""), 340282346638528859811704183484516925440.0f },
				{ TEXT("a=\"-340282346638528859811704183484516925440.0\""), -340282346638528859811704183484516925440.0f },
				// broken quotes takes the first number found
				{ TEXT("a=123\"456\""), 123.0f },
				{ TEXT("a=\"123\"456"), 123.0f },
			}));

		float Result;
		CHECK(FParse::Value(Input, TEXT("a="), Result));
		CHECK(Result == Approx(ExpectedResults).margin(0.0001f));
	}

	SECTION("double")
	{
		using Catch::Approx;

		auto [Input, ExpectedResults] = GENERATE_COPY(table<const TCHAR*, double>(
			{
				// raw
				{ TEXT("a=0.0"), 0.0 },
				{ TEXT("a=0.5"), 0.5 },
				{ TEXT("a=1.0"), 1.0 },
				{ TEXT("a=42"), 42.0 },
				{ TEXT("a=3.1415"), 3.1415 },
				{ TEXT("a=-3.1415"), -3.1415 },
				{ TEXT("a=340282346638528859811704183484516925440.0"), 340282346638528859811704183484516925440.0 },
				{ TEXT("a=-340282346638528859811704183484516925440.0"), -340282346638528859811704183484516925440.0 },
				{ TEXT("a=" UE_STRINGIZE(DBL_MAX)), DBL_MAX },
				{ TEXT("a=-" UE_STRINGIZE(DBL_MAX)), -DBL_MAX },
				// quoted
				{ TEXT("a=\"0.0\""), 0.0 },
				{ TEXT("a=\"0.5\""), 0.5 },
				{ TEXT("a=\"1.0\""), 1.0 },
				{ TEXT("a=\"42\""), 42.0 },
				{ TEXT("a=\"3.1415\""), 3.1415 },
				{ TEXT("a=\"-3.1415\""), -3.1415 },
				{ TEXT("a=\"340282346638528859811704183484516925440.0\""), 340282346638528859811704183484516925440.0 },
				{ TEXT("a=\"-340282346638528859811704183484516925440.0\""), -340282346638528859811704183484516925440.0 },
				{ TEXT("a=\"" UE_STRINGIZE(DBL_MAX) "\""), DBL_MAX},
				{ TEXT("a=\"-" UE_STRINGIZE(DBL_MAX) "\""), -DBL_MAX },
				// broken quotes takes the first number found
				{ TEXT("a=123\"456\""), 123.0 },
				{ TEXT("a=\"123\"456"), 123.0 },
			}));

		double Result;
		CHECK(FParse::Value(Input, TEXT("a="), Result));
		CHECK(Result == Approx(ExpectedResults).margin(0.0001f));
	}
}

TEST_CASE("Parse::Line", "[Parse][Smoke]")
{
	SECTION("Line")
	{
		const TCHAR* Lines =
			TEXT("Line\n")
			TEXT("Line\r\n")
			TEXT("Line\n\n\n")
			TEXT("Line//Comment\n")
			TEXT("//Comment\n")
			TEXT("\"//Comment\"\n")
			TEXT("Line1|Line2\n")
			TEXT("\"Line1|Line2\"\n");

		FString ExpectedResult;
		FString Result;

		// exact == true
		const TCHAR* InputLines = Lines;
		const TCHAR** Input = &InputLines;
		// Line\n
		CHECK(FParse::Line(Input, Result, true)); CHECK(Result == TEXT("Line"));
		// Line\r\n
		CHECK(FParse::Line(Input, Result, true)); CHECK(Result == TEXT("Line"));
		// Line\n\n\n
		CHECK(FParse::Line(Input, Result, true)); CHECK(Result == TEXT("Line"));
		CHECK(FParse::Line(Input, Result, true)); CHECK(Result == TEXT(""));
		CHECK(FParse::Line(Input, Result, true)); CHECK(Result == TEXT(""));
		// Line//Comment\n
		CHECK(FParse::Line(Input, Result, true)); CHECK(Result == TEXT("Line//Comment"));
		// //Comment\n
		CHECK(FParse::Line(Input, Result, true)); CHECK(Result == TEXT("//Comment"));
		// \"//Comment\"\n
		CHECK(FParse::Line(Input, Result, true)); CHECK(Result == TEXT("\"//Comment\""));
		// Line1|Line2\n
		CHECK(FParse::Line(Input, Result, true)); CHECK(Result == TEXT("Line1|Line2"));
		// \"Line1|Line2\"\n
		CHECK(FParse::Line(Input, Result, true)); CHECK(Result == TEXT("\"Line1|Line2\""));

		// exact == false
		InputLines = Lines;
		Input = &InputLines;
		ExpectedResult = TEXT("Line");
		// Line\n
		CHECK(FParse::Line(Input, Result, false)); CHECK(Result == TEXT("Line"));
		// Line\r\n
		CHECK(FParse::Line(Input, Result, false)); CHECK(Result == TEXT("Line"));
		// Line\n\n\n
		CHECK(FParse::Line(Input, Result, false)); CHECK(Result == TEXT("Line"));
		// Line//Comment\n
		CHECK(FParse::Line(Input, Result, false)); CHECK(Result == TEXT("Line//Comment"));
		// //Comment\n
		CHECK(FParse::Line(Input, Result, false)); CHECK(Result == TEXT(""));
		// \"//Comment\"\n
		CHECK(FParse::Line(Input, Result, false)); CHECK(Result == TEXT("\"//Comment\""));
		// Line1|Line2\n
		CHECK(FParse::Line(Input, Result, false)); CHECK(Result == TEXT("Line1"));
		CHECK(FParse::Line(Input, Result, false)); CHECK(Result == TEXT("Line2"));
		// \"Line1|Line2\"\n
		CHECK(FParse::Line(Input, Result, false)); CHECK(Result == TEXT("\"Line1|Line2\""));
	}
	SECTION("LineExtended")
	{
		const TCHAR* Lines =
			TEXT("Line\r\n\r\n\n")
			TEXT("Line//Comment\n")
			TEXT("Line;Comment\n")
			TEXT("Line1\\\nLine2\n")
			TEXT("\"Line1\\\nLine2\"\n")
			TEXT("Line1|Line2\n")
			TEXT("\"Line1|Line2\"\n")
			TEXT("{Line1\nLine2}\n")
			TEXT("\"{Line1\nLine2}\"\n");

		FString ExpectedResult;
		FString Result;
		int LinesConsumed = 0;


		// OldDefaultMode (SwallowDoubleSlashComments | BreakOnPipe | AllowBracketedMultiline | AllowEscapedEOLMultiline | SwallowExtraEOLs)
		const TCHAR* InputLines = Lines;
		const TCHAR** Input = &InputLines;
		FParse::ELineExtendedFlags Flags = FParse::ELineExtendedFlags::OldDefaultMode;
		// Line\r\n\r\n\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line"));
		// Line//Comment\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line"));
		// Line;Comment\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line;Comment"));
		// Line1\\\nLine2\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line1 Line2"));
		// \"Line1\\\nLine2\"\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("\"Line1 Line2\""));
		// Line1|Line2\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line1"));
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line2"));
		// \"Line1|Line2\"\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("\"Line1|Line2\""));
		// {Line1\nLine2}\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line1 Line2"));
		// \"{Line1\nLine2}\"\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("\"{Line1"));
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line2}\""));

		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags) == false);

		// None
		InputLines = Lines;
		Input = &InputLines;
		Flags = FParse::ELineExtendedFlags::None;
		// Line\r\n\r\n\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line"));
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT(""));
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT(""));
		// Line//Comment\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line//Comment"));
		// Line;Comment\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line;Comment"));
		// Line1\\\nLine2\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line1\\"));
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line2"));
		// \"Line1\\\nLine2\"\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("\"Line1\\"));
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line2\""));
		// Line1|Line2\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line1|Line2"));
		// \"Line1|Line2\"\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("\"Line1|Line2\""));
		// {Line1\nLine2}\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("{Line1"));
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line2}"));
		// \"{Line1\nLine2}\"\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("\"{Line1"));
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line2}\""));

		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags) == false);

		// Config System mode
		InputLines = Lines;
		Input = &InputLines;
		Flags = FParse::ELineExtendedFlags::SwallowDoubleSlashComments | FParse::ELineExtendedFlags::SwallowSemicolonComments | 
			FParse::ELineExtendedFlags::AllowBracketedMultiline | FParse::ELineExtendedFlags::AllowEscapedEOLMultiline | 
			FParse::ELineExtendedFlags::SwallowExtraEOLs;
		// Line\r\n\r\n\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line"));
		// Line//Comment\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line"));
		// Line;Comment\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line"));
		// Line1\\\nLine2\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line1 Line2"));
		// \"Line1\\\nLine2\"\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("\"Line1 Line2\""));
		// Line1|Line2\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line1|Line2"));
		// \"Line1|Line2\"\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("\"Line1|Line2\""));
		// {Line1\nLine2}\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line1 Line2"));
		// \"{Line1\nLine2}\"\n
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("\"{Line1"));
		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags)); CHECK(Result == TEXT("Line2}\""));

		CHECK(FParse::LineExtended(Input, Result, LinesConsumed, Flags) == false);
	}
}
#endif
	
