// Copyright Epic Games, Inc. All Rights Reserved.


#if WITH_TESTS

#include "Misc/NamePermissionList.h"

#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FPathPermissionListExactMatchTests, "System::Core::Misc::PathPermissionList::ExactMatch", "[Core][Misc][PathPermissionList]")
{
    SECTION("Deny all")
    {
        FPathPermissionList List;
        List.AddDenyListAll("DenyAll");
        
        // Path is denied when everything is denied
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Game/Folder")));
        
        // Deny all takes precendence over specific allo
        List.AddAllowListItem("AllowSpecific", TEXTVIEW("/Game/Folder"));
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Game/Folder")));
        
        // Removing deny all allows the path again 
        List.UnregisterOwner("DenyAll");
        CHECK(List.PassesFilter(TEXTVIEW("/Game/Folder")));
    }
    
    SECTION("Deny list")
    {
        FPathPermissionList List;
        
        List.AddDenyListItem("Deny", TEXTVIEW("/Game/Secret"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Plugin1"));
        
        CHECK(List.PassesFilter(TEXTVIEW("/")));
        CHECK(List.PassesFilter(TEXTVIEW("/Game")));
        CHECK(List.PassesFilter(TEXTVIEW("/Game/Secret/ActuallyVisible")));
        CHECK(List.PassesFilter(TEXTVIEW("/Plugin1/AlsoVisible")));
        
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Plugin1")));
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Game/Secret")));
    }
    
    SECTION("Allow list")
    {
        FPathPermissionList List;
        
        List.AddAllowListItem("Allow", TEXTVIEW("/Game"));
        List.AddAllowListItem("Allow", TEXTVIEW("/Game/Public"));
        List.AddAllowListItem("Allow", TEXTVIEW("/Plugin1"));
        
        CHECK(List.PassesFilter(TEXTVIEW("/Game")));
        CHECK(List.PassesFilter(TEXTVIEW("/Game/Public")));
        CHECK(List.PassesFilter(TEXTVIEW("/Plugin1")));

        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/")));
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Game/Secret")));
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Plugin1/InPlugin")));
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Plugin2")));
    }

    SECTION("Mixed allow and deny list")
    {
        FPathPermissionList List;
        
        List.AddAllowListItem("Allow", TEXTVIEW("/Game"));
        List.AddAllowListItem("Allow", TEXTVIEW("/Game/Maps"));
        List.AddAllowListItem("Allow", TEXTVIEW("/Plugin1"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Plugin1"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Game"));
        
        CHECK(List.PassesFilter(TEXTVIEW("/Game/Maps")));

        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/")));
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Game")));
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Game/Characters")));
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Plugin1")));
        CHECK_FALSE(List.PassesFilter(TEXTVIEW("/Plugin2")));
    }
}

TEST_CASE_NAMED(FPathPermissionListCombineTest, "System::Core::Misc::PathPermissionList::Combine", "[Core][Misc][PathPermissionList]")
{
    SECTION("One deny all")
    {
        FPathPermissionList First;
        First.AddDenyListAll("Deny");
        FPathPermissionList Second;
        Second.AddAllowListItem("Allow", TEXTVIEW("/Game"));
        Second.AddAllowListItem("Allow", TEXTVIEW("/Game/Maps"));

        FPathPermissionList Combined = First.CombinePathFilters(Second);
        
        CHECK_FALSE(Combined.PassesFilter(TEXTVIEW("/Game")));
        CHECK_FALSE(Combined.PassesFilter(TEXTVIEW("/Game/Maps")));
        CHECK_FALSE(Combined.PassesStartsWithFilter(TEXTVIEW("/Game")));
        CHECK_FALSE(Combined.PassesStartsWithFilter(TEXTVIEW("/Game/Maps")));
        CHECK_FALSE(Combined.PassesStartsWithFilter(TEXTVIEW("/Game/Characters")));
        CHECK_FALSE(Combined.PassesStartsWithFilter(TEXTVIEW("/Plugin1")));
    }   
    
    SECTION("Parent path denied")
    {
        FPathPermissionList First;
        First.AddDenyListItem("Deny", "/Game/Maps");
        FPathPermissionList Second;
        Second.AddAllowListItem("Allow", TEXTVIEW("/Game"));
        Second.AddAllowListItem("Allow", TEXTVIEW("/Game/Maps/Desert"));

        FPathPermissionList Combined = First.CombinePathFilters(Second);
        CHECK(Combined.PassesFilter(TEXTVIEW("/Game")));
        CHECK_FALSE(Combined.PassesFilter(TEXTVIEW("/Game/Maps")));
        // "StartsWith" is used when combining to remove /Game/Maps/Desert from final allow list
        CHECK_FALSE(Combined.PassesFilter(TEXTVIEW("/Game/Maps/Desert"))); 
        
        CHECK(Combined.PassesStartsWithFilter(TEXTVIEW("/Game")));
        CHECK(Combined.PassesStartsWithFilter(TEXTVIEW("/Game/Characters")));

        CHECK_FALSE(Combined.PassesStartsWithFilter(TEXTVIEW("/Plugin1")));
        CHECK_FALSE(Combined.PassesStartsWithFilter(TEXTVIEW("/Game/Maps")));
        CHECK_FALSE(Combined.PassesStartsWithFilter(TEXTVIEW("/Game/Maps/Desert")));
        CHECK_FALSE(Combined.PassesStartsWithFilter(TEXTVIEW("/Game/Maps/Jungle")));
    }
}

TEST_CASE_NAMED(FPathPermissionListStartsWithTests, "System::Core::Misc::PathPermissionList::StartsWith", "[Core][Misc][PathPermissionList]")
{
    SECTION("Deny all")
    {
        FPathPermissionList List;
        List.AddDenyListAll("DenyAll");
        
        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Game")));
    }

    SECTION("Deny only")
    {
        FPathPermissionList List;
        List.AddDenyListItem("Deny", TEXTVIEW("/Game/Secret"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Game/Maps/Secret"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Plugin1/Secret"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Plugin2"));
        
        // Some children of /Game are blocked
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Game")));
        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Game/Secret")));
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Game/Characters")));
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Game/Maps")));
        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Game/Maps/Secret")));
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Game/Maps/Desert")));
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Plugin1")));
        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Plugin1/Secret")));
        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Plugin2")));
    }   
    
    SECTION("Allow only")
    {
        FPathPermissionList List;
        List.AddAllowListItem("Allow", TEXTVIEW("/Game/Maps"));
        List.AddAllowListItem("Allow", TEXTVIEW("/Plugin1"));
        
        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Game")));
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Game/Maps")));
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Plugin1")));
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Plugin1/Characters")));
        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Plugin2")));
    }

    SECTION("Mixed allow and deny")
    {
        FPathPermissionList List;
        List.AddAllowListItem("Allow", TEXTVIEW("/Game"));        
        List.AddAllowListItem("Allow", TEXTVIEW("/Plugin1"));
        List.AddAllowListItem("Allow", TEXTVIEW("/Plugin2/Public"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Game/Secret"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Game/Maps/Secret"));
        
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Game")));
        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Game/Secret")));
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Game/Maps")));
        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Game/Maps/Secret")));

        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Plugin1")));

        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Plugin2")));
        CHECK(List.PassesStartsWithFilter(TEXTVIEW("/Plugin2/Public")));
        CHECK_FALSE(List.PassesStartsWithFilter(TEXTVIEW("/Plugin2/Private")));
    }
}

TEST_CASE_NAMED(FPathPermissionListStartsWithRecursiveTests, "System::Core::Misc::PathPermissionList::StartsWithRecursive", "[Core][Misc][PathPermissionList]")
{
    SECTION("Deny all")
    {
        FPathPermissionList List;
        List.AddDenyListAll("DenyAll");
        
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game")) == EPathPermissionPrefixResult::FailRecursive);
    }

    SECTION("Deny only")
    {
        FPathPermissionList List;
        List.AddDenyListItem("Deny", TEXTVIEW("/Game/Secret"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Game/Maps/Secret"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Plugin1/Secret"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Plugin2"));
        
        // Some children of /Game are blocked
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game")) == EPathPermissionPrefixResult::Pass);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game/Secret")) == EPathPermissionPrefixResult::FailRecursive);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game/Characters")) == EPathPermissionPrefixResult::PassRecursive);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game/Maps")) == EPathPermissionPrefixResult::Pass);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game/Maps/Secret")) == EPathPermissionPrefixResult::FailRecursive);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game/Maps/Desert")) == EPathPermissionPrefixResult::PassRecursive);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Plugin1")) == EPathPermissionPrefixResult::Pass);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Plugin1/Secret")) == EPathPermissionPrefixResult::FailRecursive);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Plugin2")) == EPathPermissionPrefixResult::FailRecursive);
    }   
    
    SECTION("Allow only")
    {
        FPathPermissionList List;
        List.AddAllowListItem("Allow", TEXTVIEW("/Game/Maps"));
        List.AddAllowListItem("Allow", TEXTVIEW("/Plugin1"));
        
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game")) == EPathPermissionPrefixResult::Fail);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game/Maps")) == EPathPermissionPrefixResult::PassRecursive);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Plugin1")) == EPathPermissionPrefixResult::PassRecursive);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Plugin1/Characters")) == EPathPermissionPrefixResult::PassRecursive);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Plugin2")) == EPathPermissionPrefixResult::FailRecursive);
    }

    SECTION("Mixed allow and deny")
    {
        FPathPermissionList List;
        List.AddAllowListItem("Allow", TEXTVIEW("/Game"));        
        List.AddAllowListItem("Allow", TEXTVIEW("/Plugin1"));
        List.AddAllowListItem("Allow", TEXTVIEW("/Plugin2/Public"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Game/Secret"));
        List.AddDenyListItem("Deny", TEXTVIEW("/Game/Maps/Secret"));
        
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game")) == EPathPermissionPrefixResult::Pass);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game/Secret")) == EPathPermissionPrefixResult::FailRecursive);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game/Maps")) == EPathPermissionPrefixResult::Pass);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Game/Maps/Secret")) == EPathPermissionPrefixResult::FailRecursive);

        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Plugin1")) == EPathPermissionPrefixResult::PassRecursive);

        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Plugin2")) == EPathPermissionPrefixResult::Fail);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Plugin2/Public")) == EPathPermissionPrefixResult::PassRecursive);
        CHECK(List.PassesStartsWithFilterRecursive(TEXTVIEW("/Plugin2/Private")) == EPathPermissionPrefixResult::FailRecursive);
    }
}

#endif // WITH_TESTS