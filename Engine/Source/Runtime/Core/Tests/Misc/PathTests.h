// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

#if WITH_TESTS

template<class PathType, class StringType>
void TestCollapseRelativeDirectories()
{
	auto Run = [](const TCHAR* Path, bool bExpectedAllCollapsed, const TCHAR* Expected)
	{
		// Run test
		StringType Actual;
		StringType IfPossibleActual;
		Actual += Path;
		IfPossibleActual += Path;
		const bool bAllCollapsed = PathType::CollapseRelativeDirectories(
			Actual, false /* bCollapseAllPossible */);
		const bool bIfPossibleAllCollapsed = PathType::CollapseRelativeDirectories(
			IfPossibleActual, true /* bCollapseAllPossible */);

		auto IsExpectedText = [Expected](const TCHAR* Actual)
			{
				return FCString::Strcmp(Actual, Expected) == 0;
			};
		if (bExpectedAllCollapsed)
		{
			// If we're looking for a result, make sure it was returned correctly
			if (!bAllCollapsed || !IsExpectedText(*Actual))
			{
				FAIL_CHECK(FString::Printf(TEXT("CollapseRelativeDirectories('%s', false) failed (got (%s, '%s'), expected (%s, '%s'))."),
					Path, *LexToString(bAllCollapsed), *Actual, *LexToString(bExpectedAllCollapsed), Expected));
			}
			if (!bIfPossibleAllCollapsed || !IsExpectedText(*IfPossibleActual))
			{
				FAIL_CHECK(FString::Printf(TEXT("CollapseRelativeDirectories('%s', true) failed (got (%s, '%s'), expected (%s, '%s'))."),
					Path, *LexToString(bIfPossibleAllCollapsed), *IfPossibleActual, *LexToString(bExpectedAllCollapsed), Expected));
			}
		}
		else
		{
			// Otherwise, make sure it failed
			if (bAllCollapsed)
			{
				// Some of the string might have been collapsed before finding out in has an incollapsible piece
				// The only well-defined option for those cases is to leave the original string unaltered, but
				// enforcing that contract is too expensive, so we don't provide that contract and instead we
				// only guarantee that the modified path is equivalent, perhaps with some .. and . collapsed.
				// We don't currently validate that equivalency on our test cases, because it is difficult to calculate.
				FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed CollapseRelativeDirectories (got (%s, '%s'), expected (%s, '<anyvalue>'))."),
					Path, *LexToString(bAllCollapsed), *Actual, *LexToString(bExpectedAllCollapsed)));
			}
			if (bIfPossibleAllCollapsed || !IsExpectedText(*IfPossibleActual))
			{
				FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed CollapseRelativeDirectoriesIfPossible (got (%s, '%s'), expected (%s, '%s'))."),
					Path, *LexToString(bIfPossibleAllCollapsed), *IfPossibleActual, *LexToString(bExpectedAllCollapsed), Expected));
			}
		}
	};

	Run(TEXT(".."),                                                   false, TEXT(".."));
	Run(TEXT("/.."),                                                  false, TEXT("/.."));
	Run(TEXT("./"),                                                   true,  TEXT(""));
	Run(TEXT("./file.txt"),                                           true,  TEXT("file.txt"));
	Run(TEXT("/."),                                                   true,  TEXT("/."));
	Run(TEXT("Folder"),                                               true,  TEXT("Folder"));
	Run(TEXT("/Folder"),                                              true,  TEXT("/Folder"));
	Run(TEXT("C:/Folder"),                                            true,  TEXT("C:/Folder"));
	Run(TEXT("C:/Folder/.."),                                         true,  TEXT("C:")); // removing the leading slash is incorrect but legacy behavior, we should fix eventually
	Run(TEXT("C:/Folder/../"),                                        true,  TEXT("C:/"));
	Run(TEXT("C:/Folder/../file.txt"),                                true,  TEXT("C:/file.txt"));
	Run(TEXT("Folder/.."),                                            true,  TEXT(""));
	Run(TEXT("Folder/../"),                                           true,  TEXT(""));
	Run(TEXT("Folder/../file.txt"),                                   true,  TEXT("file.txt"));
	Run(TEXT("/Folder/.."),                                           true,  TEXT("")); // removing the leading slash is incorrect but legacy behavior, we should fix eventually
	Run(TEXT("/Folder/../"),                                          true,  TEXT("/"));
	Run(TEXT("/Folder/../file.txt"),                                  true,  TEXT("/file.txt"));
	Run(TEXT("Folder/../.."),                                         false, TEXT(".."));
	Run(TEXT("Folder/../../"),                                        false, TEXT("../"));
	Run(TEXT("Folder/../../file.txt"),                                false, TEXT("../file.txt"));
	Run(TEXT("C:/.."),                                                false, TEXT("C:/.."));
	Run(TEXT("C:/."),                                                 true,  TEXT("C:/."));
	Run(TEXT("C:/./"),                                                true,  TEXT("C:/"));
	Run(TEXT("C:/./file.txt"),                                        true,  TEXT("C:/file.txt"));
	Run(TEXT("C:/Folder1/../Folder2"),                                true,  TEXT("C:/Folder2"));
	Run(TEXT("C:/Folder1/../Folder2/"),                               true,  TEXT("C:/Folder2/"));
	Run(TEXT("C:/Folder1/../Folder2/file.txt"),                       true,  TEXT("C:/Folder2/file.txt"));
	Run(TEXT("C:/Folder1/../Folder2/../.."),                          false, TEXT("C:/.."));
	Run(TEXT("C:/Folder1/../Folder2/../Folder3"),                     true,  TEXT("C:/Folder3"));
	Run(TEXT("C:/Folder1/../Folder2/../Folder3/"),                    true,  TEXT("C:/Folder3/"));
	Run(TEXT("C:/Folder1/../Folder2/../Folder3/file.txt"),            true,  TEXT("C:/Folder3/file.txt"));
	Run(TEXT("C:/Folder1/Folder2/../../Folder3"),                     true,  TEXT("C:/Folder3"));
	Run(TEXT("C:/Folder1/Folder2/../../Folder3/"),                    true,  TEXT("C:/Folder3/"));
	Run(TEXT("C:/Folder1/Folder2/../../Folder3/file.txt"),            true,  TEXT("C:/Folder3/file.txt"));
	Run(TEXT("C:/Folder1/Folder2/../../Folder3/../Folder4"),          true,  TEXT("C:/Folder4"));
	Run(TEXT("C:/Folder1/Folder2/../../Folder3/../Folder4/"),         true,  TEXT("C:/Folder4/"));
	Run(TEXT("C:/Folder1/Folder2/../../Folder3/../Folder4/file.txt"), true,  TEXT("C:/Folder4/file.txt"));
	Run(TEXT("C:/Folder1/Folder2/../Folder3/../../Folder4"),          true,  TEXT("C:/Folder4"));
	Run(TEXT("C:/Folder1/Folder2/../Folder3/../../Folder4/"),         true,  TEXT("C:/Folder4/"));
	Run(TEXT("C:/Folder1/Folder2/../Folder3/../../Folder4/file.txt"), true,  TEXT("C:/Folder4/file.txt"));
	Run(TEXT("C:/Folder1/Folder2/.././../Folder4"),                   true,  TEXT("C:/Folder4"));
	Run(TEXT("C:/Folder1/Folder2/.././../Folder4/"),                  true,  TEXT("C:/Folder4/"));
	Run(TEXT("C:/Folder1/Folder2/.././../Folder4/file.txt"),          true,  TEXT("C:/Folder4/file.txt"));
	Run(TEXT("C:/A/B/.././../C"),                                     true,  TEXT("C:/C"));
	Run(TEXT("C:/A/B/.././../C/"),                                    true,  TEXT("C:/C/"));
	Run(TEXT("C:/A/B/.././../C/file.txt"),                            true,  TEXT("C:/C/file.txt"));
	Run(TEXT(".svn"),                                                 true,  TEXT(".svn"));
	Run(TEXT("/.svn"),                                                true,  TEXT("/.svn"));
	Run(TEXT("./Folder/.svn"),                                        true,  TEXT("Folder/.svn"));
	Run(TEXT("./.svn/../.svn"),                                       true,  TEXT(".svn"));
	Run(TEXT(".svn/./.svn/.././../.svn"),                             true,  TEXT(".svn"));
	Run(TEXT("C:/Folder1/./Folder2/..Folder3"),                       true,  TEXT("C:/Folder1/Folder2/..Folder3"));
	Run(TEXT("C:/Folder1/./Folder2/..Folder3/Folder4"),               true,  TEXT("C:/Folder1/Folder2/..Folder3/Folder4"));
	Run(TEXT("C:/Folder1/./Folder2/..Folder3/..Folder4"),             true,  TEXT("C:/Folder1/Folder2/..Folder3/..Folder4"));
	Run(TEXT("C:/Folder1/./Folder2/..Folder3/Folder4/../Folder5"),    true,  TEXT("C:/Folder1/Folder2/..Folder3/Folder5"));
	Run(TEXT("C:/Folder1/..Folder2/Folder3/..Folder4/../Folder5"),    true,  TEXT("C:/Folder1/..Folder2/Folder3/Folder5"));
	// Handle .. that need to collapse through duplicate separators
	Run(TEXT("D:/Root/Engine//../.."),                                true,  TEXT("D:")); // removing the leading slash is incorrect but legacy behavior, we should fix eventually
	Run(TEXT("D:/Root/Engine////////../.."),                          true,  TEXT("D:")); // removing the leading slash is incorrect but legacy behavior, we should fix eventually
	Run(TEXT("D:/Root/Engine//../../"),                               true,  TEXT("D:/"));
	Run(TEXT("D:/Root/Engine////////../../"),                         true,  TEXT("D:/"));
	Run(TEXT("D:/Root//../.."),										  false, TEXT("D:/.."));
	Run(TEXT("D:/Root////////../.."),                                 false, TEXT("D:/.."));
	// But don't remove a leading // for network share when collapsing
	Run(TEXT("//.."),                                                 true, TEXT("")); // removing the leading network path is incorrect but legacy behavior, we should fix eventually
	Run(TEXT("//Root/../.."),                                         true, TEXT("")); // removing the leading network path is incorrect but legacy behavior, we should fix eventually
}

template<class PathType, class StringType>
void TestRemoveDuplicateSlashes()
{
	auto Run = [&](const TCHAR* Path, const TCHAR* Expected)
	{
		StringType Actual;
		Actual += Path;	
		PathType::RemoveDuplicateSlashes(Actual);
		CHECK_EQUALS(TEXT("RemoveDuplicateSlashes"), *Actual, Expected);
	};

	Run(TEXT(""), TEXT(""));
	Run(TEXT("C:/Folder/File.txt"), TEXT("C:/Folder/File.txt"));
	Run(TEXT("C:/Folder/File/"), TEXT("C:/Folder/File/"));
	Run(TEXT("/"), TEXT("/"));
	Run(TEXT("//"), TEXT("/"));
	Run(TEXT("////"), TEXT("/"));
	Run(TEXT("/Folder/File"), TEXT("/Folder/File"));
	Run(TEXT("//Folder/File"), TEXT("/Folder/File")); // Don't use on //UNC paths; it will be stripped!
	Run(TEXT("/////Folder//////File/////"), TEXT("/Folder/File/"));
	Run(TEXT("\\\\Folder\\\\File\\\\"), TEXT("\\\\Folder\\\\File\\\\")); // It doesn't strip backslash, and we rely on that in some places
	Run(TEXT("//\\\\//Folder//\\\\//File//\\\\//"), TEXT("/\\\\/Folder/\\\\/File/\\\\/"));
}

namespace PathTest
{

struct FTestPair
{
	FStringView Input;
	FStringView Expected;
};

extern const FStringView BaseDir;

extern TConstArrayView<FTestPair> ExpectedRelativeToAbsolutePaths;

}

#endif //WITH_TESTS