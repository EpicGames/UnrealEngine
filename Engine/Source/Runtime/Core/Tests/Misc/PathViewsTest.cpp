// Copyright Epic Games, Inc. All Rights Reserved.

#include "PathTests.h"

#include "Algo/Compare.h"
#include "Containers/StringView.h"
#include "Misc/PathViews.h"
#include "Tests/TestHarnessAdapter.h"
#include "Misc/StringBuilder.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

#if WITH_TESTS

class FPathViewsTest
{
public:

	void TestViewTransform(FStringView (*InFunction)(const FStringView& InPath), const FStringView& InPath, const TCHAR* InExpected)
	{
		const FStringView Actual = InFunction(InPath);
		if (Actual != InExpected)
		{
			FAIL_CHECK(FString::Printf(TEXT("Failed on path '%.*s' (got '%.*s', expected '%s')."),
				InPath.Len(), InPath.GetData(), Actual.Len(), Actual.GetData(), InExpected));
		}
	}

	static const TCHAR* BoolToString(bool bValue)
	{
		return bValue ? TEXT("true") : TEXT("false");
	}

	bool GetCleanFilenameTest()
	{
		auto RunGetCleanFilenameTest = [this](const TCHAR* InPath, const TCHAR* InExpected)
		{
			TestViewTransform(FPathViews::GetCleanFilename, InPath, InExpected);
		};

		RunGetCleanFilenameTest(TEXT(""), TEXT(""));
		RunGetCleanFilenameTest(TEXT(".txt"), TEXT(".txt"));
		RunGetCleanFilenameTest(TEXT(".tar.gz"), TEXT(".tar.gz"));
		RunGetCleanFilenameTest(TEXT(".tar.gz/"), TEXT(""));
		RunGetCleanFilenameTest(TEXT(".tar.gz\\"), TEXT(""));
		RunGetCleanFilenameTest(TEXT("File"), TEXT("File"));
		RunGetCleanFilenameTest(TEXT("File.tar.gz"), TEXT("File.tar.gz"));
		RunGetCleanFilenameTest(TEXT("File.tar.gz/"), TEXT(""));
		RunGetCleanFilenameTest(TEXT("File.tar.gz\\"), TEXT(""));
		RunGetCleanFilenameTest(TEXT("C:/Folder/"), TEXT(""));
		RunGetCleanFilenameTest(TEXT("C:/Folder/File"), TEXT("File"));
		RunGetCleanFilenameTest(TEXT("C:/Folder/File.tar.gz"), TEXT("File.tar.gz"));
		RunGetCleanFilenameTest(TEXT("C:/Folder/First.Last/File"), TEXT("File"));
		RunGetCleanFilenameTest(TEXT("C:/Folder/First.Last/File.tar.gz"), TEXT("File.tar.gz"));
		RunGetCleanFilenameTest(TEXT("C:\\Folder\\"), TEXT(""));
		RunGetCleanFilenameTest(TEXT("C:\\Folder\\File"), TEXT("File"));
		RunGetCleanFilenameTest(TEXT("C:\\Folder\\First.Last\\"), TEXT(""));
		RunGetCleanFilenameTest(TEXT("C:\\Folder\\First.Last\\File"), TEXT("File"));
		RunGetCleanFilenameTest(TEXT("C:\\Folder\\First.Last\\File.tar.gz"), TEXT("File.tar.gz"));
		return true;
	}

	bool GetBaseFilenameTest()
	{
		auto RunGetBaseFilenameTest = [this](const TCHAR* InPath, const TCHAR* InExpected, const TCHAR* InExpectedWithPath)
		{
			const FStringView Path = InPath;
			TestViewTransform(FPathViews::GetBaseFilename, Path, InExpected);
			TestViewTransform(FPathViews::GetBaseFilenameWithPath, Path, InExpectedWithPath);
		};

		RunGetBaseFilenameTest(TEXT(""), TEXT(""), TEXT(""));
		RunGetBaseFilenameTest(TEXT(".txt"), TEXT(""), TEXT(""));
		RunGetBaseFilenameTest(TEXT(".tar.gz"), TEXT(".tar"), TEXT(".tar"));
		RunGetBaseFilenameTest(TEXT(".tar.gz/"), TEXT(""), TEXT(".tar.gz/"));
		RunGetBaseFilenameTest(TEXT(".tar.gz\\"), TEXT(""), TEXT(".tar.gz\\"));
		RunGetBaseFilenameTest(TEXT("File"), TEXT("File"), TEXT("File"));
		RunGetBaseFilenameTest(TEXT("File.txt"), TEXT("File"), TEXT("File"));
		RunGetBaseFilenameTest(TEXT("File.tar.gz"), TEXT("File.tar"), TEXT("File.tar"));
		RunGetBaseFilenameTest(TEXT("File.tar.gz/"), TEXT(""), TEXT("File.tar.gz/"));
		RunGetBaseFilenameTest(TEXT("File.tar.gz\\"), TEXT(""), TEXT("File.tar.gz\\"));
		RunGetBaseFilenameTest(TEXT("C:/Folder/"), TEXT(""), TEXT("C:/Folder/"));
		RunGetBaseFilenameTest(TEXT("C:/Folder/File"), TEXT("File"), TEXT("C:/Folder/File"));
		RunGetBaseFilenameTest(TEXT("C:/Folder/File.tar.gz"), TEXT("File.tar"), TEXT("C:/Folder/File.tar"));
		RunGetBaseFilenameTest(TEXT("C:/Folder/First.Last/File"), TEXT("File"), TEXT("C:/Folder/First.Last/File"));
		RunGetBaseFilenameTest(TEXT("C:/Folder/First.Last/File.txt"), TEXT("File"), TEXT("C:/Folder/First.Last/File"));
		RunGetBaseFilenameTest(TEXT("C:/Folder/First.Last/File.tar.gz"), TEXT("File.tar"), TEXT("C:/Folder/First.Last/File.tar"));
		RunGetBaseFilenameTest(TEXT("C:\\Folder\\"), TEXT(""), TEXT("C:\\Folder\\"));
		RunGetBaseFilenameTest(TEXT("C:\\Folder\\File"), TEXT("File"), TEXT("C:\\Folder\\File"));
		RunGetBaseFilenameTest(TEXT("C:\\Folder\\First.Last\\"), TEXT(""), TEXT("C:\\Folder\\First.Last\\"));
		RunGetBaseFilenameTest(TEXT("C:\\Folder\\First.Last\\File"), TEXT("File"), TEXT("C:\\Folder\\First.Last\\File"));
		RunGetBaseFilenameTest(TEXT("C:\\Folder\\First.Last\\File.txt"), TEXT("File"), TEXT("C:\\Folder\\First.Last\\File"));
		RunGetBaseFilenameTest(TEXT("C:\\Folder\\First.Last\\File.tar.gz"), TEXT("File.tar"), TEXT("C:\\Folder\\First.Last\\File.tar"));

		return true;
	}

	bool GetPathTest()
	{
		auto RunGetPathTest = [this](const TCHAR* InPath, const TCHAR* InExpected)
		{
			TestViewTransform(FPathViews::GetPath, InPath, InExpected);
		};

		RunGetPathTest(TEXT(""), TEXT(""));
		RunGetPathTest(TEXT(".txt"), TEXT(""));
		RunGetPathTest(TEXT(".tar.gz"), TEXT(""));
		RunGetPathTest(TEXT(".tar.gz/"), TEXT(".tar.gz"));
		RunGetPathTest(TEXT(".tar.gz\\"), TEXT(".tar.gz"));
		RunGetPathTest(TEXT("File"), TEXT(""));
		RunGetPathTest(TEXT("File.txt"), TEXT(""));
		RunGetPathTest(TEXT("File.tar.gz"), TEXT(""));
		RunGetPathTest(TEXT("File.tar.gz/"), TEXT("File.tar.gz"));
		RunGetPathTest(TEXT("File.tar.gz\\"), TEXT("File.tar.gz"));
		RunGetPathTest(TEXT("C:/Folder/"), TEXT("C:/Folder"));
		RunGetPathTest(TEXT("C:/Folder/File"), TEXT("C:/Folder"));
		RunGetPathTest(TEXT("C:/Folder/File.tar.gz"), TEXT("C:/Folder"));
		RunGetPathTest(TEXT("C:/Folder/First.Last/File"), TEXT("C:/Folder/First.Last"));
		RunGetPathTest(TEXT("C:/Folder/First.Last/File.tar.gz"), TEXT("C:/Folder/First.Last"));
		RunGetPathTest(TEXT("C:\\Folder\\"), TEXT("C:\\Folder"));
		RunGetPathTest(TEXT("C:\\Folder\\File"), TEXT("C:\\Folder"));
		RunGetPathTest(TEXT("C:\\Folder\\First.Last\\"), TEXT("C:\\Folder\\First.Last"));
		RunGetPathTest(TEXT("C:\\Folder\\First.Last\\File"), TEXT("C:\\Folder\\First.Last"));
		RunGetPathTest(TEXT("C:\\Folder\\First.Last\\File.tar.gz"), TEXT("C:\\Folder\\First.Last"));

		return true;
	}

	bool GetExtensionTest() 
	{
		using namespace UE::Paths;

		auto RunGetExtTest = [this](const TCHAR* InPath, const TCHAR* InExpectedExt, const TCHAR* InExpectedExtDot)
		{
			const FStringView Path = InPath;
			TestViewTransform([](const FStringView& InPath2) { return FPathViews::GetExtension(InPath2, /*bIncludeDot*/ false); }, Path, InExpectedExt);
			TestViewTransform([](const FStringView& InPath2) { return FPathViews::GetExtension(InPath2); }, Path, InExpectedExt);
			TestViewTransform([](const FStringView& InPath2) { return FPathViews::GetExtension(InPath2, /*bIncludeDot*/ true); }, Path, InExpectedExtDot);
			TestViewTransform([](const FStringView& InPath2) { return FPathViews::GetExtension(InPath2, EFlags::IncludeDot); }, Path, InExpectedExtDot);
		};

		auto RunGetExtTestFull = [this](const TCHAR* InPath, const TCHAR* InExpectedExt, const TCHAR* InExpectedExtDot,
			const TCHAR* InExpectedExtCompound, const TCHAR* InExpectedExtDotCompound)
		{
			const FStringView Path = InPath;
			TestViewTransform([](const FStringView& InPath2)
				{ return FPathViews::GetExtension(InPath2); },
				Path, InExpectedExt);
			TestViewTransform([](const FStringView& InPath2)
				{ return FPathViews::GetExtension(InPath2, EFlags::None); },
				Path, InExpectedExt);
			TestViewTransform([](const FStringView& InPath2)
				{ return FPathViews::GetExtension(InPath2, EFlags::IncludeDot); },
				Path, InExpectedExtDot);
			TestViewTransform([](const FStringView& InPath2)
				{ return FPathViews::GetExtension(InPath2, EFlags::AllowCompoundExtension); },
				Path, InExpectedExtCompound);
			TestViewTransform([](const FStringView& InPath2)
				{ return FPathViews::GetExtension(InPath2, EFlags::IncludeDot | EFlags::AllowCompoundExtension); },
				Path, InExpectedExtDotCompound);
			};

		RunGetExtTest(TEXT(""),											TEXT(""),	TEXT(""));
		RunGetExtTest(TEXT(".txt"), 									TEXT("txt"),TEXT(".txt"));
		RunGetExtTest(TEXT(".tar.gz"),									TEXT("gz"), TEXT(".gz"));
		RunGetExtTest(TEXT(".tar.gz/"),									TEXT(""),	TEXT(""));
		RunGetExtTest(TEXT(".tar.gz\\"),								TEXT(""),	TEXT(""));
		RunGetExtTest(TEXT("File"),										TEXT(""),	TEXT(""));
		RunGetExtTest(TEXT("File.txt"),									TEXT("txt"),TEXT(".txt"));
		RunGetExtTestFull(TEXT("File.tar.gz"),							TEXT("gz"), TEXT(".gz"),	TEXT("tar.gz"), TEXT(".tar.gz"));
		RunGetExtTestFull(TEXT("File.tar.gz/"),							TEXT(""),	TEXT(""),		TEXT(""),		TEXT(""));
		RunGetExtTestFull(TEXT("File.tar.gz\\"),						TEXT(""),	TEXT(""),		TEXT(""),		TEXT(""));
		RunGetExtTest(TEXT("C:/Folder/File"),							TEXT(""),	TEXT(""));
		RunGetExtTest(TEXT("C:\\Folder\\File"),							TEXT(""),	TEXT(""));
		RunGetExtTest(TEXT("C:/Folder/File.txt"),						TEXT("txt"),TEXT(".txt"));
		RunGetExtTest(TEXT("C:\\Folder\\File.txt"),						TEXT("txt"),TEXT(".txt"));
		RunGetExtTestFull(TEXT("C:/Folder/File.tar.gz"),				TEXT("gz"), TEXT(".gz"),	TEXT("tar.gz"),	TEXT(".tar.gz"));
		RunGetExtTestFull(TEXT("C:\\Folder\\File.tar.gz"),				TEXT("gz"), TEXT(".gz"),	TEXT("tar.gz"),	TEXT(".tar.gz"));
		RunGetExtTestFull(TEXT("C:/Folder/First.Last/File"),			TEXT(""),	TEXT(""),		TEXT(""),		TEXT(""));
		RunGetExtTestFull(TEXT("C:\\Folder\\First.Last\\File"),			TEXT(""),	TEXT(""),		TEXT(""),		TEXT(""));
		RunGetExtTestFull(TEXT("C:/Folder/First.Last/File.txt"),		TEXT("txt"),TEXT(".txt"),	TEXT("txt"),	TEXT(".txt"));
		RunGetExtTestFull(TEXT("C:\\Folder\\First.Last\\File.txt"),		TEXT("txt"),TEXT(".txt"),	TEXT("txt"),	TEXT(".txt"));
		RunGetExtTestFull(TEXT("C:/Folder/First.Last/File.tar.gz"),		TEXT("gz"), TEXT(".gz"),	TEXT("tar.gz"), TEXT(".tar.gz"));
		RunGetExtTestFull(TEXT("C:\\Folder\\First.Last\\File.tar.gz"),	TEXT("gz"), TEXT(".gz"),	TEXT("tar.gz"), TEXT(".tar.gz"));

		return true;
	}

	bool GetPathLeafTest()
	{
		auto RunGetPathLeafTest = [this](const TCHAR* InPath, const TCHAR* InExpected)
		{
			TestViewTransform(FPathViews::GetPathLeaf, InPath, InExpected);
		};

		RunGetPathLeafTest(TEXT(""), TEXT(""));
		RunGetPathLeafTest(TEXT(".txt"), TEXT(".txt"));
		RunGetPathLeafTest(TEXT(".tar.gz"), TEXT(".tar.gz"));
		RunGetPathLeafTest(TEXT(".tar.gz/"), TEXT(".tar.gz"));
		RunGetPathLeafTest(TEXT(".tar.gz\\"), TEXT(".tar.gz"));
		RunGetPathLeafTest(TEXT("File"), TEXT("File"));
		RunGetPathLeafTest(TEXT("File.txt"), TEXT("File.txt"));
		RunGetPathLeafTest(TEXT("File.tar.gz"), TEXT("File.tar.gz"));
		RunGetPathLeafTest(TEXT("File.tar.gz/"), TEXT("File.tar.gz"));
		RunGetPathLeafTest(TEXT("File.tar.gz\\"), TEXT("File.tar.gz"));
		RunGetPathLeafTest(TEXT("C:/Folder/"), TEXT("Folder"));
		RunGetPathLeafTest(TEXT("C:/Folder/File"), TEXT("File"));
		RunGetPathLeafTest(TEXT("C:/Folder/File.tar.gz"), TEXT("File.tar.gz"));
		RunGetPathLeafTest(TEXT("C:/Folder/First.Last/File"), TEXT("File"));
		RunGetPathLeafTest(TEXT("C:/Folder/First.Last/File.tar.gz"), TEXT("File.tar.gz"));
		RunGetPathLeafTest(TEXT("C:\\Folder\\"), TEXT("Folder"));
		RunGetPathLeafTest(TEXT("C:\\Folder\\File"), TEXT("File"));
		RunGetPathLeafTest(TEXT("C:\\Folder\\First.Last\\"), TEXT("First.Last"));
		RunGetPathLeafTest(TEXT("C:\\Folder\\First.Last\\File"), TEXT("File"));
		RunGetPathLeafTest(TEXT("C:\\Folder\\First.Last\\File.tar.gz"), TEXT("File.tar.gz"));

		return true;
	}

	bool IsPathLeafTest()
	{
		auto RunIsPathLeafTest = [this](const TCHAR* InPath, bool bExpected)
		{
			bool bResult = FPathViews::IsPathLeaf(InPath);
			if (bResult != bExpected)
			{
				FAIL_CHECK(FString::Printf(TEXT("GetPathLeaf(\"%s\") == %s, expected %s."), InPath, BoolToString(bResult), BoolToString(bExpected)));
			}
		};

		RunIsPathLeafTest(TEXT(""), true);
		RunIsPathLeafTest(TEXT(".txt"), true);
		RunIsPathLeafTest(TEXT(".tar.gz"), true);
		RunIsPathLeafTest(TEXT(".tar.gz/"), true);
		RunIsPathLeafTest(TEXT(".tar.gz\\"), true);
		RunIsPathLeafTest(TEXT("File"), true);
		RunIsPathLeafTest(TEXT("File.txt"), true);
		RunIsPathLeafTest(TEXT("File.tar.gz"), true);
		RunIsPathLeafTest(TEXT("File.tar.gz/"), true);
		RunIsPathLeafTest(TEXT("File.tar.gz\\"), true);
		RunIsPathLeafTest(TEXT("//"), true);
		RunIsPathLeafTest(TEXT("\\\\"), true);
		RunIsPathLeafTest(TEXT("/"), true);
		RunIsPathLeafTest(TEXT("\\"), true);
		RunIsPathLeafTest(TEXT("C:/"), true);
		RunIsPathLeafTest(TEXT("C:\\"), true);
		RunIsPathLeafTest(TEXT("//Folder"), false);
		RunIsPathLeafTest(TEXT("\\\\Folder"), false);
		RunIsPathLeafTest(TEXT("/Folder"), false);
		RunIsPathLeafTest(TEXT("\\Folder"), false);
		RunIsPathLeafTest(TEXT("C:/Folder"), false);
		RunIsPathLeafTest(TEXT("C:\\Folder"), false);
		RunIsPathLeafTest(TEXT("C:/Folder"), false);
		RunIsPathLeafTest(TEXT("C:\\Folder"), false);

		return true;
	}

	bool EqualsAndLessTest()
	{
		auto RunEqualsLessTest = [this](const TCHAR* A, const TCHAR* B, int Expected)
		{
			bool bEqual = FPathViews::Equals(A, B);
			bool bALessThanB = FPathViews::Less(A, B);
			bool bBLessThanA = FPathViews::Less(B, A);
			if (bEqual != (Expected == 0))
			{
				FAIL_CHECK(FString::Printf(TEXT("Equals(%s,%s) == %s, expected %s"), A, B, BoolToString(bEqual), BoolToString(Expected == 0)));
			}
			if (bALessThanB != Expected < 0)
			{
				FAIL_CHECK(FString::Printf(TEXT("Less(%s,%s) == %s, expected %s"), A, B, BoolToString(bALessThanB), BoolToString(Expected < 0)));
			}
			if (bBLessThanA != Expected > 0)
			{
				FAIL_CHECK(FString::Printf(TEXT("Less(%s,%s) == %s, expected %s"), B, A, BoolToString(bBLessThanA), BoolToString(Expected > 0)));
			}
		};

		RunEqualsLessTest(TEXT("A"), TEXT("B"), -1);
		RunEqualsLessTest(TEXT("A"), TEXT("b"), -1);
		RunEqualsLessTest(TEXT("a"), TEXT("B"), -1);
		RunEqualsLessTest(TEXT("A"), TEXT("A"), 0);
		RunEqualsLessTest(TEXT("A"), TEXT("a/"), 0);
		RunEqualsLessTest(TEXT("A"), TEXT("a\\"), 0);
		RunEqualsLessTest(TEXT("A"), TEXT("abby"), -1);
		RunEqualsLessTest(TEXT("a"), TEXT("Abby"), -1);
		RunEqualsLessTest(TEXT("\\A/B"), TEXT("/A\\B/"), 0);
		//RunEqualsLessTest(TEXT("../../../Engine"), TEXT("C:/Engine"), 0); // Detecting relpath == abspath is not yet implemented
		//RunEqualsLessTest(TEXT("C:/A/B"), TEXT("C:/A/../A/./B"), 0); // Collapsing .. and . is not yet implemented
		RunEqualsLessTest(TEXT("/"), TEXT("/"), 0);
		RunEqualsLessTest(TEXT("/"), TEXT("//"), -1);
		RunEqualsLessTest(TEXT("/"), TEXT("C:/"), -1); // '/' sorts less than 'C'
		RunEqualsLessTest(TEXT("/"), TEXT("C:\\"), -1); // '\\' sorts less than 'C'
		RunEqualsLessTest(TEXT("/"), TEXT("c:/"), -1); // '/' sorts less than 'c'
		RunEqualsLessTest(TEXT("/"), TEXT("c:\\"), -1); // '\\' sorts less than 'c'
		RunEqualsLessTest(TEXT("/"), TEXT("A"), -1);
		RunEqualsLessTest(TEXT("//"), TEXT("//"), 0);
		RunEqualsLessTest(TEXT("//"), TEXT("C:/"), -1);
		RunEqualsLessTest(TEXT("//"), TEXT("A"), -1);
		RunEqualsLessTest(TEXT("C:/"), TEXT("C:/"), 0);
		RunEqualsLessTest(TEXT("C:/"), TEXT("C"), 1);
		// The directory separator is less than all characters except for '\0', so that shorter directories come before longer directories.
		// This is true for all wchar; to save time we test it just for all ascii characters.
		const TCHAR* ForwardSlashTestString = TEXT("0/1");
		const TCHAR* BackSlashTestString = TEXT("0\\1");
		TCHAR CharTestString[] = TEXT("0_1");
		for (int IntC = 1; IntC < 128; IntC++)
		{
			char c = (char)IntC;
			if (c == '/' || c == '\\')
			{
				continue;
			}
			CharTestString[1] = c;
			RunEqualsLessTest(ForwardSlashTestString, CharTestString, -1);
			RunEqualsLessTest(CharTestString, ForwardSlashTestString, 1);
			RunEqualsLessTest(BackSlashTestString, CharTestString, -1);
			RunEqualsLessTest(CharTestString, BackSlashTestString, 1);
		}
		// These tests use hyphen, which has int('-') == 45 < int("/") == 47 < int("\\") == 92
		// Foo is a shorter string than Foo-Bar so it comes first, aka / sorts earlier than all other characters
		RunEqualsLessTest(TEXT("Foo/Leaf"), TEXT("Foo-Bar/Leaf"), -1);
		RunEqualsLessTest(TEXT("Foo\\Leaf"), TEXT("Foo-Bar\\Leaf"), -1);
		// When the / is terminating, sort order needs to match what it is when it is in the middle
		RunEqualsLessTest(TEXT("Foo/"), TEXT("Foo-Bar/"), -1); 
		// When the terminating / is omitted, sort order needs to match when it is present
		RunEqualsLessTest(TEXT("Foo"), TEXT("Foo-Bar"), -1); 
		RunEqualsLessTest(TEXT("Foo"), TEXT("Foo-Bar/"), -1); 
		RunEqualsLessTest(TEXT("Foo/"), TEXT("Foo-Bar/"), -1); 

		return true;
	}

	bool TryMakeChildPathRelativeToTest()
	{
		auto RunRelChildTest = [this](const TCHAR* Child, const TCHAR* Parent, bool bExpectedIsChild, const TCHAR* ExpectedRelPath)
		{
			FStringView ActualRelPath;
			bool bActualIsChild = FPathViews::TryMakeChildPathRelativeTo(Child, Parent, ActualRelPath);
			bool bActualIsParent = FPathViews::IsParentPathOf(Parent, Child);
			if (bExpectedIsChild != bActualIsChild || !ActualRelPath.Equals(ExpectedRelPath, ESearchCase::IgnoreCase))
			{
				FAIL_CHECK(FString::Printf(TEXT("TryMakeChildPathRelativeTo(\"%s\", \"%s\") returned (%s, \"%.*s\"), expected (%s, \"%s\")."), Child, Parent,
					BoolToString(bActualIsChild), ActualRelPath.Len(), ActualRelPath.GetData(), BoolToString(bExpectedIsChild), ExpectedRelPath));
			}
			if (bActualIsParent != bExpectedIsChild)
			{
				FAIL_CHECK(FString::Printf(TEXT("IsParentPathOf(\"%s\", \"%s\") returned %s, expected %s."), Child, Parent,
					BoolToString(bActualIsParent), BoolToString(bExpectedIsChild)));
			}
		};

		RunRelChildTest(TEXT("A/B"), TEXT("A"), true, TEXT("B"));
		RunRelChildTest(TEXT("A\\B"), TEXT("A"), true, TEXT("B"));
		RunRelChildTest(TEXT("A/B"), TEXT("A/"), true, TEXT("B"));
		RunRelChildTest(TEXT("A\\B"), TEXT("A/"), true, TEXT("B"));
		RunRelChildTest(TEXT("A/B"), TEXT("A\\"), true, TEXT("B"));
		RunRelChildTest(TEXT("A\\B"), TEXT("A\\"), true, TEXT("B"));
		RunRelChildTest(TEXT("A"), TEXT("A"), true, TEXT(""));
		RunRelChildTest(TEXT("A/"), TEXT("A"), true, TEXT(""));
		RunRelChildTest(TEXT("A\\"), TEXT("A"), true, TEXT(""));
		RunRelChildTest(TEXT("A"), TEXT("A/"), true, TEXT(""));
		RunRelChildTest(TEXT("A"), TEXT("A\\"), true, TEXT(""));
		RunRelChildTest(TEXT("../A"), TEXT(".."), true, TEXT("A"));
		RunRelChildTest(TEXT("/A/B"), TEXT("/A"), true, TEXT("B"));
		RunRelChildTest(TEXT("../A/B"), TEXT("../A"), true, TEXT("B"));
		RunRelChildTest(TEXT("../"), TEXT(".."), true, TEXT(""));
		RunRelChildTest(TEXT("C:/"), TEXT("C:/"), true, TEXT(""));
		RunRelChildTest(TEXT("C:/A"), TEXT("C:/"), true, TEXT("A"));
		RunRelChildTest(TEXT("//A"), TEXT("//A"), true, TEXT(""));
		RunRelChildTest(TEXT("//A/"), TEXT("//A"), true, TEXT(""));
		RunRelChildTest(TEXT("//A"), TEXT("//A/"), true, TEXT(""));
		RunRelChildTest(TEXT("//A/"), TEXT("//A/"), true, TEXT(""));
		RunRelChildTest(TEXT("//A/B"), TEXT("//A"), true, TEXT("B"));
		RunRelChildTest(TEXT("//A/B/"), TEXT("//A"), true, TEXT("B/"));
		RunRelChildTest(TEXT("//A/B"), TEXT("//A/"), true, TEXT("B"));
		RunRelChildTest(TEXT("//A/B/"), TEXT("//A/"), true, TEXT("B/"));

		RunRelChildTest(TEXT("//A/BFoo"), TEXT("//A/B"), false, TEXT(""));
		RunRelChildTest(TEXT("//A/C"), TEXT("//A/B"), false, TEXT(""));
		RunRelChildTest(TEXT("//A/C"), TEXT("C:/A"), false, TEXT(""));
		RunRelChildTest(TEXT("//A/C"), TEXT("/A"), false, TEXT(""));
		RunRelChildTest(TEXT("C:/A/BFoo"), TEXT("C:/A/B"), false, TEXT(""));
		RunRelChildTest(TEXT("C:/A/C"), TEXT("C:/A/B"), false, TEXT(""));
		RunRelChildTest(TEXT("C:/A/C"), TEXT("//A"), false, TEXT(""));
		RunRelChildTest(TEXT("C:/A/C"), TEXT("/A"), false, TEXT(""));
		RunRelChildTest(TEXT("/A/BFoo"), TEXT("/A/B"), false, TEXT(""));
		RunRelChildTest(TEXT("/A/C"), TEXT("/A/B"), false, TEXT(""));
		RunRelChildTest(TEXT("/A/C"), TEXT("C:/A"), false, TEXT(""));
		RunRelChildTest(TEXT("/A/C"), TEXT("//A"), false, TEXT(""));

		RunRelChildTest(TEXT("/"), TEXT("/"), true, TEXT(""));
		RunRelChildTest(TEXT("/"), TEXT("//"), false, TEXT(""));
		RunRelChildTest(TEXT("/"), TEXT("C"), false, TEXT(""));
		RunRelChildTest(TEXT("/"), TEXT("C:/"), false, TEXT(""));
		RunRelChildTest(TEXT("//"), TEXT("/"), false, TEXT(""));
		RunRelChildTest(TEXT("//"), TEXT("//"), true, TEXT(""));
		RunRelChildTest(TEXT("//"), TEXT("C"), false, TEXT(""));
		RunRelChildTest(TEXT("//"), TEXT("C:/"), false, TEXT(""));
		RunRelChildTest(TEXT("C"), TEXT("/"), false, TEXT(""));
		RunRelChildTest(TEXT("C"), TEXT("//"), false, TEXT(""));
		RunRelChildTest(TEXT("C"), TEXT("C"), true, TEXT(""));
		RunRelChildTest(TEXT("C"), TEXT("C:/"), false, TEXT(""));
		RunRelChildTest(TEXT("C:/"), TEXT("/"), false, TEXT(""));
		RunRelChildTest(TEXT("C:/"), TEXT("//"), false, TEXT(""));
		RunRelChildTest(TEXT("C:/"), TEXT("C"), false, TEXT(""));
		RunRelChildTest(TEXT("C:/"), TEXT("C:/"), true, TEXT(""));

		RunRelChildTest(TEXT("C:/A/B/C"), TEXT("C:/A"), true, TEXT("B/C"));
		RunRelChildTest(TEXT("C:/A"), TEXT("C:/A/B/C"), false, TEXT(""));

		// Correctly handle paths with invalid duplicate slashes
		RunRelChildTest(TEXT("///A"), TEXT("///"), true, TEXT("A"));
		RunRelChildTest(TEXT("////////////A"), TEXT("///"), true, TEXT("A"));
		RunRelChildTest(TEXT("///A"), TEXT("//"), true, TEXT("A"));
		RunRelChildTest(TEXT("////////////A"), TEXT("/"), false, TEXT(""));
		RunRelChildTest(TEXT("C://A"), TEXT("C://"), true, TEXT("A"));
		RunRelChildTest(TEXT("C://////////A"), TEXT("C://"), true, TEXT("A"));
		RunRelChildTest(TEXT("C://A"), TEXT("C:/"), true, TEXT("A"));
		RunRelChildTest(TEXT("C://////////A"), TEXT("C:/"), true, TEXT("A"));
		RunRelChildTest(TEXT("A//B"), TEXT("A//"), true, TEXT("B"));
		RunRelChildTest(TEXT("A///////////B"), TEXT("A//"), true, TEXT("B"));
		RunRelChildTest(TEXT("A//B"), TEXT("A/"), true, TEXT("B"));
		RunRelChildTest(TEXT("A///////////B"), TEXT("A/"), true, TEXT("B"));
		RunRelChildTest(TEXT("A//B"), TEXT("A"), true, TEXT("B"));
		RunRelChildTest(TEXT("A///////////B"), TEXT("A"), true, TEXT("B"));

		return true;
	}

	bool IsRelativePathTest()
	{
		auto RunRelTest = [this](const TCHAR* A, bool bExpected)
		{
			bool bActual = FPathViews::IsRelativePath(A);
			if (bActual != bExpected)
			{
				FAIL_CHECK(FString::Printf(TEXT("IsRelativePath(\"%s\") == %s, expected %s."), A, BoolToString(bActual), BoolToString(bExpected)));
			}
		};

		RunRelTest(TEXT("A"), true);
		RunRelTest(TEXT("A/"), true);
		RunRelTest(TEXT("A/B"), true);
		RunRelTest(TEXT("A\\B"), true);
		RunRelTest(TEXT("/A"), false);
		RunRelTest(TEXT("\\A"), false);
		RunRelTest(TEXT("/A/B"), false);
		RunRelTest(TEXT("//A"), false);
		RunRelTest(TEXT("\\\\A"), false);
		RunRelTest(TEXT("//A/B"), false);
		RunRelTest(TEXT("C:/A"), false);
		RunRelTest(TEXT("C:\\A"), false);
		RunRelTest(TEXT("C:/A/B"), false);

		return true;
	}
};

TEST_CASE_NAMED(FPathViewsCollapseDirectoriesTest, "System::Core::Misc::PathViews::CollapseDirectories", "[ApplicationContextMask][SmokeFilter]")
{
	TestCollapseRelativeDirectories<FPathViews, TStringBuilder<64>>();
}

TEST_CASE_NAMED(FPathViewsRemoveDuplicateSlashesTest, "System::Core::Misc::PathViews::RemoveDuplicateSlashes", "[ApplicationContextMask][SmokeFilter]")
{
	TestRemoveDuplicateSlashes<FPathViews, TStringBuilder<64>>();
}

TEST_CASE_NAMED(FPathViewsGetCleanFilenameTest, "System::Core::Misc::PathViews::GetCleanFilename", "[ApplicationContextMask][SmokeFilter]")
{
	FPathViewsTest Instance = FPathViewsTest();
	Instance.GetCleanFilenameTest();	
}

TEST_CASE_NAMED(FPathViewsGetBaseFilenameTest, "System::Core::Misc::PathViews::GetBaseFilename", "[ApplicationContextMask][SmokeFilter]")
{
	FPathViewsTest Instance = FPathViewsTest();
	Instance.GetBaseFilenameTest();
}

TEST_CASE_NAMED(FPathViewsGetPathTest, "System::Core::Misc::PathViews::GetPath", "[ApplicationContextMask][SmokeFilter]")
{
	FPathViewsTest Instance = FPathViewsTest();
	Instance.GetPathTest();
}

TEST_CASE_NAMED(FPathViewsGetExtensionTest, "System::Core::Misc::PathViews::GetExtension", "[ApplicationContextMask][SmokeFilter]")
{
	FPathViewsTest Instance = FPathViewsTest();
	Instance.GetExtensionTest();
}

TEST_CASE_NAMED(FPathViewsGetPathLeafTest, "System::Core::Misc::PathViews::GetPathLeaf", "[ApplicationContextMask][SmokeFilter]")
{
	FPathViewsTest Instance = FPathViewsTest();
	Instance.GetPathLeafTest();
}

TEST_CASE_NAMED(FPathViewsIsPathLeafTest, "System::Core::Misc::PathViews::IsPathLeaf", "[ApplicationContextMask][SmokeFilter]")
{
	FPathViewsTest Instance = FPathViewsTest();
	Instance.IsPathLeafTest();
}

TEST_CASE_NAMED(FPathViewsSplitTest, "System::Core::Misc::PathViews::Split", "[ApplicationContextMask][SmokeFilter]")
{
	auto RunSplitTest = [](const TCHAR* InPath, const TCHAR* InExpectedPath, const TCHAR* InExpectedName, const TCHAR* InExpectedExt)
	{
		FStringView SplitPath, SplitName, SplitExt;
		FPathViews::Split(InPath, SplitPath, SplitName, SplitExt);
		if (SplitPath != InExpectedPath || SplitName != InExpectedName || SplitExt != InExpectedExt)
		{
			FAIL_CHECK(FString::Printf(TEXT("Failed to split path '%s' (got ('%.*s', '%.*s', '%.*s'), expected ('%s', '%s', '%s'))."), InPath,
				SplitPath.Len(), SplitPath.GetData(), SplitName.Len(), SplitName.GetData(), SplitExt.Len(), SplitExt.GetData(),
				InExpectedPath, InExpectedName, InExpectedExt));
		}
	};

	RunSplitTest(TEXT(""), TEXT(""), TEXT(""), TEXT(""));
	RunSplitTest(TEXT(".txt"), TEXT(""), TEXT(""), TEXT("txt"));
	RunSplitTest(TEXT(".tar.gz"), TEXT(""), TEXT(".tar"), TEXT("gz"));
	RunSplitTest(TEXT(".tar.gz/"), TEXT(".tar.gz"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT(".tar.gz\\"), TEXT(".tar.gz"), TEXT(""), TEXT(""));
	// TEXT(".") is ambiguous; we currently treat it as an extension separator but we don't guarantee that in our contract
	//RunSplitTest(TEXT("."), TEXT(""), TEXT(""), TEXT(""));
	RunSplitTest(TEXT(".."), TEXT(""), TEXT(".."), TEXT(""));
	RunSplitTest(TEXT("File"), TEXT(""), TEXT("File"), TEXT(""));
	RunSplitTest(TEXT("File.txt"), TEXT(""), TEXT("File"), TEXT("txt"));
	RunSplitTest(TEXT("File.tar.gz"), TEXT(""), TEXT("File.tar"), TEXT("gz"));
	RunSplitTest(TEXT("File.tar.gz/"), TEXT("File.tar.gz"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT("File.tar.gz\\"), TEXT("File.tar.gz"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT("C:/Folder/"), TEXT("C:/Folder"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT("C:/Folder/File"), TEXT("C:/Folder"), TEXT("File"), TEXT(""));
	RunSplitTest(TEXT("C:/Folder/File.txt"), TEXT("C:/Folder"), TEXT("File"), TEXT("txt"));
	RunSplitTest(TEXT("C:/Folder/File.tar.gz"), TEXT("C:/Folder"), TEXT("File.tar"), TEXT("gz"));
	RunSplitTest(TEXT("C:/Folder/First.Last/File"), TEXT("C:/Folder/First.Last"), TEXT("File"), TEXT(""));
	RunSplitTest(TEXT("C:/Folder/First.Last/File.txt"), TEXT("C:/Folder/First.Last"), TEXT("File"), TEXT("txt"));
	RunSplitTest(TEXT("C:/Folder/First.Last/File.tar.gz"), TEXT("C:/Folder/First.Last"), TEXT("File.tar"), TEXT("gz"));
	RunSplitTest(TEXT("C:\\Folder\\"), TEXT("C:\\Folder"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT("C:\\Folder\\File"), TEXT("C:\\Folder"), TEXT("File"), TEXT(""));
	RunSplitTest(TEXT("C:\\Folder\\First.Last\\"), TEXT("C:\\Folder\\First.Last"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT("C:\\Folder\\First.Last\\File"), TEXT("C:\\Folder\\First.Last"), TEXT("File"), TEXT(""));
	RunSplitTest(TEXT("C:\\Folder\\First.Last\\File.txt"), TEXT("C:\\Folder\\First.Last"), TEXT("File"), TEXT("txt"));
	RunSplitTest(TEXT("C:\\Folder\\First.Last\\File.tar.gz"), TEXT("C:\\Folder\\First.Last"), TEXT("File.tar"), TEXT("gz"));
}

TEST_CASE_NAMED(FPathViewsIterateTest, "System::Core::Misc::PathViews::IterateComponents", "[ApplicationContextMask][SmokeFilter]")
{
	auto RunIterateTest = [](const TCHAR* InPath, auto&&... InExpectedComponents) {
		TArray<FStringView> Components;
		TArray<FStringView> ExpectedComponents({ Forward<decltype(InExpectedComponents)>(InExpectedComponents)... });
		FPathViews::IterateComponents(InPath, [&Components](FStringView Comp) { Components.Add(Comp); });
		if (!Algo::Compare(Components, ExpectedComponents))
		{
			FAIL_CHECK(FString::Printf(TEXT("Incorrect components splitting %s - got %d, expected %d. Got components (%s), expected components (%s)"),
				InPath,
				Components.Num(),
				ExpectedComponents.Num(),
				*FString::Join(Components, TEXT(",")),
				*FString::Join(ExpectedComponents, TEXT(","))));
		}
	};

	RunIterateTest(TEXT(""), TEXT(""));
	RunIterateTest(TEXT(".txt"), TEXT(".txt"));
	RunIterateTest(TEXT(".tar.gz"), TEXT(".tar.gz"));
	RunIterateTest(TEXT(".tar.gz/"), TEXT(".tar.gz"), TEXT(""));
	RunIterateTest(TEXT(".tar.gz\\"), TEXT(".tar.gz"), TEXT(""));
	RunIterateTest(TEXT("."), TEXT("."));
	RunIterateTest(TEXT(".."), TEXT(".."));
	RunIterateTest(TEXT("File"), TEXT("File"));
	RunIterateTest(TEXT("File.txt"), TEXT("File.txt"));
	RunIterateTest(TEXT("File.tar.gz"), TEXT("File.tar.gz"));
	RunIterateTest(TEXT("File.tar.gz/"), TEXT("File.tar.gz"), TEXT(""));
	RunIterateTest(TEXT("File.tar.gz\\"), TEXT("File.tar.gz"), TEXT(""));

	RunIterateTest(TEXT("C:/Folder"), TEXT("C:"), TEXT("Folder"));
	RunIterateTest(TEXT("C:/Folder/"), TEXT("C:"), TEXT("Folder"), TEXT(""));
	RunIterateTest(TEXT("C:/Folder/File"), TEXT("C:"), TEXT("Folder"), TEXT("File"));
	RunIterateTest(TEXT("C:/Folder/File.txt"), TEXT("C:"), TEXT("Folder"), TEXT("File.txt"));
	RunIterateTest(TEXT("C:/Folder/File.tar.gz"), TEXT("C:"), TEXT("Folder"), TEXT("File.tar.gz"));
	RunIterateTest(TEXT("C:/Folder/First.Last/File"), TEXT("C:"), TEXT("Folder"), TEXT("First.Last"), TEXT("File"));
	RunIterateTest(TEXT("C:/Folder/First.Last/File.txt"), TEXT("C:"), TEXT("Folder"), TEXT("First.Last"), TEXT("File.txt"));
	RunIterateTest(TEXT("C:/Folder/First.Last/File.tar.gz"), TEXT("C:"), TEXT("Folder"), TEXT("First.Last"), TEXT("File.tar.gz"));

	RunIterateTest(TEXT("C:\\Folder"), TEXT("C:"), TEXT("Folder"));
	RunIterateTest(TEXT("C:\\Folder\\"), TEXT("C:"), TEXT("Folder"), TEXT(""));
	RunIterateTest(TEXT("C:\\Folder\\File"), TEXT("C:"), TEXT("Folder"), TEXT("File"));
	RunIterateTest(TEXT("C:\\Folder\\First.Last\\"), TEXT("C:"), TEXT("Folder"), TEXT("First.Last"), TEXT(""));
	RunIterateTest(TEXT("C:\\Folder\\First.Last\\File"), TEXT("C:"), TEXT("Folder"), TEXT("First.Last"), TEXT("File"));
	RunIterateTest(TEXT("C:\\Folder\\First.Last\\File.txt"), TEXT("C:"), TEXT("Folder"), TEXT("First.Last"), TEXT("File.txt"));
	RunIterateTest(TEXT("C:\\Folder\\First.Last\\File.tar.gz"), TEXT("C:"), TEXT("Folder"), TEXT("First.Last"), TEXT("File.tar.gz"));

	RunIterateTest(TEXT("//ShareName/Path"), TEXT(""), TEXT(""), TEXT("ShareName"), TEXT("Path"));
	RunIterateTest(TEXT("//ShareName/Path/"), TEXT(""), TEXT(""), TEXT("ShareName"), TEXT("Path"), TEXT(""));
	RunIterateTest(TEXT("//ShareName/Path/Sub"), TEXT(""), TEXT(""), TEXT("ShareName"), TEXT("Path"), TEXT("Sub"));
	RunIterateTest(TEXT("//ShareName/Path/Sub/File"), TEXT(""), TEXT(""), TEXT("ShareName"), TEXT("Path"), TEXT("Sub"), TEXT("File"));
	RunIterateTest(TEXT("//ShareName/Path/Sub/File.txt"), TEXT(""), TEXT(""), TEXT("ShareName"), TEXT("Path"), TEXT("Sub"), TEXT("File.txt"));
	RunIterateTest(TEXT("//ShareName/Path/Sub/File.tar.gz"), TEXT(""), TEXT(""), TEXT("ShareName"), TEXT("Path"), TEXT("Sub"), TEXT("File.tar.gz"));

	RunIterateTest(TEXT("/MountPoint"), TEXT(""), TEXT("MountPoint"));
	RunIterateTest(TEXT("/MountPoint/"), TEXT(""), TEXT("MountPoint"), TEXT(""));
	RunIterateTest(TEXT("/MountPoint/Sub"), TEXT(""), TEXT("MountPoint"), TEXT("Sub"));
	RunIterateTest(TEXT("/MountPoint/Sub/File"), TEXT(""), TEXT("MountPoint"), TEXT("Sub"), TEXT("File"));
	RunIterateTest(TEXT("/MountPoint/Sub/File.txt"), TEXT(""), TEXT("MountPoint"), TEXT("Sub"), TEXT("File.txt"));
	RunIterateTest(TEXT("/MountPoint/Sub/File.tar.gz"), TEXT(""), TEXT("MountPoint"), TEXT("Sub"), TEXT("File.tar.gz"));
}

TEST_CASE_NAMED(FPathViewsAncestorTest, "System::Core::Misc::PathViews::IterateAncestors", "[ApplicationContextMask][SmokeFilter]")
{
	auto RunAncestorTest = [](FStringView InPath, auto&&... Args) {
		TArray<FStringView> Ancestors;
		TArray<FStringView> ExpectedAncestors{ Args... };
		FPathViews::IterateAncestors(InPath, [&Ancestors](FStringView Comp) { Ancestors.Add(Comp); return true; });
		if (!Algo::Compare(Ancestors, ExpectedAncestors))
		{
			TStringBuilder<256> Builder;
			Builder << TEXT("Incorrect ancestors splitting ") << InPath << TEXT("- got ") << Ancestors.Num() << TEXT(", expected ") << ExpectedAncestors.Num();
			Builder << LINE_TERMINATOR TEXT("Got ancestors      (");
			Builder.Join(Ancestors, TEXT(","));
			Builder << LINE_TERMINATOR TEXT("Expected ancestors (");
			Builder.Join(ExpectedAncestors, TEXT(", ")) << TEXT(")");
			FAIL_CHECK(Builder.ToString());
		}
	};

	RunAncestorTest(TEXTVIEW(""), TEXTVIEW(""));
	RunAncestorTest(TEXTVIEW("/"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("\\"), TEXTVIEW("\\"));
	RunAncestorTest(TEXTVIEW(".txt"), TEXTVIEW(".txt"));
	RunAncestorTest(TEXTVIEW(".tar.gz"), TEXTVIEW(".tar.gz"));
	RunAncestorTest(TEXTVIEW(".tar.gz/"), TEXTVIEW(".tar.gz"));
	RunAncestorTest(TEXTVIEW(".tar.gz\\"), TEXTVIEW(".tar.gz"));
	RunAncestorTest(TEXTVIEW("."), TEXTVIEW("."));
	RunAncestorTest(TEXTVIEW(".."), TEXTVIEW(".."));
	RunAncestorTest(TEXTVIEW("File"), TEXTVIEW("File"));
	RunAncestorTest(TEXTVIEW("File/"), TEXTVIEW("File"));
	RunAncestorTest(TEXTVIEW("File.txt"), TEXTVIEW("File.txt"));
	RunAncestorTest(TEXTVIEW("File.tar.gz"), TEXTVIEW("File.tar.gz"));
	RunAncestorTest(TEXTVIEW("File.tar.gz/"), TEXTVIEW("File.tar.gz"));
	RunAncestorTest(TEXTVIEW("File.tar.gz\\"), TEXTVIEW("File.tar.gz"));

	RunAncestorTest(TEXTVIEW("C:/Folder"), TEXTVIEW("C:/Folder"), TEXTVIEW("C:/"));
	RunAncestorTest(TEXTVIEW("C:/Folder/"), TEXTVIEW("C:/Folder"), TEXTVIEW("C:/"));
	RunAncestorTest(TEXTVIEW("C:/Folder/File"), TEXTVIEW("C:/Folder/File"), TEXTVIEW("C:/Folder"), TEXTVIEW("C:/"));
	RunAncestorTest(TEXTVIEW("C:/Folder/File.txt"), TEXTVIEW("C:/Folder/File.txt"), TEXTVIEW("C:/Folder"), TEXTVIEW("C:/"));
	RunAncestorTest(TEXTVIEW("C:/Folder/File.tar.gz"), TEXTVIEW("C:/Folder/File.tar.gz"), TEXTVIEW("C:/Folder"), TEXTVIEW("C:/"));
	RunAncestorTest(TEXTVIEW("C:/Folder/First.Last/File"), TEXTVIEW("C:/Folder/First.Last/File"), TEXTVIEW("C:/Folder/First.Last"), TEXTVIEW("C:/Folder"), TEXTVIEW("C:/"));
	RunAncestorTest(TEXTVIEW("C:/Folder/First.Last/File.txt"), TEXTVIEW("C:/Folder/First.Last/File.txt"), TEXTVIEW("C:/Folder/First.Last"), TEXTVIEW("C:/Folder"), TEXTVIEW("C:/"));
	RunAncestorTest(TEXTVIEW("C:/Folder/First.Last/File.tar.gz"), TEXTVIEW("C:/Folder/First.Last/File.tar.gz"), TEXTVIEW("C:/Folder/First.Last"), TEXTVIEW("C:/Folder"), TEXTVIEW("C:/"));

	// Windows path specifying drive letter but relative path rather than absolute
	// Not well handled elsewhere in the engine but we parse it here
	RunAncestorTest(TEXTVIEW("C:Folder/File"), TEXTVIEW("C:Folder/File"), TEXTVIEW("C:Folder"), TEXTVIEW("C:"));

	RunAncestorTest(TEXTVIEW("C:\\Folder"), TEXTVIEW("C:\\Folder"), TEXTVIEW("C:\\"));
	RunAncestorTest(TEXTVIEW("C:\\Folder\\"), TEXTVIEW("C:\\Folder"), TEXTVIEW("C:\\"));
	RunAncestorTest(TEXTVIEW("C:\\Folder\\File"), TEXTVIEW("C:\\Folder\\File"), TEXTVIEW("C:\\Folder"), TEXTVIEW("C:\\"));
	RunAncestorTest(TEXTVIEW("C:\\Folder\\First.Last\\"), TEXTVIEW("C:\\Folder\\First.Last"), TEXTVIEW("C:\\Folder"), TEXTVIEW("C:\\"));
	RunAncestorTest(TEXTVIEW("C:\\Folder\\First.Last\\File"), TEXTVIEW("C:\\Folder\\First.Last\\File"), TEXTVIEW("C:\\Folder\\First.Last"), TEXTVIEW("C:\\Folder"), TEXTVIEW("C:\\"));
	RunAncestorTest(TEXTVIEW("C:\\Folder\\First.Last\\File.txt"), TEXTVIEW("C:\\Folder\\First.Last\\File.txt"), TEXTVIEW("C:\\Folder\\First.Last"), TEXTVIEW("C:\\Folder"), TEXTVIEW("C:\\"));
	RunAncestorTest(TEXTVIEW("C:\\Folder\\First.Last\\File.tar.gz"), TEXTVIEW("C:\\Folder\\First.Last\\File.tar.gz"), TEXTVIEW("C:\\Folder\\First.Last"), TEXTVIEW("C:\\Folder"), TEXTVIEW("C:\\"));

	RunAncestorTest(TEXTVIEW("/MountPoint"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/MountPoint/"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/MountPoint/Sub"), TEXTVIEW("/MountPoint/Sub"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/MountPoint/Sub/File"), TEXTVIEW("/MountPoint/Sub/File"), TEXTVIEW("/MountPoint/Sub"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/MountPoint/Sub/File.txt"), TEXTVIEW("/MountPoint/Sub/File.txt"), TEXTVIEW("/MountPoint/Sub"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/MountPoint/Sub/File.tar.gz"), TEXTVIEW("/MountPoint/Sub/File.tar.gz"), TEXTVIEW("/MountPoint/Sub"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));

	RunAncestorTest(TEXTVIEW("\\\\ShareName/Path"), TEXTVIEW("\\\\ShareName/Path"), TEXTVIEW("\\\\ShareName"));
	RunAncestorTest(TEXTVIEW("\\\\ShareName/Path/"), TEXTVIEW("\\\\ShareName/Path"), TEXTVIEW("\\\\ShareName"));
	RunAncestorTest(TEXTVIEW("\\\\ShareName/Path/Sub"), TEXTVIEW("\\\\ShareName/Path/Sub"), TEXTVIEW("\\\\ShareName/Path"), TEXTVIEW("\\\\ShareName"));
	RunAncestorTest(TEXTVIEW("\\\\ShareName/Path/Sub/File"), TEXTVIEW("\\\\ShareName/Path/Sub/File"), TEXTVIEW("\\\\ShareName/Path/Sub"), TEXTVIEW("\\\\ShareName/Path"), TEXTVIEW("\\\\ShareName"));
	RunAncestorTest(TEXTVIEW("\\\\ShareName/Path/Sub/File.txt"), TEXTVIEW("\\\\ShareName/Path/Sub/File.txt"), TEXTVIEW("\\\\ShareName/Path/Sub"), TEXTVIEW("\\\\ShareName/Path"), TEXTVIEW("\\\\ShareName"));
	RunAncestorTest(TEXTVIEW("\\\\ShareName/Path/Sub/File.tar.gz"), TEXTVIEW("\\\\ShareName/Path/Sub/File.tar.gz"), TEXTVIEW("\\\\ShareName/Path/Sub"), TEXTVIEW("\\\\ShareName/Path"), TEXTVIEW("\\\\ShareName"));

	// Non-ascii path characters
	RunAncestorTest(TEXTVIEW("/MountPoint/Sub/\u30D5\u30A1\u30A4\u30EB"), TEXTVIEW("/MountPoint/Sub/\u30D5\u30A1\u30A4\u30EB"), TEXTVIEW("/MountPoint/Sub"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/MountPoint/Sub/\u30D5\u30A1\u30A4\u30EB.txt"), TEXTVIEW("/MountPoint/Sub/\u30D5\u30A1\u30A4\u30EB.txt"), TEXTVIEW("/MountPoint/Sub"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/MountPoint/Sub/\u30D5\u30A1\u30A4\u30EB.tar.gz"), TEXTVIEW("/MountPoint/Sub/\u30D5\u30A1\u30A4\u30EB.tar.gz"), TEXTVIEW("/MountPoint/Sub"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/MountPoint/\u30B5\u30D6/File"), TEXTVIEW("/MountPoint/\u30B5\u30D6/File"), TEXTVIEW("/MountPoint/\u30B5\u30D6"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/\u6839/Sub/File"), TEXTVIEW("/\u6839/Sub/File"), TEXTVIEW("/\u6839/Sub"), TEXTVIEW("/\u6839"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/\u6839/\u30B5\u30D6/\u30D5\u30A1\u30A4\u30EB"), TEXTVIEW("/\u6839/\u30B5\u30D6/\u30D5\u30A1\u30A4\u30EB"), TEXTVIEW("/\u6839/\u30B5\u30D6"), TEXTVIEW("/\u6839"), TEXTVIEW("/"));

	// Duplicated path separators
	RunAncestorTest(TEXTVIEW("//MountPoint"), TEXTVIEW("//MountPoint"));     // Double leadings slashes treated like a windows share
	RunAncestorTest(TEXTVIEW("//\\MountPoint"), TEXTVIEW("//\\MountPoint")); // SplitVolumeSpecifier treats this as a root
	RunAncestorTest(TEXTVIEW("/\\MountPoint"), TEXTVIEW("/\\MountPoint"));   // Looks like a share name to SplitVolumeSpecifier
	RunAncestorTest(TEXTVIEW("/MountPoint//"), TEXTVIEW("/MountPoint"), TEXT("/"));
	RunAncestorTest(TEXTVIEW("/MountPoint//Sub"),
		TEXTVIEW("/MountPoint//Sub"), // We don't remove the internal duplication because we're using the source string
		TEXTVIEW("/MountPoint"),      // We strip the extras when moving to the next ancestor
		TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("/MountPoint/Sub//File"), TEXTVIEW("/MountPoint/Sub//File"), TEXTVIEW("/MountPoint/Sub"), TEXTVIEW("/MountPoint"), TEXTVIEW("/"));
	RunAncestorTest(TEXTVIEW("\\\\\\NotShareName/Path"), TEXTVIEW("\\\\\\NotShareName/Path"), TEXTVIEW("\\\\\\NotShareName")); // SplitVolumeSpecifier allows extra separators after the two-separator prefix
}

TEST_CASE_NAMED(FPathViewsAppendTest, "System::Core::Misc::PathViews::Append", "[ApplicationContextMask][SmokeFilter]")
{
	TStringBuilder<256> Path;

	FPathViews::Append(Path, TEXT("A"), TEXT(""));
	CHECK_EQUALS(TEXT("FPathViews::Append('A', '')"), FStringView(Path), TEXTVIEW("A/"));
	Path.Reset();

	FPathViews::Append(Path, TEXT(""), TEXT("B"));
	CHECK_EQUALS(TEXT("FPathViews::Append('', 'B')"), FStringView(Path), TEXTVIEW("B"));
	Path.Reset();

	FPathViews::Append(Path, TEXT("/"), TEXT("B"));
	CHECK_EQUALS(TEXT("FPathViews::Append('/', 'B')"), FStringView(Path), TEXTVIEW("/B"));
	Path.Reset();

	FPathViews::Append(Path, TEXT("A"), TEXT("B"));
	CHECK_EQUALS(TEXT("FPathViews::Append('A', 'B')"), FStringView(Path), TEXTVIEW("A/B"));
	Path.Reset();

	FPathViews::Append(Path, TEXT("A/"), TEXT("B"));
	CHECK_EQUALS(TEXT("FPathViews::Append('A/', 'B')"), FStringView(Path), TEXTVIEW("A/B"));
	Path.Reset();

	FPathViews::Append(Path, TEXT("A\\"), TEXT("B"));
	CHECK_EQUALS(TEXT("FPathViews::Append('A\\', 'B')"), FStringView(Path), TEXTVIEW("A\\B"));
	Path.Reset();

	FPathViews::Append(Path, TEXT("A/B"), TEXT("C/D"));
	CHECK_EQUALS(TEXT("FPathViews::Append('A/B', 'C/D')"), FStringView(Path), TEXTVIEW("A/B/C/D"));
	Path.Reset();

	FPathViews::Append(Path, TEXT("A/"), TEXT("B"), TEXT("C/"), TEXT("D"));
	CHECK_EQUALS(TEXT("FPathViews::Append('A/', 'B', 'C/', 'D')"), FStringView(Path), TEXTVIEW("A/B/C/D"));
	Path.Reset();

	FPathViews::Append(Path, TEXT("A/"), 16, TEXT("B"));
	CHECK_EQUALS(TEXT("FPathViews::Append('A/', 16, 'B')"), FStringView(Path), TEXTVIEW("A/16/B"));
	Path.Reset();
}

TEST_CASE_NAMED(FPathViewsChangeExtensionTest, "System::Core::Misc::PathViews::ChangeExtension", "[ApplicationContextMask][SmokeFilter]")
{
	auto RunChangeExtensionTest = [](const TCHAR* InPath, const TCHAR* InNewExt, const TCHAR* InExpectedPath)
	{
		// Run test
		const FString NewPath = FPathViews::ChangeExtension(InPath, InNewExt);
		if (NewPath != InExpectedPath)
		{
			FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to change the extension (got '%s', expected '%s')."), InPath, *NewPath, InExpectedPath));
		}
	};

	RunChangeExtensionTest(nullptr, nullptr, TEXT(""));
	RunChangeExtensionTest(TEXT(""), TEXT(""), TEXT(""));
	RunChangeExtensionTest(TEXT(""), TEXT(".txt"), TEXT(""));
	RunChangeExtensionTest(TEXT("file"), TEXT("log"), TEXT("file"));
	RunChangeExtensionTest(TEXT("file.txt"), TEXT("log"), TEXT("file.log"));
	RunChangeExtensionTest(TEXT("file.tar.gz"), TEXT("gz2"), TEXT("file.tar.gz2"));
	RunChangeExtensionTest(TEXT("file.txt"), TEXT(""), TEXT("file"));
	RunChangeExtensionTest(TEXT("C:/Folder/file"), TEXT("log"), TEXT("C:/Folder/file"));
	RunChangeExtensionTest(TEXT("C:/Folder/file.txt"), TEXT("log"), TEXT("C:/Folder/file.log"));
	RunChangeExtensionTest(TEXT("C:/Folder/file.tar.gz"), TEXT("gz2"), TEXT("C:/Folder/file.tar.gz2"));
	RunChangeExtensionTest(TEXT("C:/Folder/First.Last/file"), TEXT("log"), TEXT("C:/Folder/First.Last/file"));
	RunChangeExtensionTest(TEXT("C:/Folder/First.Last/file.txt"), TEXT("log"), TEXT("C:/Folder/First.Last/file.log"));
	RunChangeExtensionTest(TEXT("C:/Folder/First.Last/file.tar.gz"), TEXT("gz2"), TEXT("C:/Folder/First.Last/file.tar.gz2"));
}

TEST_CASE_NAMED(FPathViewsSetExtensionTest, "System::Core::Misc::PathViews::SetExtension", "[ApplicationContextMask][SmokeFilter]")
{
	auto RunSetExtensionTest = [](const TCHAR* InPath, const TCHAR* InNewExt, const TCHAR* InExpectedPath)
	{
		// Run test
		const FString NewPath = FPathViews::SetExtension(InPath, InNewExt);
		if (NewPath != InExpectedPath)
		{
			FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to set the extension (got '%s', expected '%s')."), InPath, *NewPath, InExpectedPath));
		}
	};

	RunSetExtensionTest(nullptr, nullptr, TEXT(""));
	RunSetExtensionTest(TEXT(""), TEXT(""), TEXT(""));
	RunSetExtensionTest(TEXT(""), TEXT(".txt"), TEXT(".txt"));
	RunSetExtensionTest(TEXT("file"), TEXT("log"), TEXT("file.log"));
	RunSetExtensionTest(TEXT("file.txt"), TEXT("log"), TEXT("file.log"));
	RunSetExtensionTest(TEXT("file.tar.gz"), TEXT("gz2"), TEXT("file.tar.gz2"));
	RunSetExtensionTest(TEXT("file.txt"), TEXT(""), TEXT("file"));
	RunSetExtensionTest(TEXT("C:/Folder/file"), TEXT("log"), TEXT("C:/Folder/file.log"));
	RunSetExtensionTest(TEXT("C:/Folder/file.txt"), TEXT("log"), TEXT("C:/Folder/file.log"));
	RunSetExtensionTest(TEXT("C:/Folder/file.tar.gz"), TEXT("gz2"), TEXT("C:/Folder/file.tar.gz2"));
	RunSetExtensionTest(TEXT("C:/Folder/First.Last/file"), TEXT("log"), TEXT("C:/Folder/First.Last/file.log"));
	RunSetExtensionTest(TEXT("C:/Folder/First.Last/file.txt"), TEXT("log"), TEXT("C:/Folder/First.Last/file.log"));
	RunSetExtensionTest(TEXT("C:/Folder/First.Last/file.tar.gz"), TEXT("gz2"), TEXT("C:/Folder/First.Last/file.tar.gz2"));
}


TEST_CASE_NAMED(FPathViewsEqualsAndLessTest, "System::Core::Misc::PathViews::EqualsAndLess", "[ApplicationContextMask][SmokeFilter]")
{
	FPathViewsTest Instance = FPathViewsTest();
	Instance.EqualsAndLessTest();
}

TEST_CASE_NAMED(FPathViewsTryMakeChildPathRelativeToTest, "System::Core::Misc::PathViews::TryMakeChildPathRelativeTo", "[ApplicationContextMask][SmokeFilter]")
{
	FPathViewsTest Instance = FPathViewsTest();
	Instance.TryMakeChildPathRelativeToTest();
}

TEST_CASE_NAMED(FPathViewsIsRelativePathTest, "System::Core::Misc::PathViews::IsRelativePath", "[ApplicationContextMask][SmokeFilter]")
{
	FPathViewsTest Instance = FPathViewsTest();
	Instance.IsRelativePathTest();
}

TEST_CASE_NAMED(FPathViewsHasRedundantTerminatingSeparatorTest, "System::Core::Misc::PathViews::HasRedundantTerminatingSeparator", "[ApplicationContextMask][SmokeFilter]")
{
	CHECK_EQUALS(TEXT(""), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("")), false);
	CHECK_EQUALS(TEXT("/"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("/")), false);
	CHECK_EQUALS(TEXT("//"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("//")), false);
	CHECK_EQUALS(TEXT("///"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("///")), true);
	CHECK_EQUALS(TEXT("\\"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("\\")), false);
	CHECK_EQUALS(TEXT("\\\\"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("\\\\")), false);
	CHECK_EQUALS(TEXT("\\\\\\"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("\\\\\\")), true);
	CHECK_EQUALS(TEXT("text"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("text")), false);
	CHECK_EQUALS(TEXT("text/"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("text/")), true);
	CHECK_EQUALS(TEXT("text//"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("text//")), true);
	CHECK_EQUALS(TEXT("text\\"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("text\\")), true);
	CHECK_EQUALS(TEXT("text\\\\"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("text\\\\")), true);
	CHECK_EQUALS(TEXT("D:"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("D:")), false);
	CHECK_EQUALS(TEXT("D:/"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("D:/")), false);
	CHECK_EQUALS(TEXT("D://"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("D://")), true);
	CHECK_EQUALS(TEXT("D:\\"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("D:\\")), false);
	CHECK_EQUALS(TEXT("D:\\\\"), FPathViews::HasRedundantTerminatingSeparator(TEXTVIEW("D:\\\\")), true);
}


TEST_CASE_NAMED(FPathViewsSplitFirstComponentTest, "System::Core::Misc::PathViews::SplitFirstComponent", "[ApplicationContextMask][SmokeFilter]")
{
	auto RunSplitFirstTest = [](const TCHAR* FullPath, const TCHAR* ExpectedFirst, const TCHAR* ExpectedRemaining)
	{
		FStringView ActualFirst;
		FStringView ActualRemaining;
		FPathViews::SplitFirstComponent(FullPath, ActualFirst, ActualRemaining);
		if (!ActualFirst.Equals(ExpectedFirst, ESearchCase::IgnoreCase) || !ActualRemaining.Equals(ExpectedRemaining, ESearchCase::IgnoreCase))
		{
			FAIL_CHECK(FString::Printf(TEXT("SplitFirstComponent(\"%s\") == (\"%.*s\", \"%.*s\"), expected (\"%s\", \"%s\")."),
				FullPath, ActualFirst.Len(), ActualFirst.GetData(), ActualRemaining.Len(), ActualRemaining.GetData(),
				ExpectedFirst, ExpectedRemaining));
		}
	};

	RunSplitFirstTest(TEXT(""), TEXT(""), TEXT(""));

	RunSplitFirstTest(TEXT("A"), TEXT("A"), TEXT(""));
	RunSplitFirstTest(TEXT("A/"), TEXT("A"), TEXT(""));
	RunSplitFirstTest(TEXT("A\\"), TEXT("A"), TEXT(""));
	RunSplitFirstTest(TEXT("A/B"), TEXT("A"), TEXT("B"));
	RunSplitFirstTest(TEXT("A\\B"), TEXT("A"), TEXT("B"));
	RunSplitFirstTest(TEXT("A/B/"), TEXT("A"), TEXT("B/"));
	RunSplitFirstTest(TEXT("A\\B\\"), TEXT("A"), TEXT("B\\"));
	RunSplitFirstTest(TEXT("A/B/C"), TEXT("A"), TEXT("B/C"));
	RunSplitFirstTest(TEXT("A\\B\\C"), TEXT("A"), TEXT("B\\C"));

	RunSplitFirstTest(TEXT("/A"), TEXT("/"), TEXT("A"));
	RunSplitFirstTest(TEXT("\\A"), TEXT("\\"), TEXT("A"));
	RunSplitFirstTest(TEXT("/A/"), TEXT("/"), TEXT("A/"));
	RunSplitFirstTest(TEXT("\\A\\"), TEXT("\\"), TEXT("A\\"));
	RunSplitFirstTest(TEXT("/A/B"), TEXT("/"), TEXT("A/B"));
	RunSplitFirstTest(TEXT("\\A\\B"), TEXT("\\"), TEXT("A\\B"));

	RunSplitFirstTest(TEXT("//A"), TEXT("//"), TEXT("A"));
	RunSplitFirstTest(TEXT("\\\\A"), TEXT("\\\\"), TEXT("A"));
	RunSplitFirstTest(TEXT("//A/"), TEXT("//"), TEXT("A/"));
	RunSplitFirstTest(TEXT("\\\\A\\"), TEXT("\\\\"), TEXT("A\\"));
	RunSplitFirstTest(TEXT("//A/B"), TEXT("//"), TEXT("A/B"));
	RunSplitFirstTest(TEXT("\\\\A\\B"), TEXT("\\\\"), TEXT("A\\B"));

	RunSplitFirstTest(TEXT("C:/A"), TEXT("C:/"), TEXT("A"));
	RunSplitFirstTest(TEXT("C:\\A"), TEXT("C:\\"), TEXT("A"));
	RunSplitFirstTest(TEXT("C:/A/"), TEXT("C:/"), TEXT("A/"));
	RunSplitFirstTest(TEXT("C:\\A\\"), TEXT("C:\\"), TEXT("A\\"));
	RunSplitFirstTest(TEXT("C:/A/B"), TEXT("C:/"), TEXT("A/B"));
	RunSplitFirstTest(TEXT("C:\\A\\B"), TEXT("C:\\"), TEXT("A\\B"));

	// Correctly handle paths with invalid duplicate slashes
	RunSplitFirstTest(TEXT("///A"), TEXT("//"), TEXT("A"));
	RunSplitFirstTest(TEXT("////////////A"), TEXT("//"), TEXT("A"));
	RunSplitFirstTest(TEXT("///A"), TEXT("//"), TEXT("A"));
	RunSplitFirstTest(TEXT("C://A"), TEXT("C:/"), TEXT("A"));
	RunSplitFirstTest(TEXT("C://////////A"), TEXT("C:/"), TEXT("A"));
	RunSplitFirstTest(TEXT("A//B"), TEXT("A"), TEXT("B"));
	RunSplitFirstTest(TEXT("A///////////B"), TEXT("A"), TEXT("B"));

}

TEST_CASE_NAMED(FPathViewsAppendPathTest, "System::Core::Misc::PathViews::AppendPath", "[ApplicationContextMask][SmokeFilter]")
{
	auto RunAppendTest = [](const TCHAR* Base, const TCHAR* Append, const TCHAR* ExpectedNewBase)
	{
		TStringBuilder<128> BaseBuilder;
		BaseBuilder << Base;
		FPathViews::AppendPath(BaseBuilder, Append);
		if (!FStringView(BaseBuilder).Equals(ExpectedNewBase, ESearchCase::IgnoreCase))
		{
			FAIL_CHECK(FString::Printf(TEXT("AppendPath(\"%s\", \"%s\") == \"%.*s\", expected \"%s\"."),
				Base, Append, BaseBuilder.Len(), BaseBuilder.GetData(), ExpectedNewBase));
		}
	};

	RunAppendTest(TEXT(""), TEXT("A"), TEXT("A"));
	RunAppendTest(TEXT(""), TEXT("A/B"), TEXT("A/B"));
	RunAppendTest(TEXT(""), TEXT("A\\B"), TEXT("A\\B"));

	RunAppendTest(TEXT("Root"), TEXT("A"), TEXT("Root/A"));
	RunAppendTest(TEXT("Root"), TEXT("A/B"), TEXT("Root/A/B"));
	RunAppendTest(TEXT("Root"), TEXT("A\\B"), TEXT("Root/A\\B"));

	RunAppendTest(TEXT("/"), TEXT("A"), TEXT("/A"));
	RunAppendTest(TEXT("/"), TEXT("A/B"), TEXT("/A/B"));
	RunAppendTest(TEXT("/"), TEXT("A\\B"), TEXT("/A\\B"));
	RunAppendTest(TEXT("\\"), TEXT("A"), TEXT("\\A"));
	RunAppendTest(TEXT("\\"), TEXT("A/B"), TEXT("\\A/B"));
	RunAppendTest(TEXT("\\"), TEXT("A\\B"), TEXT("\\A\\B"));

	RunAppendTest(TEXT("/Root"), TEXT("A"), TEXT("/Root/A"));
	RunAppendTest(TEXT("/Root"), TEXT("A/B"), TEXT("/Root/A/B"));
	RunAppendTest(TEXT("/Root"), TEXT("A\\B"), TEXT("/Root/A\\B"));
	RunAppendTest(TEXT("\\Root"), TEXT("A"), TEXT("\\Root/A"));
	RunAppendTest(TEXT("\\Root"), TEXT("A/B"), TEXT("\\Root/A/B"));
	RunAppendTest(TEXT("\\Root"), TEXT("A\\B"), TEXT("\\Root/A\\B"));
	RunAppendTest(TEXT("/Root/"), TEXT("A"), TEXT("/Root/A"));
	RunAppendTest(TEXT("/Root/"), TEXT("A/B"), TEXT("/Root/A/B"));
	RunAppendTest(TEXT("/Root/"), TEXT("A\\B"), TEXT("/Root/A\\B"));
	RunAppendTest(TEXT("\\Root\\"), TEXT("A"), TEXT("\\Root\\A"));
	RunAppendTest(TEXT("\\Root\\"), TEXT("A/B"), TEXT("\\Root\\A/B"));
	RunAppendTest(TEXT("\\Root\\"), TEXT("A\\B"), TEXT("\\Root\\A\\B"));

	RunAppendTest(TEXT("//"), TEXT("A"), TEXT("//A"));
	RunAppendTest(TEXT("//"), TEXT("A/B"), TEXT("//A/B"));
	RunAppendTest(TEXT("//"), TEXT("A\\B"), TEXT("//A\\B"));
	RunAppendTest(TEXT("\\\\"), TEXT("A"), TEXT("\\\\A"));
	RunAppendTest(TEXT("\\\\"), TEXT("A/B"), TEXT("\\\\A/B"));
	RunAppendTest(TEXT("\\\\"), TEXT("A\\B"), TEXT("\\\\A\\B"));

	RunAppendTest(TEXT("//Root"), TEXT("A"), TEXT("//Root/A"));
	RunAppendTest(TEXT("//Root"), TEXT("A/B"), TEXT("//Root/A/B"));
	RunAppendTest(TEXT("//Root"), TEXT("A\\B"), TEXT("//Root/A\\B"));
	RunAppendTest(TEXT("\\\\Root"), TEXT("A"), TEXT("\\\\Root/A"));
	RunAppendTest(TEXT("\\\\Root"), TEXT("A/B"), TEXT("\\\\Root/A/B"));
	RunAppendTest(TEXT("\\\\Root"), TEXT("A\\B"), TEXT("\\\\Root/A\\B"));
	RunAppendTest(TEXT("//Root/"), TEXT("A"), TEXT("//Root/A"));
	RunAppendTest(TEXT("//Root/"), TEXT("A/B"), TEXT("//Root/A/B"));
	RunAppendTest(TEXT("//Root/"), TEXT("A\\B"), TEXT("//Root/A\\B"));
	RunAppendTest(TEXT("\\\\Root\\"), TEXT("A"), TEXT("\\\\Root\\A"));
	RunAppendTest(TEXT("\\\\Root\\"), TEXT("A/B"), TEXT("\\\\Root\\A/B"));
	RunAppendTest(TEXT("\\\\Root\\"), TEXT("A\\B"), TEXT("\\\\Root\\A\\B"));

	RunAppendTest(TEXT("C:/"), TEXT("A"), TEXT("C:/A"));
	RunAppendTest(TEXT("C:/"), TEXT("A/B"), TEXT("C:/A/B"));
	RunAppendTest(TEXT("C:/"), TEXT("A\\B"), TEXT("C:/A\\B"));
	RunAppendTest(TEXT("C:\\"), TEXT("A"), TEXT("C:\\A"));
	RunAppendTest(TEXT("C:\\"), TEXT("A/B"), TEXT("C:\\A/B"));
	RunAppendTest(TEXT("C:\\"), TEXT("A\\B"), TEXT("C:\\A\\B"));

	RunAppendTest(TEXT("C:/Root"), TEXT("A"), TEXT("C:/Root/A"));
	RunAppendTest(TEXT("C:/Root"), TEXT("A/B"), TEXT("C:/Root/A/B"));
	RunAppendTest(TEXT("C:/Root"), TEXT("A\\B"), TEXT("C:/Root/A\\B"));
	RunAppendTest(TEXT("C:\\Root"), TEXT("A"), TEXT("C:\\Root/A"));
	RunAppendTest(TEXT("C:\\Root"), TEXT("A/B"), TEXT("C:\\Root/A/B"));
	RunAppendTest(TEXT("C:\\Root"), TEXT("A\\B"), TEXT("C:\\Root/A\\B"));
	RunAppendTest(TEXT("C:/Root/"), TEXT("A"), TEXT("C:/Root/A"));
	RunAppendTest(TEXT("C:/Root/"), TEXT("A/B"), TEXT("C:/Root/A/B"));
	RunAppendTest(TEXT("C:/Root/"), TEXT("A\\B"), TEXT("C:/Root/A\\B"));
	RunAppendTest(TEXT("C:\\Root\\"), TEXT("A"), TEXT("C:\\Root\\A"));
	RunAppendTest(TEXT("C:\\Root\\"), TEXT("A/B"), TEXT("C:\\Root\\A/B"));
	RunAppendTest(TEXT("C:\\Root\\"), TEXT("A\\B"), TEXT("C:\\Root\\A\\B"));

	// No matter the prefix, appending a rooted path should result in only the rooted path
	for (const TCHAR* Prefix : { TEXT(""),
		TEXT("/"), TEXT("\\"), TEXT("/Root"), TEXT("\\Root"), TEXT("/Root/"), TEXT("\\Root\\"),
		TEXT("//"), TEXT("\\\\"), TEXT("//Root"), TEXT("\\\\Root"), TEXT("//Root/"), TEXT("\\\\Root\\"),
		TEXT("C:/"), TEXT("C:\\"), TEXT("C:/Root"), TEXT("C:\\Root"), TEXT("C:/Root/"), TEXT("C:\\Root\\")})
	{
		RunAppendTest(Prefix, TEXT("/A"), TEXT("/A"));
		RunAppendTest(Prefix, TEXT("\\A"), TEXT("\\A"));
		RunAppendTest(Prefix, TEXT("/A/B"), TEXT("/A/B"));
		RunAppendTest(Prefix, TEXT("\\A\\B"), TEXT("\\A\\B"));

		RunAppendTest(Prefix, TEXT("//A"), TEXT("//A"));
		RunAppendTest(Prefix, TEXT("\\\\A"), TEXT("\\\\A"));
		RunAppendTest(Prefix, TEXT("//A/B"), TEXT("//A/B"));
		RunAppendTest(Prefix, TEXT("\\\\A\\B"), TEXT("\\\\A\\B"));

		RunAppendTest(Prefix, TEXT("C:/A"), TEXT("C:/A"));
		RunAppendTest(Prefix, TEXT("C:\\A"), TEXT("C:\\A"));
		RunAppendTest(Prefix, TEXT("C:/A/B"), TEXT("C:/A/B"));
		RunAppendTest(Prefix, TEXT("C:\\A\\B"), TEXT("C:\\A\\B"));
	}

}


TEST_CASE_NAMED(FPathViewsGetMountPointNameTest, "System::Core::Misc::PathViews::GetMountPointNameFromPath", "[ApplicationContextMask][SmokeFilter]")
{
	auto RunGetMountPointName = [](const TCHAR* Base, const TCHAR* ExpectedWithoutSlash, const TCHAR* ExpectedWithSlash)
	{
		TStringBuilder<128> BaseBuilder;
		BaseBuilder << Base;
		bool bHadClassPrefix = false;
		FStringView Result = FPathViews::GetMountPointNameFromPath(Base, &bHadClassPrefix, true);
		if (Result != FStringView(ExpectedWithoutSlash))
		{
			FAIL_CHECK(FString::Printf(TEXT("GetMountPointNameFromPath(\"%s\", &bHadClassPrefix, true) == \"%.*s\", expected \"%s\"."),
				Base, Result.Len(), Result.GetData(), ExpectedWithoutSlash));
		}
		Result = FPathViews::GetMountPointNameFromPath(Base, &bHadClassPrefix, false);
		if (Result != FStringView(ExpectedWithSlash))
		{
			FAIL_CHECK(FString::Printf(TEXT("GetMountPointNameFromPath(\"%s\", &bHadClassPrefix, false) == \"%.*s\", expected \"%s\"."),
				Base, Result.Len(), Result.GetData(), ExpectedWithSlash));
		}
	};

	RunGetMountPointName(TEXT(""), TEXT(""), TEXT(""));
	RunGetMountPointName(TEXT("Root"), TEXT(""), TEXT(""));
	RunGetMountPointName(TEXT("/"), TEXT(""), TEXT("/"));
	RunGetMountPointName(TEXT("/Root"), TEXT("Root"), TEXT("/Root"));
	RunGetMountPointName(TEXT("\\Root"), TEXT(""), TEXT(""));
	RunGetMountPointName(TEXT("/Root/"), TEXT("Root"), TEXT("/Root"));
	RunGetMountPointName(TEXT("/Root/A/B/"), TEXT("Root"), TEXT("/Root"));
	RunGetMountPointName(TEXT("/Classes/A/"), TEXT("Classes"), TEXT("/Classes"));
	RunGetMountPointName(TEXT("/Classes_A/B/"), TEXT("A"), TEXT("/Classes_A"));

}


TEST_CASE_NAMED(FPathViewsToAbsoluteTest, "System::Core::Misc::PathViews::ToAbsolute", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace PathTest;

	for (FTestPair Pair : ExpectedRelativeToAbsolutePaths)
	{
		TStringBuilder<64> ActualAppend;
		FPathViews::ToAbsolutePath(BaseDir, Pair.Input, ActualAppend);
		CHECK_EQUALS(TEXT("ToAbsolutePath"), ActualAppend.ToView(), Pair.Expected);

		TStringBuilder<64> ActualInline;
		ActualInline << Pair.Input;
		FPathViews::ToAbsolutePathInline(BaseDir, ActualInline);
		CHECK_EQUALS(TEXT("ToAbsolutePathInline"), ActualInline.ToView(), Pair.Expected);

		const FStringView Original = TEXTVIEW("\\\\la/./.././la////");
		TStringBuilder<64> ActualNondestructive;
		ActualNondestructive << Original;
		FPathViews::ToAbsolutePath(BaseDir, Pair.Input, ActualNondestructive);
		CHECK_EQUALS(TEXT("ToAbsolutePath non-destructive append"), ActualNondestructive.ToView().Left(Original.Len()), Original);
		CHECK_EQUALS(TEXT("ToAbsolutePath non-destructive append"), ActualNondestructive.ToView().RightChop(Original.Len()), Pair.Expected);
	}

}

TEST_CASE_NAMED(FPathViewsVolumeSpecifierTest, "System::Core::Misc::PathViews::VolumeSpecifier", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace PathTest;


	struct FTestCase
	{
		FStringView Input;
		bool bDriveSpecifier;
		FStringView Volume;
		FStringView Remainder;
	};
	FTestCase TestCases[] = {
		{TEXTVIEW(""),                 false, TEXTVIEW(""),           TEXTVIEW("")            },
		{ TEXTVIEW("D:"),              true,  TEXTVIEW("D:"),         TEXTVIEW("")            },
		{ TEXTVIEW("D:/"),             false, TEXTVIEW("D:"),         TEXTVIEW("/")           },
		{ TEXTVIEW("D:\\"),            false, TEXTVIEW("D:"),         TEXTVIEW("\\")          },
		{ TEXTVIEW("D:root/path"),     true,  TEXTVIEW("D:"),         TEXTVIEW("root/path")   },
		{ TEXTVIEW("D:/root/path"),    false, TEXTVIEW("D:"),         TEXTVIEW("/root/path")  },
		{ TEXTVIEW("D:\\root\\path"),  false, TEXTVIEW("D:"),         TEXTVIEW("\\root\\path")},
		{ TEXTVIEW("//volume"),        false, TEXTVIEW("//volume"),   TEXTVIEW("")            },
		{ TEXTVIEW("\\\\volume"),      false, TEXTVIEW("\\\\volume"), TEXTVIEW("")            },
		{ TEXTVIEW("/\\volume"),       false, TEXTVIEW("/\\volume"),  TEXTVIEW("")            }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW("\\/volume"),       false, TEXTVIEW("\\/volume"),  TEXTVIEW("")            }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW("//\\volume"),      false, TEXTVIEW("//\\volume"), TEXTVIEW("")            }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW("//volume/"),       false, TEXTVIEW("//volume"),   TEXTVIEW("/")           },
		{ TEXTVIEW("//volume/root"),   false, TEXTVIEW("//volume"),   TEXTVIEW("/root")       },
		{ TEXTVIEW("/root/path"),      false, TEXTVIEW(""),           TEXTVIEW("/root/path")  },
		{ TEXTVIEW("\\root\\path"),    false, TEXTVIEW(""),           TEXTVIEW("\\root\\path")},
		{ TEXTVIEW("root/path"),       false, TEXTVIEW(""),           TEXTVIEW("root/path")   },
		{ TEXTVIEW("/"),               false, TEXTVIEW(""),           TEXTVIEW("/")           },
		{ TEXTVIEW("\\"),              false, TEXTVIEW(""),           TEXTVIEW("\\")          },
		{ TEXTVIEW("//"),              false, TEXTVIEW("//"),         TEXTVIEW("")            }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW("\\\\"),            false, TEXTVIEW("\\\\"),       TEXTVIEW("")            }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW("/\\"),             false, TEXTVIEW("/\\"),        TEXTVIEW("")            }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW("\\/"),             false, TEXTVIEW("\\/"),        TEXTVIEW("")            }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW("/:"),              false, TEXTVIEW(""),           TEXTVIEW("/:")          }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW(":"),               true,  TEXTVIEW(":"),          TEXTVIEW("")            }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW(":/"),              false, TEXTVIEW(":"),          TEXTVIEW("/")           }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW(":root"),           true,  TEXTVIEW(":"),          TEXTVIEW("root")        }, // Poorly defined case, somewhat arbitrary
		{ TEXTVIEW("////volume/path"), false, TEXTVIEW("////volume"), TEXTVIEW("/path")       }, // Poorly defined case, somewhat arbitrary, @see RemoveDuplicateSlashes
	};

	for (const FTestCase& TestCase : TestCases)
	{
		bool bDriveSpecifier;
		FStringView Volume;
		FStringView Remainder;
		bDriveSpecifier = FPathViews::IsDriveSpecifierWithoutRoot(TestCase.Input);
		FPathViews::SplitVolumeSpecifier(TestCase.Input, Volume, Remainder);
		CHECK_EQUALS(FString(TestCase.Input), bDriveSpecifier, TestCase.bDriveSpecifier);
		CHECK_EQUALS(FString(TestCase.Input), Volume, TestCase.Volume);
		CHECK_EQUALS(FString(TestCase.Input), Remainder, TestCase.Remainder);
	};
}


#endif //WITH_TESTS
