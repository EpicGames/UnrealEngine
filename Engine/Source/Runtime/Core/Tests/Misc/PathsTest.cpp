// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "PathTests.h"

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

void TestCreateStandardFilename();

const FStringView PathTest::BaseDir = TEXTVIEW("/root");

static PathTest::FTestPair StaticExpectedRelativeToAbsolutePaths[] =
{
	{ TEXTVIEW(""),					TEXTVIEW("/root/") },
	{ TEXTVIEW("dir"),				TEXTVIEW("/root/dir") },
	{ TEXTVIEW("/groot"),			TEXTVIEW("/groot") },
	{ TEXTVIEW("/groot/"),			TEXTVIEW("/groot/") },
	{ TEXTVIEW("/r/dir"),			TEXTVIEW("/r/dir") },
	{ TEXTVIEW("/r/dir"),			TEXTVIEW("/r/dir") },
	{ TEXTVIEW("C:\\"),				TEXTVIEW("C:/") },
	{ TEXTVIEW("C:\\A\\B"),			TEXTVIEW("C:/A/B") },
	{ TEXTVIEW("a/b/../c"),			TEXTVIEW("/root/a/c") },
	{ TEXTVIEW("/a/b/../c"),		TEXTVIEW("/a/c") },
	{ TEXTVIEW("D:/Root/Engine//../.."), TEXTVIEW("D:") }, // removing the leading slash is incorrect but legacy behavior, we should fix eventually
	{ TEXTVIEW("D:/Root/Engine//../../"), TEXTVIEW("D:/") },
};
TConstArrayView<PathTest::FTestPair> PathTest::ExpectedRelativeToAbsolutePaths = StaticExpectedRelativeToAbsolutePaths;

TEST_CASE_NAMED(FPathTests, "System::Core::Misc::Paths", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::Paths;

	TestCollapseRelativeDirectories<FPaths, FString>();
	TestCreateStandardFilename();

	// Extension texts
	{
		auto RunGetTest = [](const TCHAR* InPath, const TCHAR* InExpectedExt, const TCHAR* InExpectedExtDot)
			{
				FString Ext = FPaths::GetExtension(FString(InPath));
				if (Ext != InExpectedExt)
				{
					FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to get the extension (got '%s', expected '%s')."),
						InPath, *Ext, InExpectedExt));
				}
				Ext = FPaths::GetExtension(FString(InPath), false /* bIncludeDot */);
				if (Ext != InExpectedExt)
				{
					FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to get the extension (got '%s', expected '%s')."),
						InPath, *Ext, InExpectedExt));
				}
				Ext = FPaths::GetExtension(FString(InPath), true/* bIncludeDot */);
				if (Ext != InExpectedExtDot)
				{
					FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to get the extension (got '%s', expected '%s')."),
						InPath, *Ext, InExpectedExtDot));
				}
				Ext = FPaths::GetExtension(FString(InPath), EFlags::IncludeDot);
				if (Ext != InExpectedExtDot)
				{
					FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to get the extension (got '%s', expected '%s')."),
						InPath, *Ext, InExpectedExtDot));
				}
			};

		auto RunGetTestFull = [](const TCHAR* InPath, const TCHAR* InExpectedExt, const TCHAR* InExpectedExtDot,
			const TCHAR* InExpectedExtCompound, const TCHAR* InExpectedExtDotCompound)
			{
				FString Ext = FPaths::GetExtension(FString(InPath));
				if (Ext != InExpectedExt)
				{
					FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to get the extension (got '%s', expected '%s')."),
						InPath, *Ext, InExpectedExt));
				}
				Ext = FPaths::GetExtension(FString(InPath), EFlags::None);
				if (Ext != InExpectedExt)
				{
					FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to get the extension (got '%s', expected '%s')."),
						InPath, *Ext, InExpectedExt));
				}
				Ext = FPaths::GetExtension(FString(InPath), EFlags::IncludeDot);
				if (Ext != InExpectedExtDot)
				{
					FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to get the extension (got '%s', expected '%s')."),
						InPath, *Ext, InExpectedExtDot));
				}
				Ext = FPaths::GetExtension(FString(InPath), EFlags::AllowCompoundExtension);
				if (Ext != InExpectedExtCompound)
				{
					FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to get the extension (got '%s', expected '%s')."),
						InPath, *Ext, InExpectedExtCompound));
				}
				Ext = FPaths::GetExtension(FString(InPath), EFlags::IncludeDot | EFlags::AllowCompoundExtension);
				if (Ext != InExpectedExtDotCompound)
				{
					FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to get the extension (got '%s', expected '%s')."),
						InPath, *Ext, InExpectedExtDotCompound));
				}
			};



		RunGetTest(TEXT("file"),								TEXT(""),	TEXT(""));
		RunGetTest(TEXT("file.txt"),							TEXT("txt"),TEXT(".txt"));
		RunGetTest(TEXT("file.tar.gz"),							TEXT("gz"),	TEXT(".gz"));
		RunGetTest(TEXT("C:/Folder/file"),						TEXT(""),	TEXT(""));
		RunGetTest(TEXT("C:/Folder/file.txt"),					TEXT("txt"),TEXT(".txt"));
		RunGetTestFull(TEXT("C:/Folder/file.tar.gz"),			TEXT("gz"),	TEXT(".gz"),	TEXT("tar.gz"), TEXT(".tar.gz"));
		RunGetTestFull(TEXT("C:/Folder/First.Last/file"),		TEXT(""),	TEXT(""),		TEXT(""),		TEXT(""));
		RunGetTestFull(TEXT("C:/Folder/First.Last/file.txt"),	TEXT("txt"),TEXT(".txt"),	TEXT("txt"),	TEXT(".txt"));
		RunGetTestFull(TEXT("C:/Folder/First.Last/file.tar.gz"),TEXT("gz"),	TEXT(".gz"),	TEXT("tar.gz"), TEXT(".tar.gz"));

		auto RunSetExtensionTest = [](const TCHAR* InPath, const TCHAR* InNewExt, const FString& InExpectedPath)
		{
			// Run test
			const FString NewPath = FPaths::SetExtension(FString(InPath), FString(InNewExt));
			if (NewPath != InExpectedPath)
			{
				FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to set the extension (got '%s', expected '%s')."), InPath, *NewPath, *InExpectedPath));
			}
		};

		RunSetExtensionTest(TEXT("file"),									TEXT("log"),	TEXT("file.log"));
		RunSetExtensionTest(TEXT("file.txt"),								TEXT("log"),	TEXT("file.log"));
		RunSetExtensionTest(TEXT("file.tar.gz"),							TEXT("gz2"),	TEXT("file.tar.gz2"));
		RunSetExtensionTest(TEXT("C:/Folder/file"),							TEXT("log"),	TEXT("C:/Folder/file.log"));
		RunSetExtensionTest(TEXT("C:/Folder/file.txt"),						TEXT("log"),	TEXT("C:/Folder/file.log"));
		RunSetExtensionTest(TEXT("C:/Folder/file.tar.gz"),					TEXT("gz2"),	TEXT("C:/Folder/file.tar.gz2"));
		RunSetExtensionTest(TEXT("C:/Folder/First.Last/file"),				TEXT("log"),	TEXT("C:/Folder/First.Last/file.log"));
		RunSetExtensionTest(TEXT("C:/Folder/First.Last/file.txt"),			TEXT("log"),	TEXT("C:/Folder/First.Last/file.log"));
		RunSetExtensionTest(TEXT("C:/Folder/First.Last/file.tar.gz"),		TEXT("gz2"),	TEXT("C:/Folder/First.Last/file.tar.gz2"));

		auto RunChangeExtensionTest = [](const TCHAR* InPath, const TCHAR* InNewExt, const FString& InExpectedPath)
		{
			// Run test
			const FString NewPath = FPaths::ChangeExtension(FString(InPath), FString(InNewExt));
			if (NewPath != InExpectedPath)
			{
				FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed to change the extension (got '%s', expected '%s')."), InPath, *NewPath, *InExpectedPath));
			}
		};

		RunChangeExtensionTest(TEXT("file"),								TEXT("log"),	TEXT("file"));
		RunChangeExtensionTest(TEXT("file.txt"),							TEXT("log"),	TEXT("file.log"));
		RunChangeExtensionTest(TEXT("file.tar.gz"),							TEXT("gz2"),	TEXT("file.tar.gz2"));
		RunChangeExtensionTest(TEXT("C:/Folder/file"),						TEXT("log"),	TEXT("C:/Folder/file"));
		RunChangeExtensionTest(TEXT("C:/Folder/file.txt"),					TEXT("log"),	TEXT("C:/Folder/file.log"));
		RunChangeExtensionTest(TEXT("C:/Folder/file.tar.gz"),				TEXT("gz2"),	TEXT("C:/Folder/file.tar.gz2"));
		RunChangeExtensionTest(TEXT("C:/Folder/First.Last/file"),			TEXT("log"),	TEXT("C:/Folder/First.Last/file"));
		RunChangeExtensionTest(TEXT("C:/Folder/First.Last/file.txt"),		TEXT("log"),	TEXT("C:/Folder/First.Last/file.log"));
		RunChangeExtensionTest(TEXT("C:/Folder/First.Last/file.tar.gz"),	TEXT("gz2"),	TEXT("C:/Folder/First.Last/file.tar.gz2"));
	}

	// IsUnderDirectory
	{
		auto RunIsUnderDirectoryTest = [](const TCHAR* InPath1, const TCHAR* InPath2, bool ExpectedResult)
		{
			// Run test
			bool Result = FPaths::IsUnderDirectory(FString(InPath1), FString(InPath2));
			if (Result != ExpectedResult)
			{
				FAIL_CHECK(FString::Printf(TEXT("FPaths::IsUnderDirectory('%s', '%s') != %s."), InPath1, InPath2, ExpectedResult ? TEXT("true") : TEXT("false")));
			}
		};

		RunIsUnderDirectoryTest(TEXT("C:/Folder"),			TEXT("C:/FolderN"), false);
		RunIsUnderDirectoryTest(TEXT("C:/Folder1"),			TEXT("C:/Folder2"), false);
		RunIsUnderDirectoryTest(TEXT("C:/Folder"),			TEXT("C:/Folder/SubDir"), false);

		RunIsUnderDirectoryTest(TEXT("C:/Folder"),			TEXT("C:/Folder"), true);
		RunIsUnderDirectoryTest(TEXT("C:/Folder/File"),		TEXT("C:/Folder"), true);
		RunIsUnderDirectoryTest(TEXT("C:/Folder/File"),		TEXT("C:/Folder/"), true);
		RunIsUnderDirectoryTest(TEXT("C:/Folder/"),			TEXT("C:/Folder"), true);
		RunIsUnderDirectoryTest(TEXT("C:/Folder/"),			TEXT("C:/Folder/"), true);
		RunIsUnderDirectoryTest(TEXT("C:/Folder/Subdir/"),	TEXT("C:/Folder"), true);
		RunIsUnderDirectoryTest(TEXT("C:/Folder/Subdir/"),	TEXT("C:/Folder/"), true);
	}

	TestRemoveDuplicateSlashes<FPaths, FString>();

	// ConvertRelativePathToFull
	{
		using namespace PathTest;

		for (FTestPair Pair : ExpectedRelativeToAbsolutePaths)
		{
			FString Actual = FPaths::ConvertRelativePathToFull(FString(BaseDir), FString(Pair.Input));
			CHECK_EQUALS(TEXT("ConvertRelativePathToFull"), FStringView(Actual), Pair.Expected);
		}
	}

	// Split
	auto RunSplitTest = [](const TCHAR* InPath, const TCHAR* InExpectedPath, const TCHAR* InExpectedName, const TCHAR* InExpectedExt)
		{
			FString SplitPath, SplitName, SplitExt;
			FPaths::Split(InPath, SplitPath, SplitName, SplitExt);
			if (SplitPath != InExpectedPath || SplitName != InExpectedName || SplitExt != InExpectedExt)
			{
				FAIL_CHECK(FString::Printf(TEXT("Failed to split path '%s' (got ('%s', '%s', '%s'), expected ('%s', '%s', '%s'))."), InPath,
					*SplitPath, *SplitName, *SplitExt, InExpectedPath, InExpectedName, InExpectedExt));
			}
		};

	RunSplitTest(TEXT(""), TEXT(""), TEXT(""), TEXT(""));
	RunSplitTest(TEXT(".txt"), TEXT(""), TEXT(""), TEXT("txt"));
	RunSplitTest(TEXT(".tar.gz"), TEXT(""), TEXT(".tar"), TEXT("gz"));
	RunSplitTest(TEXT(".tar.gz/"), TEXT(".tar.gz"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT(".tar.gz\\"), TEXT(".tar.gz"), TEXT(""), TEXT(""));
	// TEXT(".") is ambiguous; we currently treat it as an extension separator but we don't guarantee that in our contract
	//RunSplitTest(TEXT("."), TEXT(""), TEXT(""), TEXT(""));
	RunSplitTest(TEXT(".."), TEXT(""), TEXT(".."), TEXT(""));
	RunSplitTest(TEXT("File"), TEXT(""), TEXT("File"), TEXT(""));
	RunSplitTest(TEXT("File.txt"), TEXT(""), TEXT("File"), TEXT("txt"));
	RunSplitTest(TEXT("File.tar.gz"), TEXT(""), TEXT("File.tar"), TEXT("gz"));
	RunSplitTest(TEXT("File.tar.gz/"), TEXT("File.tar.gz"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT("File.tar.gz\\"), TEXT("File.tar.gz"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT("C:/Folder/"), TEXT("C:/Folder"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT("C:/Folder/File"), TEXT("C:/Folder"), TEXT("File"), TEXT(""));
	RunSplitTest(TEXT("C:/Folder/File.txt"), TEXT("C:/Folder"), TEXT("File"), TEXT("txt"));
	RunSplitTest(TEXT("C:/Folder/File.tar.gz"), TEXT("C:/Folder"), TEXT("File.tar"), TEXT("gz"));
	RunSplitTest(TEXT("C:/Folder/First.Last/File"), TEXT("C:/Folder/First.Last"), TEXT("File"), TEXT(""));
	RunSplitTest(TEXT("C:/Folder/First.Last/File.txt"), TEXT("C:/Folder/First.Last"), TEXT("File"), TEXT("txt"));
	RunSplitTest(TEXT("C:/Folder/First.Last/File.tar.gz"), TEXT("C:/Folder/First.Last"), TEXT("File.tar"), TEXT("gz"));
	RunSplitTest(TEXT("C:\\Folder\\"), TEXT("C:\\Folder"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT("C:\\Folder\\File"), TEXT("C:\\Folder"), TEXT("File"), TEXT(""));
	RunSplitTest(TEXT("C:\\Folder\\First.Last\\"), TEXT("C:\\Folder\\First.Last"), TEXT(""), TEXT(""));
	RunSplitTest(TEXT("C:\\Folder\\First.Last\\File"), TEXT("C:\\Folder\\First.Last"), TEXT("File"), TEXT(""));
	RunSplitTest(TEXT("C:\\Folder\\First.Last\\File.txt"), TEXT("C:\\Folder\\First.Last"), TEXT("File"), TEXT("txt"));
	RunSplitTest(TEXT("C:\\Folder\\First.Last\\File.tar.gz"), TEXT("C:\\Folder\\First.Last"), TEXT("File.tar"), TEXT("gz"));

	// ValidatePath
	auto RunValidatePathTest = [](const TCHAR* Path, bool bExpectedValue)
		{
			bool bValue = FPaths::ValidatePath(FString(Path));
			if (bValue != bExpectedValue)
			{
				FAIL_CHECK(FString::Printf(TEXT("ValidatePath(\"%s\") == %s, expected %s."),
					Path, *LexToString(bValue), *LexToString(bExpectedValue)));
			}
		};
	RunValidatePathTest(TEXT("C:/Path/Dir"), true);
	RunValidatePathTest(TEXT("C:\\Path\\Dir"), true);
	RunValidatePathTest(TEXT("//Path/Dir"), true);
	RunValidatePathTest(TEXT("\\\\Path\\Dir"), true);
	RunValidatePathTest(TEXT("/Path/Dir"), true);
	RunValidatePathTest(TEXT("\\Path\\Dir"), true);
	RunValidatePathTest(TEXT("Path/Dir"), true);
	RunValidatePathTest(TEXT("Path\\Dir"), true);
	RunValidatePathTest(TEXT("Path"), true);
	RunValidatePathTest(TEXT("/Path/"), true);
	RunValidatePathTest(TEXT("\\Path\\"), true);
	// LongPaths on windows allow the otherwise illegal character ? at the beginning, in "\\?\"
	RunValidatePathTest(TEXT("\\\\?\\K:\\myfolder\\myfile.fbx"), true);
	// Otherwise ? is not allowed
	RunValidatePathTest(TEXT("C:\\Path\\Dir?"), false);
	// : is not allowed outside of a drive specifier at beginning
	RunValidatePathTest(TEXT("C:\\Path\\Dir:"), false);
	// A few other banned characters from FPaths::GetInvalidFileSystemChars
	RunValidatePathTest(TEXT("C:\\Path\\Dir&"), false);
	RunValidatePathTest(TEXT("C:\\Path\\Dir^"), false);
}

void TestCreateStandardFilename()
{
	auto NormalizeDirForTest = [](FString& Dir)
		{
			while (!Dir.IsEmpty() && FPathViews::IsSeparator(Dir[Dir.Len() - 1]))
			{
				Dir.LeftChopInline(1);
			}
		};
	FString RootDirectory = FPaths::ConvertRelativePathToFull(FPaths::RootDir());
	FString BaseDirectory = FPaths::ConvertRelativePathToFull(FPlatformProcess::BaseDir());
	FString RootRelative = FPaths::GetRelativePathToRoot();
	NormalizeDirForTest(RootDirectory);
	NormalizeDirForTest(BaseDirectory);
	NormalizeDirForTest(RootRelative);

	FStringView RelPathToBase;
	if (RootDirectory.IsEmpty() || !FPathViews::TryMakeChildPathRelativeTo(BaseDirectory, RootDirectory, RelPathToBase))
	{
		// In the case where the base path (expected <root>/Engine/Binaries/Win64) is not a subdirectory of <root>,
		// we don't know how to verify the results.
		// Skip the test.
		return;
	}
	FString RootDirectoryWithSlash = RootDirectory + TEXT("/");
	FString StandardPathOfBase = FPaths::Combine(RootRelative, RelPathToBase);
	FStringView StandardPathOfBaseParent = FPathViews::GetPath(StandardPathOfBase);
	FStringView StandardPathOfBaseParent2 = FPathViews::GetPath(StandardPathOfBaseParent);
	FStringView StandardPathOfBaseParent3 = FPathViews::GetPath(StandardPathOfBaseParent2);
	if (StandardPathOfBaseParent.EndsWith(TEXT("..")) || StandardPathOfBaseParent.IsEmpty()
		|| StandardPathOfBaseParent2.EndsWith(TEXT("..")) || StandardPathOfBaseParent2.IsEmpty()
		|| StandardPathOfBaseParent3.EndsWith(TEXT("..")) || StandardPathOfBaseParent3.IsEmpty())
	{
		// We expect StandardPathOfBase == ../../../Engine/Binaries/Win64
		// We want to be able to use this to know that ".." -> "../../../Engine/Binaries" and
		// similar for ../.. and ../../..
		// If we don't have a parent directory, or the "parent directory" is a ".." directory,
		// then we don't know what the value for these directories in standard form should be.
		// Skip the test.
		return;
	}
	if (RootDirectory.StartsWith(TEXT("X:")))
	{
		// We use X: as the root of absolute paths that aren't under root and so standard path will be their
		// absolute path. If we're running on a workspace that is under the X: drive, skip the test since
		// it will create different answers for those paths.
		return;
	}
	FString RootDirectoryParent = FPaths::GetPath(RootDirectory);
	bool bAllowEscapeRootDirectoryTests = !RootDirectoryParent.IsEmpty()
		&& RootDirectory == FPaths::Combine(RootDirectoryParent, FPaths::GetBaseFilename(RootDirectory));
	FString RelativePathToRootDirectoryParent;
	if (bAllowEscapeRootDirectoryTests)
	{
		RelativePathToRootDirectoryParent = FPaths::Combine(RootRelative, TEXT(".."));
	}
	
	auto Run = [&RootRelative](const TCHAR* Path, FStringView Expected)
		{
			// Run test
			FString Actual = FPaths::CreateStandardFilename(Path);
			if (!Expected.Equals(Actual, ESearchCase::CaseSensitive))
			{
				FAIL_CHECK(FString::Printf(TEXT("Path '%s' failed CreateStandardFilename (got '%s', expected '%s')."),
					Path, *Actual, *FString(Expected)));
			}
		};

	Run(TEXT(".."), StandardPathOfBaseParent);
	Run(TEXT("/.."), TEXT("/.."));
	Run(TEXT("./"), StandardPathOfBase + TEXT("/")); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories
	Run(TEXT("./file.txt"), FPaths::Combine(StandardPathOfBase, TEXT("file.txt")));
	Run(TEXT("/."), TEXT("/."));
	Run(TEXT("Folder"), FPaths::Combine(StandardPathOfBase, TEXT("Folder")));
	Run(TEXT("/Folder"), TEXT("/Folder"));
	Run(TEXT("X:/Folder"), TEXT("X:/Folder"));
	// Current contract is that paths that cannot be standardized do not CollapseRelativeDirectories; they remain unchanged
	Run(TEXT("X:/Folder/.."), TEXT("X:"));
	Run(TEXT("X:/Folder/../"), TEXT("X:/"));
	Run(TEXT("X:/Folder/../file.txt"), TEXT("X:/file.txt")); // cannot be standardized
	Run(TEXT("Folder/.."), StandardPathOfBase);
	Run(TEXT("Folder/../"), StandardPathOfBase + TEXT("/")); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories
	Run(TEXT("Folder/../file.txt"), FPaths::Combine(StandardPathOfBase, TEXT("file.txt")));
	Run(TEXT("/Folder/.."), TEXT(""));
	Run(TEXT("/Folder/../"), TEXT("/"));
	Run(TEXT("/Folder/../file.txt"), TEXT("/file.txt")); // cannot be standardized
	Run(TEXT("Folder/../.."), StandardPathOfBaseParent);
	Run(TEXT("Folder/../../"), FString(StandardPathOfBaseParent) + TEXT("/")); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories
	Run(TEXT("Folder/../../file.txt"), FPaths::Combine(StandardPathOfBaseParent, TEXT("file.txt")));
	Run(TEXT("X:/.."), TEXT("X:/.."));
	Run(TEXT("X:/."), TEXT("X:/."));
	Run(TEXT("X:/./"), TEXT("X:/"));
	Run(TEXT("X:/./file.txt"), TEXT("X:/file.txt"));
	Run(TEXT("X:/Folder1/../Folder2"), TEXT("X:/Folder2"));
	Run(TEXT("Folder1/../Folder2"), FPaths::Combine(StandardPathOfBase, TEXT("Folder2")));
	Run(TEXT("Folder1/../Folder2/"), FPaths::Combine(StandardPathOfBase, TEXT("Folder2/"))); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories
	Run(TEXT("Folder1/../Folder2/file.txt"), FPaths::Combine(StandardPathOfBase, TEXT("Folder2/file.txt")));
	Run(TEXT("Folder1/../Folder2/../.."), StandardPathOfBaseParent);
	Run(TEXT("Folder1/../Folder2/../Folder3"), FPaths::Combine(StandardPathOfBase, TEXT("Folder3")));
	Run(TEXT("Folder1/../Folder2/../Folder3/"), FPaths::Combine(StandardPathOfBase, TEXT("Folder3/")));
	Run(TEXT("Folder1/../Folder2/../Folder3/file.txt"), FPaths::Combine(StandardPathOfBase, TEXT("Folder3/file.txt")));
	Run(TEXT("Folder1/Folder2/../../Folder3"), FPaths::Combine(StandardPathOfBase, TEXT("Folder3")));
	Run(TEXT("Folder1/Folder2/../../Folder3/"), FPaths::Combine(StandardPathOfBase, TEXT("Folder3/")));
	Run(TEXT("Folder1/Folder2/../../Folder3/file.txt"), FPaths::Combine(StandardPathOfBase, TEXT("Folder3/file.txt")));
	Run(TEXT("Folder1/Folder2/../../Folder3/../Folder4"), FPaths::Combine(StandardPathOfBase, TEXT("Folder4")));
	Run(TEXT("Folder1/Folder2/../../Folder3/../Folder4/"), FPaths::Combine(StandardPathOfBase, TEXT("Folder4/"))); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories
	Run(TEXT("Folder1/Folder2/../../Folder3/../Folder4/file.txt"), FPaths::Combine(StandardPathOfBase, TEXT("Folder4/file.txt")));
	Run(TEXT("Folder1/Folder2/../Folder3/../../Folder4"), FPaths::Combine(StandardPathOfBase, TEXT("Folder4")));
	Run(TEXT("Folder1/Folder2/../Folder3/../../Folder4/"), FPaths::Combine(StandardPathOfBase, TEXT("Folder4/"))); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories
	Run(TEXT("Folder1/Folder2/../Folder3/../../Folder4/file.txt"), FPaths::Combine(StandardPathOfBase, TEXT("Folder4/file.txt")));
	Run(TEXT("Folder1/Folder2/.././../Folder4"), FPaths::Combine(StandardPathOfBase, TEXT("Folder4")));
	Run(TEXT("Folder1/Folder2/.././../Folder4/"), FPaths::Combine(StandardPathOfBase, TEXT("Folder4/"))); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories
	Run(TEXT("Folder1/Folder2/.././../Folder4/file.txt"), FPaths::Combine(StandardPathOfBase, TEXT("Folder4/file.txt")));
	Run(TEXT("A/B/.././../C"), FPaths::Combine(StandardPathOfBase, TEXT("C")));
	Run(TEXT("A/B/.././../C/"), FPaths::Combine(StandardPathOfBase, TEXT("C/"))); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories
	Run(TEXT("A/B/.././../C/file.txt"), FPaths::Combine(StandardPathOfBase, TEXT("C/file.txt")));
	Run(TEXT(".svn"), FPaths::Combine(StandardPathOfBase, TEXT(".svn")));
	Run(TEXT("/.svn"), TEXT("/.svn"));
	Run(TEXT("./Folder/.svn"), FPaths::Combine(StandardPathOfBase, TEXT("Folder/.svn")));
	Run(TEXT("./.svn/../.svn"), FPaths::Combine(StandardPathOfBase, TEXT(".svn")));
	Run(TEXT(".svn/./.svn/.././../.svn"), FPaths::Combine(StandardPathOfBase, TEXT(".svn")));
	Run(TEXT("Folder1/./Folder2/..Folder3"), FPaths::Combine(StandardPathOfBase, TEXT("Folder1/Folder2/..Folder3")));
	Run(TEXT("Folder1/./Folder2/..Folder3/Folder4"), FPaths::Combine(StandardPathOfBase, TEXT("Folder1/Folder2/..Folder3/Folder4")));
	Run(TEXT("Folder1/./Folder2/..Folder3/..Folder4"), FPaths::Combine(StandardPathOfBase, TEXT("Folder1/Folder2/..Folder3/..Folder4")));
	Run(TEXT("Folder1/./Folder2/..Folder3/Folder4/../Folder5"), FPaths::Combine(StandardPathOfBase, TEXT("Folder1/Folder2/..Folder3/Folder5")));
	Run(TEXT("Folder1/..Folder2/Folder3/..Folder4/../Folder5"), FPaths::Combine(StandardPathOfBase, TEXT("Folder1/..Folder2/Folder3/Folder5")));
	Run(TEXT("../../../Dir/Dir"), FPaths::Combine(StandardPathOfBaseParent3, TEXT("Dir/Dir")));
	Run(TEXT("../../../Dir/Dir/"), FPaths::Combine(StandardPathOfBaseParent3, TEXT("Dir/Dir/"))); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories
	Run(TEXT("../../../Dir/Dir/../../Dir/Dir/../../Dir/Dir/File"), FPaths::Combine(StandardPathOfBaseParent3, TEXT("Dir/Dir/File")));
	Run(TEXT("../../../Dir/Dir/File.ext"), FPaths::Combine(StandardPathOfBaseParent3, TEXT("Dir/Dir/File.ext")));
	Run(TEXT("../../../Dir/Dir/File"), FPaths::Combine(StandardPathOfBaseParent3, TEXT("Dir/Dir/File")));
	Run(TEXT("../../../Dir/"), FPaths::Combine(StandardPathOfBaseParent3, TEXT("Dir/"))); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories
	Run(TEXT("../../../Dir/Binaries/Win64/File.Ext"), FPaths::Combine(StandardPathOfBaseParent3, TEXT("Dir/Binaries/Win64/File.Ext")));
	Run(TEXT("../../../Dir/Binaries/Win64/../../File.Ext"), FPaths::Combine(StandardPathOfBaseParent3, TEXT("Dir/File.Ext")));
	Run(TEXT("../../Dir/Dir/File"), FPaths::Combine(StandardPathOfBaseParent2, TEXT("Dir/Dir/File")));
	// Run(*RootDirectory, RootRelative); // Not currently handled without the slash
	Run(*RootDirectoryWithSlash, RootRelative + TEXT("/")); // Keeping the trailing slash is legacy behavior in CollapseSubdirectories 
	Run(*FPaths::Combine(RootDirectory, TEXT("Dir/Dir/File.ext")), FPaths::Combine(RootRelative, TEXT("Dir/Dir/File.ext")));
	Run(*FPaths::Combine(RootDirectory, TEXT("Engine/Dir/File.ext")), FPaths::Combine(RootRelative, TEXT("Engine/Dir/File.ext")));
	Run(*(RootDirectory + TEXT("\\Dir\\File")), FPaths::Combine(RootRelative, TEXT("/Dir/File")));
	if (bAllowEscapeRootDirectoryTests)
	{
		// Relative paths that escape the root directory (e.g. specify the root directory's parent or a child other than the root directory
		// of the root directory's parent should be written as absolute paths
		// With Base D:\Root\Engine\Binaries\Win64
		// ../../../.. -> D:\ rather than ../../../..
		// ../../../../Peer -> D:\Peer rather than ../../../../Peer
		Run(*RelativePathToRootDirectoryParent, RootDirectoryParent);
		Run(*FPaths::Combine(RelativePathToRootDirectoryParent, TEXT("Peer")), FPaths::Combine(RootDirectoryParent, TEXT("Peer")));
	}
}

#endif //WITH_TESTS
