// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "HAL/PlatformString.h"
#include "Templates/UniquePtr.h"
#include "Tests/TestHarnessAdapter.h"
#include "Containers/UnrealString.h"
#include <type_traits>

#if WITH_TESTS

TEST_CASE_NAMED(FStringConvertTest, "System::Core::Misc::StringConvertTests", "[ApplicationContextMask][SmokeFilter]")
{
	auto CheckNullTerminatedStringConversion = [](const TCHAR* TestName, const auto* From, const auto* To)
	{
		using ToType = std::decay_t<decltype(*To)>;

		// Basic version of Strcmp for arbitrary types
		auto Strcmp = [](const auto* Lhs, const auto* Rhs)
		{
			for (;;)
			{
				if (!*Lhs)
				{
					return *Rhs ? -1 : 0;
				}

				if (!*Rhs)
				{
					return 1;
				}

				if (*Lhs != *Rhs)
				{
					return *Lhs < *Rhs ? -1 : 1;
				}

				++Lhs;
				++Rhs;
			}
		};

		int32 ConvertedLength = FPlatformString::ConvertedLength<ToType>(From);

		// Allocate a buffer to hold the converted string and set it with 0xCD
		TUniquePtr<ToType[]> UniquePtr = MakeUniqueForOverwrite<ToType[]>(ConvertedLength + 8);
		ToType* Ptr = UniquePtr.Get();
		FMemory::Memset(Ptr, 0xCD, (ConvertedLength + 8) * sizeof(ToType));

		ToType* End = FPlatformString::Convert(Ptr + 4, ConvertedLength, From);

		CHECK_MESSAGE(FString::Printf(TEXT("%s converted length matches"), TestName), End == Ptr + 4 + ConvertedLength);

		// != changed to < because of false positive in MSVC's static analyzer: warning C6295: Ill-defined for-loop.  Loop executes infinitely.
		for (int i = 0; i < 4 * sizeof(ToType); ++i)
		{
			CHECK_MESSAGE(FString::Printf(TEXT("%s guard block is preserved"), TestName), ((unsigned char*)Ptr)[i] == 0xCD && ((unsigned char*)End)[i] == 0xCD);
		}

		CHECK_MESSAGE(FString::Printf(TEXT("%s converted string matches"), TestName), Strcmp(To, Ptr + 4) == 0);
	};

	// We need to do UTF-8 literals like below because MSVC doesn't encode numeric escape sequences properly in u8"" literals:
	// https://developercommunity.visualstudio.com/t/hex-escape-codes-in-a-utf8-literal-are-t/225847
	//
	// We cast to UTF16CHAR* because UTF16TEXT doesn't currently result in an array of UTF16CHAR.

	// Freely interconvertible
	{
		const UTF16CHAR* UnrealEngineIsGreat16 = (const UTF16CHAR*)UTF16TEXT("\xD835\xDE50\x043F\xD835\xDDCB\xD835\xDE26\xD835\xDD52\x006C\x0020\xD835\xDC6C\xD835\xDD2B\xFF47\xD835\xDCBE\xD835\xDD93\xD835\xDD56\x0020\xD835\xDE92\xFF53\x0020\xD835\xDC6E\xD835\xDE9B\x212E\xD835\xDF36\xFF54\x0021");
		const UTF8CHAR*  UnrealEngineIsGreat8  = (const UTF8CHAR*)"\xF0\x9D\x99\x90\xD0\xBF\xF0\x9D\x97\x8B\xF0\x9D\x98\xA6\xF0\x9D\x95\x92\x6C\x20\xF0\x9D\x91\xAC\xF0\x9D\x94\xAB\xEF\xBD\x87\xF0\x9D\x92\xBE\xF0\x9D\x96\x93\xF0\x9D\x95\x96\x20\xF0\x9D\x9A\x92\xEF\xBD\x93\x20\xF0\x9D\x91\xAE\xF0\x9D\x9A\x9B\xE2\x84\xAE\xF0\x9D\x9C\xB6\xEF\xBD\x94\x21";
		CheckNullTerminatedStringConversion(TEXT("UnrealEngineIsGreat UTF-16 -> UTF-8"), UnrealEngineIsGreat16, UnrealEngineIsGreat8);
		CheckNullTerminatedStringConversion(TEXT("UnrealEngineIsGreat UTF-8 -> UTF-16"), UnrealEngineIsGreat8, UnrealEngineIsGreat16);

		const UTF16CHAR* Internationalization16 = (const UTF16CHAR*)UTF16TEXT("\x0049\x00F1\x0074\x00EB\x0072\x006E\x00E2\x0074\x0069\x00F4\x006E\x00E0\x006C\x0069\x007A\x00E6\x0074\x0069\x00F8\x006E");
		const UTF8CHAR*  Internationalization8  = (const UTF8CHAR*)"\x49\xC3\xB1\x74\xC3\xAB\x72\x6E\xC3\xA2\x74\x69\xC3\xB4\x6E\xC3\xA0\x6C\x69\x7A\xC3\xA6\x74\x69\xC3\xB8\x6E";
		CheckNullTerminatedStringConversion(TEXT("Internationalization UTF-16 -> UTF-8"), Internationalization16, Internationalization8);
		CheckNullTerminatedStringConversion(TEXT("Internationalization UTF-8 -> UTF-16"), Internationalization8, Internationalization16);

		const UTF16CHAR* WomanEmoji16 = (const UTF16CHAR*)UTF16TEXT("\xD83D\xDC69");
		const UTF8CHAR*  WomanEmoji8  = (const UTF8CHAR*)"\xf0\x9f\x91\xa9";
		CheckNullTerminatedStringConversion(TEXT("WomanEmoji UTF-16 -> UTF-8"), WomanEmoji16, WomanEmoji8);
		CheckNullTerminatedStringConversion(TEXT("WomanEmoji UTF-8 -> UTF-16"), WomanEmoji8, WomanEmoji16);

		const UTF16CHAR* LanguageProcessingChinese16 = (const UTF16CHAR*)UTF16TEXT("\x8bed\x8a00\x5904\x7406");
		const UTF8CHAR*  LanguageProcessingChinese8  = (const UTF8CHAR*)"\xe8\xaf\xad\xe8\xa8\x80\xe5\xa4\x84\xe7\x90\x86";
		CheckNullTerminatedStringConversion(TEXT("LanguageProcessingChinese UTF-16 -> UTF-8"), LanguageProcessingChinese16, LanguageProcessingChinese8);
		CheckNullTerminatedStringConversion(TEXT("LanguageProcessingChinese UTF-8 -> UTF-16"), LanguageProcessingChinese8, LanguageProcessingChinese16);
	}

	// Broken UTF-16 encodings
	{
		CheckNullTerminatedStringConversion(TEXT("Double UTF-16 high surrogate"),   (const UTF16CHAR*)UTF16TEXT("q\xD83D\xD83Dq"), (const UTF8CHAR*)"q??q");
		CheckNullTerminatedStringConversion(TEXT("Double UTF-16 low surrogate"),    (const UTF16CHAR*)UTF16TEXT("q\xDC69\xDC69q"), (const UTF8CHAR*)"q??q");
		CheckNullTerminatedStringConversion(TEXT("Orphaned UTF-16 high surrogate"), (const UTF16CHAR*)UTF16TEXT("q\xDC69q"),       (const UTF8CHAR*)"q?q");
		CheckNullTerminatedStringConversion(TEXT("Orphaned UTF-16 low surrogate"),  (const UTF16CHAR*)UTF16TEXT("q\xDC69q"),       (const UTF8CHAR*)"q?q");
	}

}

#endif //WITH_TESTS
