// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Containers/Queue.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(FQueueTest, "System::Core::Misc::Queue", "[ApplicationContextMask][SmokeFilter]")
{
	// empty queues
	{
		TQueue<int32> Queue;
		int32 OutItem = 0;

		CHECK_MESSAGE(TEXT("A new queue must be empty"), Queue.IsEmpty());

		CHECK_FALSE_MESSAGE(TEXT("A new queue must not dequeue anything"), Queue.Dequeue(OutItem));
		CHECK_FALSE_MESSAGE(TEXT("A new queue must not peek anything"), Queue.Peek(OutItem));
	}

	// insertion & removal
	{
		TQueue<int32> Queue;
		int32 Item1 = 1;
		int32 Item2 = 2;
		int32 Item3 = 3;
		int32 OutItem = 0;

		CHECK_MESSAGE(TEXT("Inserting into a new queue must succeed"), Queue.Enqueue(Item1));
		CHECK_MESSAGE(TEXT("Peek must succeed on a queue with one item"), Queue.Peek(OutItem));
		CHECK_EQUALS(TEXT("Peek must return the first value"), OutItem, Item1);

		CHECK_MESSAGE(TEXT("Inserting into a non-empty queue must succeed"), Queue.Enqueue(Item2));
		CHECK_MESSAGE(TEXT("Peek must succeed on a queue with two items"), Queue.Peek(OutItem));
		CHECK_EQUALS(TEXT("Peek must return the first item"), OutItem, Item1);

		Queue.Enqueue(Item3);

		CHECK_MESSAGE(TEXT("Dequeue must succeed on a queue with three items"), Queue.Dequeue(OutItem));
		CHECK_EQUALS(TEXT("Dequeue must return the first item"), OutItem, Item1);
		CHECK_MESSAGE(TEXT("Dequeue must succeed on a queue with two items"), Queue.Dequeue(OutItem));
		CHECK_EQUALS(TEXT("Dequeue must return the second item"), OutItem, Item2);
		CHECK_MESSAGE(TEXT("Dequeue must succeed on a queue with one item"), Queue.Dequeue(OutItem));
		CHECK_EQUALS(TEXT("Dequeue must return the third item"), OutItem, Item3);

		CHECK_MESSAGE(TEXT("After removing all items, the queue must be empty"), Queue.IsEmpty());
	}

	// emptying
	{
		TQueue<int32> Queue;

		Queue.Enqueue(1);
		Queue.Enqueue(2);
		Queue.Enqueue(3);
		Queue.Empty();

		CHECK_MESSAGE(TEXT("An emptied queue must be empty"), Queue.IsEmpty());
	}
}


#endif //WITH_TESTS
