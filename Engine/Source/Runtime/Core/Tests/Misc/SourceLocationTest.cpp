// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Logging/StructuredLog.h"
#include "Containers/Utf8String.h"
#include "Misc/SourceLocationUtils.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::SourceLocation::Private
{

UE::FSourceLocation GetTestSourceLocation()
{
	return UE::FSourceLocation::Current();
}

/*
 * Helper function to find and remove a single occurrence of the parameter search text
 * from the parameter string builder.
 * @return True if an occurrence was found and removed.
 */
template <typename CharType>
bool RemoveSingle(TStringBuilderBase<CharType>& Builder, TStringView<CharType> SearchText)
{
	if (SearchText.IsEmpty())
	{
		return false;
	}
	const int32 FindPos = Builder.ToView().Find(SearchText);
	if (FindPos != INDEX_NONE)
	{
		Builder.RemoveAt(FindPos, SearchText.Len());
		return true;
	}
	return false;
}

/*
 * Helper function to verify full formatting.
 * @note This validation assumes that the formatted elements appear in a specific order,
 * since it relies on substring removal rather than whole-word matching.
 * e.g. if column "15" is searched in the string "151:15" (line:column),
 * the "15" would match part of "151", causing the subsequent search for "151" to fail.
 */
template <typename CharType>
bool CheckFullFormatting(TStringView<CharType> Text, UE::FSourceLocation Location)
{
	TStringBuilderWithBuffer<CharType, 512> Builder;
	Builder << Text;

	// Verify all required elements are present by removing them one by one.
	if (!RemoveSingle<CharType>(Builder, StringCast<CharType>(Location.GetFileName())))
	{
		return false;
	}
	if (!RemoveSingle<CharType>(Builder, TStringBuilderWithBuffer<CharType, 16>(InPlace, Location.GetLine())))
	{
		return false;
	}
	if (!RemoveSingle<CharType>(Builder, TStringBuilderWithBuffer<CharType, 16>(InPlace, Location.GetColumn())))
	{
		return false;
	}
	if (!RemoveSingle<CharType>(Builder, StringCast<CharType>(Location.GetFunctionName())))
	{
		return false;
	}
	return true;
}
/*
 * Helper function to verify file and line formatting.
 * @note This validation assumes that the formatted elements appear in a specific order,
 * since it relies on substring removal rather than whole-word matching.
 * e.g. if column "15" is searched in the string "151:15" (line:column),
 * the "15" would match part of "151", causing the subsequent search for "151" to fail.
 */
template <typename CharType>
bool CheckFileAndLineFormatting(TStringView<CharType> Text, UE::FSourceLocation Location)
{
	TStringBuilderWithBuffer<CharType, 512> Builder;
	Builder << Text;

	// Verify all required elements are present by removing them one by one.
	if (!RemoveSingle<CharType>(Builder, StringCast<CharType>(Location.GetFileName())))
	{
		return false;
	}
	if (!RemoveSingle<CharType>(Builder, TStringBuilderWithBuffer<CharType, 16>(InPlace, Location.GetLine())))
	{
		return false;
	}
	if (RemoveSingle<CharType>(Builder, TStringBuilderWithBuffer<CharType, 16>(InPlace, Location.GetColumn())))
	{
		return false;
	}
	if (RemoveSingle<CharType>(Builder, StringCast<CharType>(Location.GetFunctionName())))
	{
		return false;
	}
	return true;
}

bool ValidateBinaryForFull(FCbField Field, UE::FSourceLocation Location)
{
	// Must have a value.
	if (!Field.HasValue())
	{
		return false;
	}

	FCbObject Object = Field.AsObject();
	// Check for required fields in full formatting.
	if (!Object.Find("$type").HasValue())
	{
		return false;
	}
	if (!Object.Find("File").HasValue())
	{
		return false;
	}
	if (!Object.Find("Line").HasValue())
	{
		return false;
	}
	if (!Object.Find("Column").HasValue())
	{
		return false;
	}
	if (!Object.Find("Function").HasValue())
	{
		return false;
	}

	// Validate the text field.
	FCbFieldView TextField = Object.Find("$text");
	if (!TextField.HasValue() || !TextField.IsString())
	{
		return false;
	}
	return CheckFullFormatting(TextField.AsString(), Location);
}

bool ValidateBinaryForFileAndLine(FCbField Field, UE::FSourceLocation Location)
{
	// Must have a value.
	if (!Field.HasValue())
	{
		return false;
	}

	FCbObject Object = Field.AsObject();
	// Check for required fields in file and line formatting.
	if (!Object.Find("$type").HasValue())
	{
		return false;
	}
	if (!Object.Find("File").HasValue())
	{
		return false;
	}
	if (!Object.Find("Line").HasValue())
	{
		return false;
	}
	// The following fields must NOT be present
	if (Object.Find("Column").HasValue())
	{
		return false;
	}
	if (Object.Find("Function").HasValue())
	{
		return false;
	}

	// Validate the text field.
	FCbFieldView TextField = Object.Find("$text");
	if (!TextField.HasValue() || !TextField.IsString())
	{
		return false;
	}
	return CheckFileAndLineFormatting(TextField.AsString(), Location);
}

}  // namespace UE::SourceLocation::Private

namespace UE::SourceLocation
{

TEST_CASE("System::Core::Misc::SourceLocation", "[Core][SourceLocation][ApplicationContextMask][SmokeFilter]")
{
	// Get a real source location from a function call - FSourceLocation doesn't allow assigning made-up values for
	// testing.
	UE::FSourceLocation TestLocation = Private::GetTestSourceLocation();

	SECTION("UE::SourceLocation::Full (ToString)")
	{
		const FString String = UE::SourceLocation::Full(TestLocation).ToString();

#if UE_INCLUDE_SOURCE_LOCATION
		CHECK(Private::CheckFullFormatting(MakeStringView(String), TestLocation));
#else
		CHECK(String.IsEmpty());
#endif
	}

	SECTION("UE::SourceLocation::Full (ToUtf8String)")
	{
		const FUtf8String String = UE::SourceLocation::Full(TestLocation).ToUtf8String();

#if UE_INCLUDE_SOURCE_LOCATION
		CHECK(Private::CheckFullFormatting(MakeStringView(String), TestLocation));
#else
		CHECK(String.IsEmpty());
#endif
	}

	SECTION("UE::SourceLocation::Full (Wide String Builder)")
	{
		TWideStringBuilder<256> Builder;
		Builder << UE::SourceLocation::Full(TestLocation);

#if UE_INCLUDE_SOURCE_LOCATION
		CHECK(Private::CheckFullFormatting(Builder.ToView(), TestLocation));
#else
		CHECK(Builder.Len() == 0);
#endif
	}

	SECTION("UE::SourceLocation::Full (Utf8 String Builder)")
	{
		TUtf8StringBuilder<256> Builder;
		Builder << UE::SourceLocation::Full(TestLocation);

#if UE_INCLUDE_SOURCE_LOCATION
		CHECK(Private::CheckFullFormatting(Builder.ToView(), TestLocation));
#else
		CHECK(Builder.Len() == 0);
#endif
	}

	SECTION("UE::SourceLocation::Full (Compact Binary Writer)")
	{
		FCbWriter Writer;
		SerializeForLog(Writer, UE::SourceLocation::Full(TestLocation));
		FCbField Field = Writer.Save();

#if UE_INCLUDE_SOURCE_LOCATION
		CHECK(Private::ValidateBinaryForFull(Field, TestLocation));
#else
		REQUIRE(Field.IsObject());
		// Use FCbObjectView::operator bool() to check if the object has any fields.
		CHECK(!Field.AsObjectView());
#endif
	}

	SECTION("UE::SourceLocation::FileAndLine (ToString)")
	{
		const FString String = UE::SourceLocation::FileAndLine(TestLocation).ToString();

#if UE_INCLUDE_SOURCE_LOCATION
		CHECK(Private::CheckFileAndLineFormatting(MakeStringView(String), TestLocation));
#else
		CHECK(String.IsEmpty());
#endif
	}

	SECTION("UE::SourceLocation::FileAndLine (ToUtf8String)")
	{
		const FUtf8String String = UE::SourceLocation::FileAndLine(TestLocation).ToUtf8String();

#if UE_INCLUDE_SOURCE_LOCATION
		CHECK(Private::CheckFileAndLineFormatting(MakeStringView(String), TestLocation));
#else
		CHECK(String.IsEmpty());
#endif
	}

	SECTION("UE::SourceLocation::FileAndLine (Wide String Builder)")
	{
		TWideStringBuilder<256> Builder;
		Builder << UE::SourceLocation::FileAndLine(TestLocation);

#if UE_INCLUDE_SOURCE_LOCATION
		CHECK(Private::CheckFileAndLineFormatting(Builder.ToView(), TestLocation));
#else
		CHECK(Builder.Len() == 0);
#endif
	}

	SECTION("UE::SourceLocation::FileAndLine (Utf8 String Builder)")
	{
		TUtf8StringBuilder<256> Builder;
		Builder << UE::SourceLocation::FileAndLine(TestLocation);

#if UE_INCLUDE_SOURCE_LOCATION
		CHECK(Private::CheckFileAndLineFormatting(Builder.ToView(), TestLocation));
#else
		CHECK(Builder.Len() == 0);
#endif
	}

	SECTION("UE::SourceLocation::FileAndLine (Compact Binary Writer)")
	{
		FCbWriter Writer;
		SerializeForLog(Writer, UE::SourceLocation::FileAndLine(TestLocation));
		FCbField Field = Writer.Save();

#if UE_INCLUDE_SOURCE_LOCATION
		CHECK(Private::ValidateBinaryForFileAndLine(Field, TestLocation));
#else
		REQUIRE(Field.IsObject());
		// Use FCbObjectView::operator bool() to check if the object has any fields.
		CHECK(!Field.AsObjectView());
#endif
	}
}

}  // namespace UE::SourceLocation

#endif	// WITH_TESTS
