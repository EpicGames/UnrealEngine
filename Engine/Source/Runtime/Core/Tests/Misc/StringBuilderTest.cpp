// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Misc/StringBuilder.h"

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "String/Find.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

namespace UE::Core::Private
{

template <typename T>
	requires requires (FStringBuilderBase& Builder, const T& Value) { Builder << Value; }
struct TAppendable
{
	T Value{};

	inline bool operator==(const TAppendable& Other) const
		requires requires (const T& V) { V == V; }
	{
		return Value == Other.Value;
	}

	friend uint32 GetTypeHash(const TAppendable& Appendable)
		requires requires (const T& V) { GetTypeHash(V); }
	{
		return GetTypeHash(Appendable.Value);
	}

	friend FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const TAppendable& Appendable)
	{
		return Builder << Appendable.Value;
	}
};

template <typename T> TAppendable(const T&) -> TAppendable<T>;
template <typename T> TAppendable(T&&) -> TAppendable<T>;

} // UE::Core::Private

TEST_CASE("Core::String::StringBuilder", "[Core][String][Smoke]")
{
	SECTION("Static")
	{
		STATIC_REQUIRE(std::is_same_v<typename FStringBuilderBase::ElementType, TCHAR>);
		STATIC_REQUIRE(std::is_same_v<typename FAnsiStringBuilderBase::ElementType, ANSICHAR>);
		STATIC_REQUIRE(std::is_same_v<typename FWideStringBuilderBase::ElementType, WIDECHAR>);

		STATIC_REQUIRE(std::is_same_v<FStringBuilderBase, TStringBuilderBase<TCHAR>>);
		STATIC_REQUIRE(std::is_same_v<FAnsiStringBuilderBase, TStringBuilderBase<ANSICHAR>>);
		STATIC_REQUIRE(std::is_same_v<FWideStringBuilderBase, TStringBuilderBase<WIDECHAR>>);

		STATIC_REQUIRE(TIsContiguousContainer<FStringBuilderBase>::Value);
		STATIC_REQUIRE(TIsContiguousContainer<FAnsiStringBuilderBase>::Value);
		STATIC_REQUIRE(TIsContiguousContainer<FWideStringBuilderBase>::Value);

		STATIC_REQUIRE(TIsContiguousContainer<TStringBuilder<128>>::Value);
		STATIC_REQUIRE(TIsContiguousContainer<TAnsiStringBuilder<128>>::Value);
		STATIC_REQUIRE(TIsContiguousContainer<TWideStringBuilder<128>>::Value);
	}

	SECTION("Empty with Buffer")
	{
		TStringBuilder<16> Builder;
		CHECK(Builder.Len() == 0);
		CHECK(Builder == TEXTVIEW(""));
	}

	SECTION("Append Char")
	{
		TStringBuilder<7> Builder;
		Builder << TEXT('A') << TEXT('B') << TEXT('C');
		Builder << 'D' << 'E' << 'F';
		Builder.AppendChar('G').AppendChar('H').AppendChar('I');
		CHECK(Builder == TEXTVIEW("ABCDEFGHI"));
	}

	SECTION("Append Char to ANSI")
	{
		TAnsiStringBuilder<4> Builder;
		Builder << 'A' << 'B' << 'C';
		CHECK(Builder == ANSITEXTVIEW("ABC"));
	}

	SECTION("Append C String")
	{
		TStringBuilder<7> Builder;
		Builder << TEXT("ABC");
		Builder << "DEF";
		CHECK(Builder == TEXTVIEW("ABCDEF"));
	}

	SECTION("Append C String ANSI")
	{
		TAnsiStringBuilder<4> Builder;
		Builder << "ABC";
		CHECK(Builder == ANSITEXTVIEW("ABC"));
	}

	SECTION("Append FStringView")
	{
		TStringBuilder<7> Builder;
		Builder << TEXTVIEW("ABC");
		Builder << ANSITEXTVIEW("DEF");
		CHECK(Builder == TEXTVIEW("ABCDEF"));
	}

	SECTION("Append FAnsiStringView ANSI")
	{
		TAnsiStringBuilder<4> Builder;
		Builder << ANSITEXTVIEW("ABC");
		CHECK(Builder == ANSITEXTVIEW("ABC"));
	}

	SECTION("Append FStringBuilderBase")
	{
		TStringBuilder<4> Builder;
		Builder << TEXT("ABC");
		TStringBuilder<4> BuilderCopy;
		BuilderCopy << Builder;
		CHECK(BuilderCopy == TEXTVIEW("ABC"));
	}

	SECTION("Append FStringBuilderBase ANSI")
	{
		TAnsiStringBuilder<4> Builder;
		Builder << "ABC";
		TAnsiStringBuilder<4> BuilderCopy;
		BuilderCopy << Builder;
		CHECK(BuilderCopy == ANSITEXTVIEW("ABC"));
	}

	SECTION("Append FString")
	{
		TStringBuilder<4> Builder;
		Builder << FString("ABC");
		CHECK(Builder == TEXTVIEW("ABC"));
	}

	SECTION("Append Char Array")
	{
		const auto& String = TEXT("ABC");
		TStringBuilder<4> Builder;
		Builder << String;
		CHECK(Builder == TEXTVIEW("ABC"));
	}

	SECTION("Append Char Array ANSI")
	{
		const ANSICHAR String[16] = "ABC";
		TAnsiStringBuilder<4> Builder;
		Builder << String;
		CHECK(Builder == ANSITEXTVIEW("ABC"));
	}

	SECTION("Simple ReplaceAt")
	{
		TAnsiStringBuilder<4> Builder;
		Builder.ReplaceAt(0, 0, ANSITEXTVIEW(""));
		CHECK(Builder == ANSITEXTVIEW(""));

		Builder.AppendChar('a');
		
		Builder.ReplaceAt(0, 0, ANSITEXTVIEW(""));
		CHECK(Builder == ANSITEXTVIEW("a"));

		Builder.ReplaceAt(0, 1, ANSITEXTVIEW("b"));
		CHECK(Builder == ANSITEXTVIEW("b"));
	}

	SECTION("Advanced ReplaceAt")
	{
		const auto [Original, SearchFor, ReplaceWith, Expected] = GENERATE(table<FAnsiStringView, FAnsiStringView, FAnsiStringView, FAnsiStringView>(
		{
			// Test single character erase
			{".foo", ".", "", "foo"},
			{"f.oo", ".", "", "foo"},
			{"foo.", ".", "", "foo"},
		
			// Test multi character erase
			{"FooBar", "Bar", "", "Foo"},
			{"FooBar", "Foo", "", "Bar"},
			{"FooBar", "Foo", "fOOO", "fOOOBar"},

			// Test replace everything
			{"Foo", "Foo", "", ""},
			{"Foo", "Foo", "Bar", "Bar"},
			{"Foo", "Foo", "0123456789", "0123456789"},

			// Test expanding replace
			{".foo", ".", "<dot>", "<dot>foo"},
			{"foo.", ".", "<dot>", "foo<dot>"},
			{"f.oo", ".", "<dot>", "f<dot>oo"},

			// Test shrinking replace
			{"aabbcc", "aa", "A", "Abbcc"},
			{"aabbcc", "bb", "B", "aaBcc"},
			{"aabbcc", "cc", "C", "aabbC"},
		}));

		int32 ReplacePos = UE::String::FindFirst(Original, SearchFor);
		check(ReplacePos != INDEX_NONE);

		TAnsiStringBuilder<4> Builder;
		Builder << Original;
		Builder.ReplaceAt(ReplacePos, SearchFor.Len(), ReplaceWith);

		CHECK(Builder == Expected);
	}

	SECTION("Prepend")
	{
		TAnsiStringBuilder<4> Builder;

		// Prepend nothing to empty
		Builder.Prepend("");
		CHECK(Builder.Len() == 0);
		
		// Prepend single characer
		Builder.Prepend("e");
		CHECK(Builder == ANSITEXTVIEW("e"));
		
		// Prepend substring
		Builder.Prepend("abcd");
		CHECK(Builder == ANSITEXTVIEW("abcde"));

		// Prepend nothing to non-empty
		Builder.Prepend("");
		CHECK(Builder == ANSITEXTVIEW("abcde"));
	}
	
	SECTION("InsertAt")
	{
		TAnsiStringBuilder<4> Builder;
		Builder.InsertAt(0, "");

		// Insert nothing to empty
		CHECK(Builder.Len() == 0);

		// Insert first char

		Builder.InsertAt(0, "d");
		CHECK(Builder == ANSITEXTVIEW("d"));
		
		// Insert single char
		Builder.InsertAt(0, "c");
		Builder.InsertAt(0, "a");
		Builder.InsertAt(1, "b");
		Builder.InsertAt(4, "e");
		CHECK(Builder == ANSITEXTVIEW("abcde"));
		
		// Insert substrings
		Builder.InsertAt(3, "__");
		Builder.InsertAt(0, "__");
		Builder.InsertAt(Builder.Len(), "__");
		CHECK(Builder == ANSITEXTVIEW("__abc__de__"));

		// Insert nothing
		Builder.InsertAt(Builder.Len(), "");
		CHECK(Builder == ANSITEXTVIEW("__abc__de__"));
	}

	SECTION("RemoveAt")
	{
		TAnsiStringBuilder<4> Builder;
		Builder << "0123456789";

		// Remove nothing
		Builder.RemoveAt(0, 0);
		Builder.RemoveAt(Builder.Len(), 0);
		Builder.RemoveAt(Builder.Len() / 2, 0);
		CHECK(Builder == ANSITEXTVIEW("0123456789"));
		
		// Remove last char
		Builder.RemoveAt(Builder.Len() - 1, 1);
		CHECK(Builder == ANSITEXTVIEW("012345678"));
		
		// Remove first char
		Builder.RemoveAt(0, 1);
		CHECK(Builder == ANSITEXTVIEW("12345678"));
		
		// Remove middle
		Builder.RemoveAt(4, 2);
		CHECK(Builder == ANSITEXTVIEW("123478"));

		// Remove end
		Builder.RemoveAt(4, 2);
		CHECK(Builder == ANSITEXTVIEW("1234"));
		
		// Remove start
		Builder.RemoveAt(0, 2);
		CHECK(Builder == ANSITEXTVIEW("34"));

		// Remove start
		Builder.RemoveAt(0, 2);
		CHECK(Builder == ANSITEXTVIEW(""));
	}

	SECTION("Join")
	{
		using namespace UE::Core::Private;

		TStringBuilder<64> Builder;
		Builder.Join(TArray<TAppendable<int32>>{{1}, {2}, {3}}, TAppendable{", "});
		CHECK(Builder == TEXTVIEW("1, 2, 3"));

		Builder.Reset();
		Builder.Join(TSet<TAppendable<int32>>{{1}, {2}, {3}}, TAppendable{", "});
		CHECK(Builder == TEXTVIEW("1, 2, 3"));
	}

	SECTION("JoinQuoted")
	{
		using namespace UE::Core::Private;

		TStringBuilder<64> Builder;
		Builder.JoinQuoted(TArray<TAppendable<int32>>{{1}, {2}, {3}}, TAppendable{", "}, TAppendable{'"'});
		CHECK(Builder == TEXTVIEW(R"("1", "2", "3")"));

		Builder.Reset();
		Builder.JoinQuoted(TSet<TAppendable<int32>>{{1}, {2}, {3}}, TAppendable{", "}, TAppendable{'"'});
		CHECK(Builder == TEXTVIEW(R"("1", "2", "3")"));
	}

	SECTION("Appendf")
	{
		TStringBuilder<64> Builder;
		const TCHAR* HexDigits = TEXT("0123456789abcdef");
		Builder.Appendf(TEXT("%s%s%s%s%s%s%s%s"), HexDigits, HexDigits, HexDigits, HexDigits, HexDigits, HexDigits, HexDigits, HexDigits);
		CHECK(Builder == TEXTVIEW("0123456789abcdef0123456789abcdef0123456789abcdef0123456789abcdef0123456789abcdef0123456789abcdef0123456789abcdef0123456789abcdef"));
	}

	SECTION("NullTerminator")
	{
		// Validate that space for a null terminator is only allocated when null termination is requested.
		TStringBuilder<16> Builder;
		const TCHAR* ExpectedData = Builder.GetData();
		Builder.Append(TEXTVIEW("0123456789abcdef"));
		CHECK(Builder.GetData() == ExpectedData);
		CHECK_FALSE(*Builder == ExpectedData);
	}
}

#endif