// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(FTCharToUTF8Test, "System::Core::Misc::TCharToUtf8", "[ApplicationContextMask][SmokeFilter]")
{
	{
		// Hebrew letter final nun
		const FString TestString = UTF16_TO_TCHAR(u"\x05DF");
		const ANSICHAR ExpectedResult[] = "\xD7\x9F";
		{
			const FTCHARToUTF8 ConvertedValue(*TestString);
			const FUTF8ToTCHAR ReverseConvertedValue(ConvertedValue.Get());
			CHECK_MESSAGE(FString("Expected test string 1 to have different encoded value"), FCStringAnsi::Strcmp(ExpectedResult, (const ANSICHAR*)ConvertedValue.Get()) == 0);
			CHECK_MESSAGE(FString("Expected test string 1 to have different decoded value"), FCString::Strcmp(*TestString, ReverseConvertedValue.Get()) == 0);
		}

		{
			const FUTF8ToTCHAR ConvertedValue(ExpectedResult);
			const FTCHARToUTF8 ReverseConvertedValue(ConvertedValue.Get());
			CHECK_MESSAGE(FString("Expected test string 2 to have different encoded value"), FCString::Strcmp(*TestString, ConvertedValue.Get()) == 0);
			CHECK_MESSAGE(FString("Expected test string 2 to have different decoded value"), FCStringAnsi::Strcmp(ExpectedResult, (const ANSICHAR*)ReverseConvertedValue.Get()) == 0);
		}
	}

	{
		// Smiling face with open mouth and tightly-closed eyes and a Four of Circles
		const FString TestString = UTF16_TO_TCHAR(u"\xD83D\xDE06\xD83C\xDC1C");
		const ANSICHAR ExpectedResult[] = "\xF0\x9F\x98\x86\xF0\x9F\x80\x9C";
		{
			const FTCHARToUTF8 ConvertedValue(*TestString);
			const FUTF8ToTCHAR ReverseConvertedValue(ConvertedValue.Get());
			CHECK_MESSAGE(FString("Expected test string 3 to have different encoded value"), FCStringAnsi::Strcmp(ExpectedResult, (const ANSICHAR*)ConvertedValue.Get()) == 0);
			CHECK_MESSAGE(FString("Expected test string 3 to have different decoded value"), FCString::Strcmp(*TestString, ReverseConvertedValue.Get()) == 0);
		}

		{
			const FUTF8ToTCHAR ConvertedValue(ExpectedResult);
			const FTCHARToUTF8 ReverseConvertedValue(ConvertedValue.Get());
			CHECK_MESSAGE(FString("Expected test string 4 to have different encoded value"), FCString::Strcmp(*TestString, ConvertedValue.Get()) == 0);
			CHECK_MESSAGE(FString("Expected test string 4 to have different decoded value"), FCStringAnsi::Strcmp(ExpectedResult, (const ANSICHAR*)ReverseConvertedValue.Get()) == 0);
		}
	}

	{
		// "Internationalization" Test string
		const FString TestString = UTF16_TO_TCHAR(u"\x0049\x00F1\x0074\x00EB\x0072\x006E\x00E2\x0074\x0069\x00F4\x006E\x00E0\x006C\x0069\x007A\x00E6\x0074\x0069\x00F8\x006E");
		const ANSICHAR ExpectedResult[] = "\x49\xC3\xB1\x74\xC3\xAB\x72\x6E\xC3\xA2\x74\x69\xC3\xB4\x6E\xC3\xA0\x6C\x69\x7A\xC3\xA6\x74\x69\xC3\xB8\x6E";
		{
			const FTCHARToUTF8 ConvertedValue(*TestString);
			const FUTF8ToTCHAR ReverseConvertedValue(ConvertedValue.Get());
			CHECK_MESSAGE(FString("Expected test string 5 to have different encoded value"), FCStringAnsi::Strcmp(ExpectedResult, (const ANSICHAR*)ConvertedValue.Get()) == 0);
			CHECK_MESSAGE(FString("Expected test string 5 to have different decoded value"), FCString::Strcmp(*TestString, ReverseConvertedValue.Get()) == 0);
		}

		{
			const FUTF8ToTCHAR ConvertedValue(ExpectedResult);
			const FTCHARToUTF8 ReverseConvertedValue(ConvertedValue.Get());
			CHECK_MESSAGE(FString("Expected test string 6 to have different encoded value"), FCString::Strcmp(*TestString, ConvertedValue.Get()) == 0);
			CHECK_MESSAGE(FString("Expected test string 6 to have different decoded value"), FCStringAnsi::Strcmp(ExpectedResult, (const ANSICHAR*)ReverseConvertedValue.Get()) == 0);
		}
	}

	{
		// UE Test string
		const FString TestString = UTF16_TO_TCHAR(u"\xD835\xDE50\x043F\xD835\xDDCB\xD835\xDE26\xD835\xDD52\x006C\x0020\xD835\xDC6C\xD835\xDD2B\xFF47\xD835\xDCBE\xD835\xDD93\xD835\xDD56\x0020\xD835\xDE92\xFF53\x0020\xD835\xDC6E\xD835\xDE9B\x212E\xD835\xDF36\xFF54\x0021");
		const ANSICHAR ExpectedResult[] = "\xF0\x9D\x99\x90\xD0\xBF\xF0\x9D\x97\x8B\xF0\x9D\x98\xA6\xF0\x9D\x95\x92\x6C\x20\xF0\x9D\x91\xAC\xF0\x9D\x94\xAB\xEF\xBD\x87\xF0\x9D\x92\xBE\xF0\x9D\x96\x93\xF0\x9D\x95\x96\x20\xF0\x9D\x9A\x92\xEF\xBD\x93\x20\xF0\x9D\x91\xAE\xF0\x9D\x9A\x9B\xE2\x84\xAE\xF0\x9D\x9C\xB6\xEF\xBD\x94\x21";
		{
			const FTCHARToUTF8 ConvertedValue(*TestString);
			const FUTF8ToTCHAR ReverseConvertedValue(ConvertedValue.Get());
			CHECK_MESSAGE(FString("Expected test string 7 to have different encoded value"), FCStringAnsi::Strcmp(ExpectedResult, (const ANSICHAR*)ConvertedValue.Get()) == 0);
			CHECK_MESSAGE(FString("Expected test string 7 to have different decoded value"), FCString::Strcmp(*TestString, ReverseConvertedValue.Get()) == 0);
		}

		{
			const FUTF8ToTCHAR ConvertedValue(ExpectedResult);
			const FTCHARToUTF8 ReverseConvertedValue(ConvertedValue.Get());
			CHECK_MESSAGE(FString("Expected test string 8 to have different encoded value"), FCString::Strcmp(*TestString, ConvertedValue.Get()) == 0);
			CHECK_MESSAGE(FString("Expected test string 8 to have different decoded value"), FCStringAnsi::Strcmp(ExpectedResult, (const ANSICHAR*)ReverseConvertedValue.Get()) == 0);
		}
	}

	{
		// Null terminator
		const FString TestString = "Test";
		{
			const FTCHARToUTF8 ConvertedValue(*TestString);
			const bool bIsNullTerminated = *(ConvertedValue.Get() + ConvertedValue.Length()) == 0;
			CHECK_MESSAGE(TEXT("Expected converted value to be null-terminated"), bIsNullTerminated);
		}
		{
			const FTCHARToUTF8 ConvertedValue(*TestString, TestString.Len() + 1); // include the null terminator
			const bool bIsNullTerminated = *(ConvertedValue.Get() + ConvertedValue.Length()) == 0;
			CHECK_MESSAGE(TEXT("Expected converted value to be null-terminated"), bIsNullTerminated);
		}
	}

	{
		// Bad UTF-16 surrogates
		{
			const FTCHARToUTF8 EndWithHighSurrogate(UTF16_TO_TCHAR(u"Hello\xD83D"));
			CHECK_MESSAGE(TEXT("UTF-16 string ending with a single high surrogate converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)EndWithHighSurrogate.Get(), (const UTF8CHAR*)"Hello?") == 0);
		}
		{
			const FTCHARToUTF8 EndWithTwoHighSurrogates(UTF16_TO_TCHAR(u"Hello\xD83D\xD83D"));
			CHECK_MESSAGE(TEXT("UTF-16 string ending with two high surrogates converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)EndWithTwoHighSurrogates.Get(), (const UTF8CHAR*)"Hello??") == 0);
		}
		{
			const FTCHARToUTF8 EndWithThreeHighSurrogates(UTF16_TO_TCHAR(u"Hello\xD83D\xD83D\xD83D"));
			CHECK_MESSAGE(TEXT("UTF-16 string ending with three high surrogates converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)EndWithThreeHighSurrogates.Get(), (const UTF8CHAR*)"Hello???") == 0);
		}
		{
			const FTCHARToUTF8 EndWithHighSurrogateAndPair(UTF16_TO_TCHAR(u"Hello\xD83D\xD83D\xDC69"));
			CHECK_MESSAGE(TEXT("UTF-16 string ending with a high surrogate and a surrogate pair converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)EndWithHighSurrogateAndPair.Get(), (const UTF8CHAR*)"Hello?\xf0\x9f\x91\xa9") == 0);
		}
		{
			const FTCHARToUTF8 EndWithLowSurrogate(UTF16_TO_TCHAR(u"Hello\xDC69"));
			CHECK_MESSAGE(TEXT("UTF-16 string ending with a single low surrogate converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)EndWithLowSurrogate.Get(), (const UTF8CHAR*)"Hello?") == 0);
		}
		{
			const FTCHARToUTF8 EndWithTwoLowSurrogates(UTF16_TO_TCHAR(u"Hello\xDC69\xDC69"));
			CHECK_MESSAGE(TEXT("UTF-16 string ending with two low surrogates converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)EndWithTwoLowSurrogates.Get(), (const UTF8CHAR*)"Hello??") == 0);
		}
		{
			const FTCHARToUTF8 EndWithThreeLowSurrogates(UTF16_TO_TCHAR(u"Hello\xDC69\xDC69\xDC69"));
			CHECK_MESSAGE(TEXT("UTF-16 string ending with three low surrogates converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)EndWithThreeLowSurrogates.Get(), (const UTF8CHAR*)"Hello???") == 0);
		}
		{
			const FTCHARToUTF8 EndWithLowSurrogateAndPair(UTF16_TO_TCHAR(u"Hello\xDC69\xD83D\xDC69"));
			CHECK_MESSAGE(TEXT("UTF-16 string ending with a low surrogate and a surrogate pair converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)EndWithLowSurrogateAndPair.Get(), (const UTF8CHAR*)"Hello?\xf0\x9f\x91\xa9") == 0);
		}
		{
			const FTCHARToUTF8 MidHighSurrogate(UTF16_TO_TCHAR(u"Hello\xD83DWorld"));
			CHECK_MESSAGE(TEXT("UTF-16 string containing a single high surrogate converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)MidHighSurrogate.Get(), (const UTF8CHAR*)"Hello?World") == 0);
		}
		{
			const FTCHARToUTF8 MidTwoHighSurrogates(UTF16_TO_TCHAR(u"Hello\xD83D\xD83DWorld"));
			CHECK_MESSAGE(TEXT("UTF-16 string containing two high surrogates converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)MidTwoHighSurrogates.Get(), (const UTF8CHAR*)"Hello??World") == 0);
		}
		{
			const FTCHARToUTF8 MidThreeHighSurrogates(UTF16_TO_TCHAR(u"Hello\xD83D\xD83D\xD83DWorld"));
			CHECK_MESSAGE(TEXT("UTF-16 string containing three high surrogates converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)MidThreeHighSurrogates.Get(), (const UTF8CHAR*)"Hello???World") == 0);
		}
		{
			const FTCHARToUTF8 MidHighSurrogateAndPair(UTF16_TO_TCHAR(u"Hello\xD83D\xD83D\xDC69World"));
			CHECK_MESSAGE(TEXT("UTF-16 string containing a high surrogate and a surrogate pair converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)MidHighSurrogateAndPair.Get(), (const UTF8CHAR*)"Hello?\xf0\x9f\x91\xa9World") == 0);
		}
		{
			const FTCHARToUTF8 MidLowSurrogate(UTF16_TO_TCHAR(u"Hello\xDC69World"));
			CHECK_MESSAGE(TEXT("UTF-16 string containing a single low surrogate converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)MidLowSurrogate.Get(), (const UTF8CHAR*)"Hello?World") == 0);
		}
		{
			const FTCHARToUTF8 MidTwoLowSurrogates(UTF16_TO_TCHAR(u"Hello\xDC69\xDC69World"));
			CHECK_MESSAGE(TEXT("UTF-16 string containing two low surrogates converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)MidTwoLowSurrogates.Get(), (const UTF8CHAR*)"Hello??World") == 0);
		}
		{
			const FTCHARToUTF8 MidThreeLowSurrogates(UTF16_TO_TCHAR(u"Hello\xDC69\xDC69\xDC69World"));
			CHECK_MESSAGE(TEXT("UTF-16 string containing three low surrogates converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)MidThreeLowSurrogates.Get(), (const UTF8CHAR*)"Hello???World") == 0);
		}
		{
			const FTCHARToUTF8 MidLowSurrogateAndPair(UTF16_TO_TCHAR(u"Hello\xDC69\xD83D\xDC69World"));
			CHECK_MESSAGE(TEXT("UTF-16 string containing a low surrogate and a surrogate pair converted to UTF-8"), FCStringUtf8::Strcmp((const UTF8CHAR*)MidLowSurrogateAndPair.Get(), (const UTF8CHAR*)"Hello?\xf0\x9f\x91\xa9World") == 0);
		}
	}

	{
		// Verify that we handle invalid UTF-16 strings that ends in half a surrogate pair w/o crashing
#if PLATFORM_TCHAR_IS_UTF8CHAR
#ifdef _MSC_VER
		// We need to do this because MSVC doesn't encode numeric escape sequences properly in u8"" literals:
		// https://developercommunity.visualstudio.com/t/hex-escape-codes-in-a-utf8-literal-are-t/225847
		char EndWithIllegalSurrogatePairImpl[] = "ab\xED\xA0\x80";
		const UTF8CHAR* EndWithIllegalSurrogatePair = (const UTF8CHAR*)EndWithIllegalSurrogatePairImpl;
#else
		const TCHAR* EndWithIllegalSurrogatePair = TEXT("ab\xED\xA0\x80"));
#endif // #ifdef _MSC_VER
#else
		const TCHAR* EndWithIllegalSurrogatePair = TEXT("ab\xD800");
#endif // #if PLATFORM_TCHAR_IS_UTF8CHAR

		CHECK_EQUALS("IllegalSurrogatePair", TCHAR_TO_UTF8(EndWithIllegalSurrogatePair), "ab?");
	}
}

#endif //WITH_TESTS
