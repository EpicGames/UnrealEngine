// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Math/UnrealMath.h"
#include "Misc/TVariant.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::Core::Private
{
	// TIsVariant_V of variants
	static_assert(TIsVariant_V<      TVariant<int, float>>, "Expected TIsVariant_V<TVariant> to be true");
	static_assert(TIsVariant_V<const TVariant<int, float>>, "Expected TIsVariant_V<const TVariant> to be true");

	// TIsVariant_V of non-variants (references are not variants)
	static_assert(TIsVariant_V<      TVariant<int, float>&>  == false, "Expected TIsVariant_V<TVariant&> to be false");
	static_assert(TIsVariant_V<const TVariant<int, float>&>  == false, "Expected TIsVariant_V<const TVariant&> to be false");
	static_assert(TIsVariant_V<      TVariant<int, float>&&> == false, "Expected TIsVariant_V<TVariant&&> to be false");
	static_assert(TIsVariant_V<      bool>                   == false, "Expected TIsVariant_V<non-TVariant> to be false");

	// TVariantSize_V of variants (references are not variants)
	static_assert(TVariantSize_V<      TVariant<int, float>> == 2, "Expected TVariantSize_V<TVariant> to be 2");
	static_assert(TVariantSize_V<const TVariant<int, float>> == 2, "Expected TVariantSize_V<const TVariant> to be 2");

	// TVariantSize_V of non-variants
	static_assert(TVariantSize_V<      TVariant<int, float>&>  == 0, "Expected TVariantSize_V<TVariant&> to be 0");
	static_assert(TVariantSize_V<const TVariant<int, float>&>  == 0, "Expected TVariantSize_V<const TVariant&> to be 0");
	static_assert(TVariantSize_V<      TVariant<int, float>&&> == 0, "Expected TVariantSize_V<TVariant&&> to be 0");
	static_assert(TVariantSize_V<      bool>                   == 0, "Expected TVariantSize_V<non-TVariant> to be 0");

	// Ensure that TVariant has a trivial destructor if all the types are
	static_assert( std::is_trivially_destructible_v<TVariant<int>>);
	static_assert( std::is_trivially_destructible_v<TVariant<int, FVector>>);
	static_assert( std::is_trivially_destructible_v<TVariant<int, FVector, FBox>>);
	static_assert(!std::is_trivially_destructible_v<TVariant<FString>>);
	static_assert(!std::is_trivially_destructible_v<TVariant<int, FString>>);
	static_assert(!std::is_trivially_destructible_v<TVariant<int, FVector, FString>>);
} // namespace UE::Core::Private

struct FNonDefaultConstructible
{
	int _A;
	float _B;
	FNonDefaultConstructible(int A, float B) : _A(A), _B(B) {}

	bool operator==(const FNonDefaultConstructible& Other) const
	{
		return _A == Other._A && _B == Other._B;
	}
};

TEST_CASE_NAMED(FTVariantInPlaceConstructionTest, "System::Core::Misc::TVariant::InPlaceConstruction", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant supports in-place construction of a type in its parameter pack
	using FVariant = TVariant<FNonDefaultConstructible>;
	
	FVariant Var{ TInPlaceType<FNonDefaultConstructible>(), 30, 40.0f };
	check(Var.IsType<FNonDefaultConstructible>());
	check(Var.Get<FNonDefaultConstructible>() == FNonDefaultConstructible(30, 40.0f));

}

TEST_CASE_NAMED(FTVariantDefaultConstructorTest, "System::Core::Misc::TVariant::DefaultConstruction", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant default constructor constructs the first type in the parameter pack
	TVariant<int, float, FString> Int;
	check(Int.IsType<int>());
	check(Int.Get<int>() == 0);
	
	TVariant<FString, int, float> String;
	check(String.IsType<FString>());
	check(String.Get<FString>() == FString{});
	
}

TEST_CASE_NAMED(FTVariantEmptyVariantStateTest, "System::Core::Misc::TVariant::EmptyVariantState", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant default constructor works for the provided FEmptyVariantState
	TVariant<FEmptyVariantState, FNonDefaultConstructible> Var;
	check(Var.IsType<FEmptyVariantState>());
	
}

TEST_CASE_NAMED(FTVariantCopyTest, "System::Core::Misc::TVariant.Copy", "[ApplicationContextMask][EngineFilter]")
{
	// TVariants can be copied
	using FVariant = TVariant<int, float, double>;
	FVariant Int{ TInPlaceType<int>(), 40 };
	FVariant Copy{ Int };

	check(Int.IsType<int>());
	check(Copy.IsType<int>());
	check(Int.Get<int>() == Copy.Get<int>());

}

TEST_CASE_NAMED(FTVariantMoveTest, "System::Core::Misc::TVariant::Move", "[ApplicationContextMask][EngineFilter]")
{
	// TVariants can be moved
	using FVariant = TVariant<int, float, double>;
	FVariant Original{ TInPlaceType<int>(), 40 };
	FVariant NewLoc{ MoveTemp(Original) };

	check(NewLoc.IsType<int>());
	check(NewLoc.Get<int>() == 40);

}

TEST_CASE_NAMED(FTVariantAssignTest, "System::Core::Misc::TVariant::Assign", "[ApplicationContextMask][EngineFilter]")
{
	// TVariants can be assigned from TVariants of the same template parameter pack

	using FVariant = TVariant<int, float, double>;

	// Copy Assign
	{
		FVariant One{ TInPlaceType<double>(), 20.0 };
		FVariant Two{ TInPlaceType<int>(), 40 };

		One = Two;
		check(One.IsType<int>());
		check(One.Get<int>() == 40);

		check(Two.IsType<int>());
		check(Two.Get<int>() == 40);
	}

	// Move Assign
	{
		FVariant One{ TInPlaceType<double>(), 20.0 };
		FVariant Two{ TInPlaceType<int>(), 40 };

		One = MoveTemp(Two);
		check(One.IsType<int>());
		check(One.Get<int>() == 40);
	}

}

TEST_CASE_NAMED(FTVariantConstGetTest, "System::Core::Misc::TVariant::ConstGet", "[ApplicationContextMask][EngineFilter]")
{
	// const TVariant values can be gotten
	const TVariant<int, float, double> ConstVar{ TInPlaceType<float>(), 20.0f };
	check(ConstVar.IsType<float>());
	check(ConstVar.Get<float>() == 20.0f);

}

TEST_CASE_NAMED(FTVariantGetDefaultTest, "System::Core::Misc::TVariant::GetDefault", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant returns provided default when requested (valid) type isn't the one actually stored
	{
		const TVariant<int, float, double> ConstVar{ TInPlaceType<float>(), 20.0f }; // Set float
		check(!ConstVar.IsType<int>()); // Not int

		static constexpr int DefaultInt = 55;
		check(ConstVar.Get<int>(DefaultInt) == DefaultInt); // Get int, should return default value
	}

	// TVariant returns provided default when requested (valid) type isn't the one actually stored
	{
		const TVariant<int, float, double> ConstVar{ TInPlaceType<float>(), 20.0f }; // Set float
		check(!ConstVar.IsType<int>()); // Not int

		static constexpr int DefaultInt = 55;
		check(ConstVar.Get<int>(DefaultInt) == DefaultInt); // Get int, should return default value
	}

	// TVariant returns stored value when requested type is the one actually stored
	{
		static constexpr int StoredInt = 15;
		const TVariant<int, float, double> ConstVar{ TInPlaceType<int>(), StoredInt }; // Set int
		check(ConstVar.IsType<int>()); // Is int

		static constexpr int DefaultInt = 55;
		check(ConstVar.Get<int>(DefaultInt) == StoredInt); // Get int, should return stored value, not the provided default value
	}

}

TEST_CASE_NAMED(FTVariantTryGetTest, "System::Core::Misc::TVariant::TryGet", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant supports conditional get

	// mutable
	{
		TVariant<int, float, double> Int{ TInPlaceType<int>(), 20 };

		check(Int.TryGet<double>() == nullptr);

		int* Value = Int.TryGet<int>();
		check(Value != nullptr);
		check(*Value == 20);
	}

	// const
	{
		const TVariant<int, float, double> Int{ TInPlaceType<int>(), 20 };

		check(Int.TryGet<double>() == nullptr);

		const int* Value = Int.TryGet<int>();
		check(Value != nullptr);
		check(*Value == 20);
	}

}

TEST_CASE_NAMED(FTVariantSharedPtrTest, "System::Core::Misc::TVariant::SharedPtr", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant can successfully hold a TSharedPtr that goes out of scope
	
	// Assignment
	{
		TVariant<TSharedPtr<int>> Var;
		{
			TSharedPtr<int> Ptr = MakeShared<int>(20);
			Var.Set<TSharedPtr<int>>(Ptr);
		}

		check(Var.IsType<TSharedPtr<int>>());

		TSharedPtr<int>& PtrRef = Var.Get<TSharedPtr<int>>();
		check(PtrRef.IsValid());
		check(*PtrRef == 20);
	}

	// Construction
	{
		TUniquePtr<TVariant<TSharedPtr<int>>> Var = nullptr;
		{
			TSharedPtr<int> Ptr = MakeShared<int>(20);
			Var = MakeUnique<TVariant<TSharedPtr<int>>>(TInPlaceType<TSharedPtr<int>>(), Ptr);
		}

		check(Var->IsType<TSharedPtr<int>>());

		TSharedPtr<int>& PtrRef = Var->Get<TSharedPtr<int>>();
		check(PtrRef.IsValid());
		check(*PtrRef == 20);
	}
}

TEST_CASE_NAMED(FTVariantUniquePtrTest, "System::Core::Misc::TVariant::UniquePtr", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant can successfully hold a TUniquePtr

	// Assignment
	{
		TVariant<TUniquePtr<int>> Var;
		Var.Set<TUniquePtr<int>>(MakeUnique<int>(20));

		check(Var.IsType<TUniquePtr<int>>());

		TUniquePtr<int>& PtrRef = Var.Get<TUniquePtr<int>>();
		check(*PtrRef == 20);
	}

	// Construction
	{
		TVariant<TUniquePtr<int>> Var{ TInPlaceType<TUniquePtr<int>>(), MakeUnique<int>(20) };
		check(Var.IsType<TUniquePtr<int>>());

		TUniquePtr<int>& PtrRef = Var.Get<TUniquePtr<int>>();
		check(*PtrRef == 20);
	}

}

TEST_CASE_NAMED(FTVariantMoveAssignNonCopyableTest, "System::Core::Misc::TVariant::MoveAssignNonCopyable", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant holding a non-copyable can be move assigned
	struct FNoCopy
	{
		int Data = 0;
		FNoCopy() = default;
		explicit FNoCopy(int _Data) : Data(_Data) {}
		FNoCopy(const FNoCopy&) = delete;
		FNoCopy(FNoCopy&& Other) = default;
		FNoCopy& operator=(const FNoCopy&) = delete;
		FNoCopy& operator=(FNoCopy&& Other) = default;
	};

	TVariant<FNoCopy> Var{ TInPlaceType<FNoCopy>(), 20 };

	TVariant<FNoCopy> Moved;
	Moved = MoveTemp(Var);
	check(Moved.Get<FNoCopy>().Data == 20);

}

TEST_CASE_NAMED(FTVariantEmplaceTest, "System::Core::Misc::TVariant::Emplace", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant can emplace values directly
	TVariant<int, float, FString> Var;

	Var.Emplace<int>(20);
	check(Var.IsType<int>());
	check(Var.Get<int>() == 20);

	Var.Emplace<float>(30.0f);
	check(Var.IsType<float>());
	check(Var.Get<float>() == 30.0f);

	Var.Emplace<FString>(TEXT("FOO"));
	check(Var.IsType<FString>());
	check(Var.Get<FString>() == FString{ "FOO" });

}

TEST_CASE_NAMED(FTVariantEmplaceNonMoveNonCopyTest, "System::Core::Misc::TVariant::EmplaceNonMoveNonCopy", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant can emplace values that are neither moveable nor copyable
	struct FNonMoveNonCopy
	{
		int Data;
		explicit FNonMoveNonCopy(int _Data) : Data(_Data) {}
		~FNonMoveNonCopy() = default;
		FNonMoveNonCopy(FNonMoveNonCopy&&) = delete;
		FNonMoveNonCopy(const FNonMoveNonCopy&) = delete;
		FNonMoveNonCopy& operator=(FNonMoveNonCopy&&) = delete;
		FNonMoveNonCopy& operator=(const FNonMoveNonCopy&) = delete;
	};

	TVariant<FEmptyVariantState, FNonMoveNonCopy> Var;
	Var.Emplace<FNonMoveNonCopy>(20);
	check(Var.Get<FNonMoveNonCopy>().Data == 20);

}

TEST_CASE_NAMED(FTVariantSelfAssignTest, "System::Core::Misc::TVariant::SelfAssign", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant self-assignment is harmless
	TVariant<int, float, double> Var{ TInPlaceType<int>(), 20 };

	const TVariant<int, float, double>& Ref = Var; // trick clang into thinking we're not assigning the same variable to itself.

	Var = Ref;
	check(Var.Get<int>() == 20);

	Var = MoveTemp(Var);
	check(Var.Get<int>() == 20);

}

TEST_CASE_NAMED(FTVariantSwitchPatternMatchingTest, "System::Core::Misc::TVariant::SwitchPatternMatching", "[ApplicationContextMask][EngineFilter]")
{
	// TVariant supports switch-based 'pattern matching
	using FVariant = TVariant<int, float, double>;
	check(FVariant::IndexOfType<int>() == 0);
	check(FVariant::IndexOfType<float>() == 1);
	check(FVariant::IndexOfType<double>() == 2);

	SIZE_T IndexTaken = (SIZE_T)-1;
	auto Tester = [&IndexTaken](const FVariant& Var)
	{
		switch (IndexTaken = Var.GetIndex())
		{
			case FVariant::IndexOfType<int>():
			{
				int Value = Var.Get<int>();
				check(Value == 20);
				break;
			}

			case FVariant::IndexOfType<float>():
			{
				float Value = Var.Get<float>();
				check(Value == 30.0f);
				break;
			}

			case FVariant::IndexOfType<double>():
			{
				double Value = Var.Get<double>();
				check(Value == 10.0);
				break;
			}

			default:
				check(!"Should not get here");
				break;
		}
	};

	FVariant Var;

	Var.Set<int>(20);
	Tester(Var);
	check(IndexTaken == FVariant::IndexOfType<int>());

	Var.Set<float>(30.0f);
	Tester(Var);
	check(IndexTaken == FVariant::IndexOfType<float>());

	Var.Set<double>(10.0);
	Tester(Var);
	check(IndexTaken == FVariant::IndexOfType<double>());

}


TEST_CASE_NAMED(FTVariantDestructorTest, "System::Core::Misc::TVariant::Destructor", "[ApplicationContextMask][EngineFilter]")
{
	// Destructor test
	struct FTestItem
	{
		int* DestructorCallCount;
		FTestItem(int* _DestructorCallCount) : DestructorCallCount(_DestructorCallCount) {}
		~FTestItem()
		{
			(*DestructorCallCount)++;
		}
	};

	int DestructorCallCount = 0;
	{
		TVariant<TUniquePtr<FTestItem>> MovedFrom{ TInPlaceType<TUniquePtr<FTestItem>>(), MakeUnique<FTestItem>(&DestructorCallCount) };
		TVariant<TUniquePtr<FTestItem>> MovedInto{ MoveTemp(MovedFrom) };
	}
	check(DestructorCallCount == 1);

	DestructorCallCount = 0;
	{
		TVariant<TSharedPtr<FTestItem>> MovedFrom{ TInPlaceType<TSharedPtr<FTestItem>>(), MakeShared<FTestItem>(&DestructorCallCount) };
		TVariant<TSharedPtr<FTestItem>> MovedInto{ MoveTemp(MovedFrom) };
	}
	check(DestructorCallCount == 1);

}

TEST_CASE_NAMED(FTVariantVisitTest, "System::Core::Misc::TVariant::Visit", "[ApplicationContextMask][EngineFilter]")
{
	// Visit calls the correct function overload
	enum class EVisitType
	{
		IntFloat,
		IntInt,
		IntStr,
		FloatInt,
		FloatFloat,
		FloatStr,
		StrInt,
		StrFloat,
		StrStr,
	};
	struct FCallable
	{
		EVisitType operator()(int, float) { return EVisitType::IntFloat; }
		EVisitType operator()(int, int) { return EVisitType::IntInt; }
		EVisitType operator()(int, const FString&) { return EVisitType::IntStr; }
		EVisitType operator()(float, int) { return EVisitType::FloatInt; }
		EVisitType operator()(float, float) { return EVisitType::FloatFloat; }
		EVisitType operator()(float, const FString&) { return EVisitType::FloatStr; }
		EVisitType operator()(const FString&, int) { return EVisitType::StrInt; }
		EVisitType operator()(const FString&, float) { return EVisitType::StrFloat; }
		EVisitType operator()(const FString&, const FString&) { return EVisitType::StrStr; }
	};

	TVariant<int, float> IntV(TInPlaceType<int>(), 0);
	const TVariant<int, float, FString> FloatV(TInPlaceType<float>(), 0.0f);
	const TVariant<FString> StrV(TInPlaceType<FString>(), TEXT(""));

#define CHECK_VISIT(Type1, Type2) check(Visit(FCallable{}, Type1##V, Type2##V) == EVisitType::Type1##Type2)
	CHECK_VISIT(Int, Int);
	CHECK_VISIT(Int, Float);
	CHECK_VISIT(Int, Str);
	CHECK_VISIT(Float, Int);
	CHECK_VISIT(Float, Float);
	CHECK_VISIT(Float, Str);
	CHECK_VISIT(Str, Int);
	CHECK_VISIT(Str, Float);
	CHECK_VISIT(Str, Str);
#undef CHECK_VISIT

}

TEST_CASE_NAMED(FTVariantVisitorRvalueReferenceParamTest, "System::Core::Misc::TVariant::VisitorRvalueReferenceParam", "[ApplicationContextMask][EngineFilter]")
{
	// Visitor implementation can have rvalue reference parameters
	struct FAnonOne {};
	struct FAnonTwo {};

	using FVariant = TVariant<FAnonOne, FAnonTwo>;

	struct FAnonVisitor
	{
		void operator()(FAnonOne&&)
		{
		}

		void operator()(FAnonTwo&&)
		{
		}
	};

	FVariant MoveThisVariant;
	Visit(FAnonVisitor{}, MoveTemp(MoveThisVariant));

	// No checks, this is a compilation test.
}

TEST_CASE_NAMED(FTVariantSerializationTest, "System::Core::Misc::TVariant::Serialization", "[ApplicationContextMask][EngineFilter]")
{
	// TVariants can be serialized
	TArray<uint8> Data;
	FMemoryWriter Writer(Data);

	TVariant<int, FString> VarData;
	VarData.Emplace<int>(25);

	Writer << VarData;

	VarData.Emplace<FString>(TEXT("Foo"));
	Writer << VarData;


	FMemoryReader Reader(Data);
	Reader << VarData;
	check(VarData.IsType<int>());
	check(VarData.Get<int>() == 25);

	Reader << VarData;
	check(VarData.IsType<FString>());
	check(VarData.Get<FString>() == FString("Foo"));

}

#endif //WITH_TESTS
