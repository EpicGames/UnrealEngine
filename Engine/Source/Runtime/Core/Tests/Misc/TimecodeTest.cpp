// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Async/Async.h"
#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "Misc/Timespan.h"
#include "Tests/TestHarnessAdapter.h"


#if WITH_TESTS

/**
 * Run a suite of timecode conversion operations to validate conversion from timecode to timespan/FrameNumber are working
 *
 * Drop Frame drop a frame every minute except every 10th minute
 * 29.97fps
 * 00:58:01:28 ; 00:58:01:29 ; 00:58:02:00 ; 00:58:02:01 (no skip)
 * 01:00:59:28 ; 01:00:59:29 ; 01:01:00:02 ; 01:01:00:03 (every minute, we skip frame 0 and 1)
 * 01:09:59:28 ; 01:09:59:29 ; 01:10:00:00 ; 01:10:00:01 (except every 10th minute, we include frame 0 and 1)
 */
TEST_CASE_NAMED(FTimecodeTest, "System::Core::Misc::Timecode", "[ApplicationContextMask][EngineFilter]")
{
	FFrameRate CommonFrameRates[]{
		FFrameRate(12, 1),
		FFrameRate(15, 1),
		FFrameRate(24, 1),
		FFrameRate(25, 1),
		FFrameRate(30, 1),
		FFrameRate(48, 1),
		FFrameRate(48, 2), // Should give the same result as 24/1
		FFrameRate(50, 1),
		FFrameRate(60, 1),
		FFrameRate(100, 1),
		FFrameRate(120, 1),
		FFrameRate(240, 1),
		FFrameRate(24000, 1001),
		FFrameRate(30000, 1001),
		FFrameRate(48000, 1001),
		FFrameRate(60000, 1001),
	};

	auto ConversionWithFrameRateTest = [](const FFrameRate FrameRate)
	{
		const bool bIsDropFrame = FTimecode::IsDropFormatTimecodeSupported(FrameRate);
		int32 NumberOfErrors = 0;
		FTimecode PreviousTimecodeValue;

		const int64 StartIndex = 0;
		for (int64 FrameIndex = StartIndex; FrameIndex <= MAX_int32; ++FrameIndex)
		{
			const FFrameNumber FrameNumber = static_cast<int32>(FrameIndex);
			const FTimecode TimecodeValue = FTimecode::FromFrameNumber(FrameNumber, FrameRate, bIsDropFrame);
			bool bDoTest = true;

			// Conversion from FrameNumber to Timecode
			if (bDoTest)
			{
				const FFrameNumber ExpectedFrameNumber = TimecodeValue.ToFrameNumber(FrameRate);
				if (FrameNumber != ExpectedFrameNumber)
				{
					FAIL_CHECK(FString::Printf(TEXT("Timecode '%s' didn't convert properly from FrameNumber '%d' for FrameRate '%s'.")
						, *TimecodeValue.ToString()
						, FrameNumber.Value
						, *FrameRate.ToPrettyText().ToString()
					));
					bDoTest = false;
					++NumberOfErrors;
				}
			}
			
			// Conversion from Timespan to Timecode
			if (bDoTest)
			{
				const FTimespan TimespanFromTimecode = TimecodeValue.ToTimespan(FrameRate);
				const FTimecode TimecodeFromTimespanWithRollover = FTimecode::FromTimespan(TimespanFromTimecode, FrameRate, bIsDropFrame, true);
				const FTimecode TimecodeFromTimespanWithoutRollover = FTimecode::FromTimespan(TimespanFromTimecode, FrameRate, bIsDropFrame, false);

				if (TimecodeFromTimespanWithoutRollover != TimecodeValue)
				{
					FAIL_CHECK(FString::Printf(TEXT("Timecode '%s' didn't convert properly from Timespan '%f' with rollover for frame rate '%s'.")
						, *TimecodeValue.ToString()
						, TimespanFromTimecode.GetTotalSeconds()
						, *FrameRate.ToPrettyText().ToString()
					));
					bDoTest = false;
					++NumberOfErrors;
				}
				else if (TimecodeFromTimespanWithoutRollover.Minutes != TimecodeValue.Minutes || TimecodeFromTimespanWithoutRollover.Seconds != TimecodeValue.Seconds || TimecodeFromTimespanWithoutRollover.Frames != TimecodeValue.Frames)
				{
					FAIL_CHECK(FString::Printf(TEXT("Timecode '%s' didn't convert properly from Timespan '%f' without rollover for frame rate '%s'.")
						, *TimecodeValue.ToString()
						, TimespanFromTimecode.GetTotalSeconds()
						, *FrameRate.ToPrettyText().ToString()
					));
					bDoTest = false;
					++NumberOfErrors;
				}
				else if (!bIsDropFrame)
				{
					// Do they have the same hours, minutes, seconds
					// To test this, we start from the number of events (FrameIndex) from which we got our timecode first
					// Timecode is just a label and doesn't necessarily reflect real time especially with 23.976 FrameRate like
					const int32 TotalSeconds = FMath::FloorToInt32((double)FrameIndex * FrameRate.AsInterval());
					const int32 FrameHours = (TotalSeconds / (60 * 60));
					const int32 FrameMinutes = ((TotalSeconds % (60 * 60)) / 60);
					const int32 FrameSeconds = ((TotalSeconds % (60 * 60)) % 60);
					const bool bHoursAreValid = (FrameHours % 24) == TimespanFromTimecode.GetHours() && (FrameHours / 24) == TimespanFromTimecode.GetDays();
					const bool bMinutesAreValid = FrameMinutes == TimespanFromTimecode.GetMinutes();
					const bool bSecondsAreValid = FrameSeconds == TimespanFromTimecode.GetSeconds();
					if (!bHoursAreValid || !bMinutesAreValid || !bSecondsAreValid)
					{
						FAIL_CHECK(FString::Printf(TEXT("Timecode hours/minutes/seconds doesn't matches with Timespan '%s' from frame rate '%s'.")
							, *TimespanFromTimecode.ToString()
							, *FrameRate.ToPrettyText().ToString()
						));
						bDoTest = false;
						++NumberOfErrors;
					}
				}
			}

			// Test if the frame number is incrementing
			bool bIsPreviousTimecodeValid = FrameIndex != StartIndex;
			if (bDoTest && bIsPreviousTimecodeValid)
			{
				bool bWrongFrame = PreviousTimecodeValue.Frames + 1 != TimecodeValue.Frames
					&& TimecodeValue.Frames != 0;
				const bool bWrongSeconds = PreviousTimecodeValue.Seconds != TimecodeValue.Seconds
					&& PreviousTimecodeValue.Seconds + 1 != TimecodeValue.Seconds
					&& TimecodeValue.Seconds != 0;
				const bool bWrongMinutes = PreviousTimecodeValue.Minutes != TimecodeValue.Minutes
					&& PreviousTimecodeValue.Minutes + 1 != TimecodeValue.Minutes
					&& TimecodeValue.Minutes != 0;

				if (bWrongFrame && bIsDropFrame)
				{
					// if new minute but not multiple of 10 mins, 2|4 is expected
					const int32 NumberOfFramesInSecond = FMath::CeilToInt((float)FrameRate.AsDecimal());
					const int32 NumberOfTimecodesToDrop = NumberOfFramesInSecond <= 30 ? 2 : 4;
					bWrongFrame = !(TimecodeValue.Frames == NumberOfTimecodesToDrop && PreviousTimecodeValue.Minutes + 1 == TimecodeValue.Minutes && TimecodeValue.Minutes % 10 != 0);
				}

				if (bWrongFrame || bWrongSeconds || bWrongMinutes)
				{
					FAIL_CHECK(FString::Printf(TEXT("Timecode '%s' is not a continuity of the previous timecode '%s' from frame rate '%s'.")
						, *TimecodeValue.ToString()
						, *PreviousTimecodeValue.ToString()
						, *FrameRate.ToPrettyText().ToString()
					));
					bDoTest = false;
					++NumberOfErrors;
				}
			}

			// Test frame rate that should be equivalent
			if (bDoTest)
			{
				const FFrameRate EquivalentFrameRate = FFrameRate(FrameRate.Numerator * 3, FrameRate.Denominator * 3);
				const FTimecode EquivalentTimecodeValue = FTimecode::FromFrameNumber(FrameNumber, EquivalentFrameRate, bIsDropFrame);
				if (TimecodeValue != EquivalentTimecodeValue)
				{
					FAIL_CHECK(FString::Printf(TEXT("Timecode '%s' didn't convert properly from FrameNumber '%d' when the frame rate is tripled.")
						, *TimecodeValue.ToString()
						, FrameNumber.Value
					));
					bDoTest = false;
					++NumberOfErrors;
				}
			}

			// If we have a lot of errors with this frame rate, there is no need to log them all.
			if (NumberOfErrors > 10)
			{
				WARN(FString::Printf(TEXT("Skip test for frame rate '%s'. Other errors may exists.")
					, *FrameRate.ToPrettyText().ToString()
				));
				break;
			}

			PreviousTimecodeValue = TimecodeValue;

			// LTC timecode support up to 40 hours
			if (TimecodeValue.Hours >= 40)
			{
				break;
			}
		}

		// Conversion from current time to Timecode
		if (NumberOfErrors == 0)
		{
			const FTimespan CurrentTimespan = FTimespan(11694029893428);
			const double CurrentSeconds = 1169402.9893428; //from FPlatformTime::Seconds()

			const FTimecode FromTimespanTimecodeValueWithRollover = FTimecode::FromTimespan(CurrentTimespan, FrameRate, bIsDropFrame, true);
			const FTimecode FromTimespanTimecodeValueWithoutRollover = FTimecode::FromTimespan(CurrentTimespan, FrameRate, bIsDropFrame, false);
			const FTimecode FromSecondsTimecodeValueWithRollover = FTimecode(CurrentSeconds, FrameRate, bIsDropFrame, true);
			const FTimecode FromSecondsTimecodeValueWithoutRollover = FTimecode(CurrentSeconds, FrameRate, bIsDropFrame, false);

			if (FromTimespanTimecodeValueWithRollover != FromSecondsTimecodeValueWithRollover)
			{
				FAIL_CHECK(FString::Printf(TEXT("The timecode '%s' do not match timecode '%s' when converted from the computer clock's time and the frame rate is '%s'")
					, *FromTimespanTimecodeValueWithRollover.ToString()
					, *FromSecondsTimecodeValueWithRollover.ToString()
					, *FrameRate.ToPrettyText().ToString()
				));
				++NumberOfErrors;
			}
			else if (FromTimespanTimecodeValueWithoutRollover != FromSecondsTimecodeValueWithoutRollover)
			{
				FAIL_CHECK(FString::Printf(TEXT("The timecode '%s' do not match timecode '%s' when converted from the computer clock's time and the frame rate is '%s'")
					, *FromTimespanTimecodeValueWithoutRollover.ToString()
					, *FromSecondsTimecodeValueWithoutRollover.ToString()
					, *FrameRate.ToPrettyText().ToString()
				));
				++NumberOfErrors;
			}
			// Can't really test frame number matching between rollver timecode labels. We would need to exclude NDF fractional frame rates
		}

		INFO(FString::Printf(TEXT("Timecode test was completed with frame rate '%s'"), *FrameRate.ToPrettyText().ToString()));

		return NumberOfErrors == 0;
	};

	// Test the conversion for all common frame rate
	TArray<TFuture<bool>> Futures;
	for (const FFrameRate& FrameRate : CommonFrameRates)
	{
		Futures.Add(Async(EAsyncExecution::Thread, [FrameRate, &ConversionWithFrameRateTest](){ return ConversionWithFrameRateTest(FrameRate); }));
	}

	bool bSuccessfully = true;
	for (const TFuture<bool>& Future : Futures)
	{
		Future.Wait();
		bSuccessfully = bSuccessfully && Future.Get();
	}

	REQUIRE(bSuccessfully);
}

namespace UE::TimecodeParserTest::Private
{
	/**
	 * Bypass the FTimecode constructor to avoid the checkSlow for non conforming timecode tests
	 * i.e. we want to test parser results even if not conforming to FTimecode constructor.
	 */
	static FTimecode MakeTimecodeNoCheck(int32 InHours, int32 InMinutes, int32 InSeconds, int32 InFrames, float InSubframe, bool bInDropFrame)
	{
		FTimecode Timecode;
		Timecode.Hours = InHours;
		Timecode.Minutes = InMinutes;
		Timecode.Seconds = InSeconds;
		Timecode.Frames = InFrames;
		Timecode.Subframe = InSubframe;
		Timecode.bDropFrameFormat = bInDropFrame;
		return Timecode;
	};

	/**
	 * Formats a timecode to string for testing.
	 * Including the signed sub-frame and up to 6 decimal of precision on sub-frame fraction.
	 */
	static FString TimecodeToString(const FTimecode& InTimecode)
	{
		const bool bHasNegativeComponent = InTimecode.Hours < 0 || InTimecode.Minutes < 0 || InTimecode.Seconds < 0 || InTimecode.Frames < 0 || InTimecode.Subframe < 0.0;
		const TCHAR* SignText = bHasNegativeComponent ? TEXT("- ") : TEXT("");

		TStringBuilder<64> Builder;	// (Allow for up to 10 digits per values + sign and separator) x 5 values = 60 + terminating char.
		Builder.Appendf(TEXT("%s%02d:%02d:%02d%c%02d"), SignText,
			FMath::Abs(InTimecode.Hours), FMath::Abs(InTimecode.Minutes), FMath::Abs(InTimecode.Seconds), InTimecode.bDropFrameFormat ? TEXT(';') : TEXT(':'), FMath::Abs(InTimecode.Frames));

		if (InTimecode.Subframe != 0.0f)
		{
			// Allow for up to 6 decimals of precision.
			TStringBuilder<32> SubFrameBuilder;
			SubFrameBuilder.Appendf(TEXT("%02d"), FMath::Clamp(static_cast<int32>(1000000 * FMath::Abs(InTimecode.Subframe)),0,999999));

			// Trim trailing zeros, but leave at least 2 digits.
			const FStringView SubFrameValue = SubFrameBuilder.ToView();
			int NumDigits = SubFrameValue.Len();
			while (NumDigits > 2 && SubFrameValue[NumDigits - 1] == TEXT('0'))
			{
				NumDigits--;
			}
			Builder.Appendf(TEXT(".%.*s"), NumDigits, SubFrameValue.GetData());
		}
		return FString(Builder);
	};

	static bool IsSame(const FTimecode& InTimecode, const FTimecode& InOtherTimecode)
	{
		// Remark: FTimecode == operator ignores bDropFrameFormat.
		return InTimecode == InOtherTimecode && InTimecode.bDropFrameFormat == InOtherTimecode.bDropFrameFormat;
	}
}

TEST_CASE_NAMED(FTimecodeParserTest, "System::Core::Misc::TimecodeParser", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::TimecodeParserTest::Private;
	
	struct FTimecodeParserTestEntry
	{
		FString TimecodeString;
		FTimecode ExpectedTimecode;
	};

	constexpr bool bDropFrame = true;	// a.k.a DF
	constexpr bool bStandard = false;	// a.k.a NDF

	TArray<FTimecodeParserTestEntry> TimecodeParseSuccessTests =
	{
		{ TEXT("00:00:00:00"), FTimecode(0,0,0,0,bStandard)},
		{ TEXT("00:00:00;00"), FTimecode(0,0,0,0,bDropFrame)},
		{ TEXT("00:00:00:00.00"), FTimecode(0,0,0,0, 0.0f, bStandard)},
		{ TEXT("00:00:00;00.00"), FTimecode(0,0,0,0, 0.0f, bDropFrame)},
		{ TEXT("10:20:30:40.50"), FTimecode(10,20,30,40, 0.5f, bStandard)},
		{ TEXT("15:55:22;09.90"), FTimecode(15,55,22,9, 0.9f, bDropFrame)},
		{ TEXT("10: 11: 12; 191"), FTimecode(10,11,12,191, bDropFrame)},
		// drop frame variations separator
		{ TEXT("01;02;03;04"), FTimecode(1,2,3,4,bDropFrame)},
		{ TEXT("01:02;03;04"), FTimecode(1,2,3,4,bDropFrame)},
		{ TEXT("01;02;03;04.50"), FTimecode(1,2,3,4, 0.5f, bDropFrame)},
		// drop frame variations with '.' separator
		{ TEXT("01:02:03.04"), FTimecode(1,2,3,4,bDropFrame)},
		{ TEXT("01.02.03.04"), FTimecode(1,2,3,4,bDropFrame)},
		{ TEXT("01:02.03.04"), FTimecode(1,2,3,4,bDropFrame)},
		{ TEXT("01.02.03.04.50"), FTimecode(1,2,3,4, 0.5f, bDropFrame)},
		// higher precision sub-frame
		{ TEXT("01:02:03:04.777"), FTimecode(1,2,3,4, 0.777f, bStandard)},
		{ TEXT("01.02.03.04.555"), FTimecode(1,2,3,4, 0.555f, bDropFrame)},
		// sign tests
		{ TEXT("+ 15:55:22;09.90"), FTimecode(15,55,22,9, 0.9f, bDropFrame)},
		{ TEXT("+15:55:22;09.90"), FTimecode(15,55,22,9, 0.9f, bDropFrame)},
		{ TEXT("- 15:55:22;09.90"), FTimecode(-15,55,22,9, 0.9f, bDropFrame)},
		{ TEXT("-15:55:22;09.90"), FTimecode(-15,55,22,9, 0.9f, bDropFrame)},
		{ TEXT("15:-55:22;09.90"), FTimecode(15,-55,22,9, 0.9f, bDropFrame)},
		{ TEXT("15:55:-22;09.90"), FTimecode(15,55,-22,9, 0.9f, bDropFrame)},
		{ TEXT("15:55:22;-09.90"), FTimecode(15,55,22,-9, 0.9f, bDropFrame)},
		// sign tests - sign gets applied to first non-zero value.
		{ TEXT("- 00:55:22:09"), FTimecode(0,-55,22,9, bStandard)},
		{ TEXT("- 00:00:22:09"), FTimecode(0,0,-22,9, bStandard)},
		{ TEXT("- 00:00:00:09"), FTimecode(0,0,0,-9, bStandard)},
		// sign tests - negative on the sub-frame. The sub-frame is negative only if there is no other way to preserve sign.
		// -- Note: this would only happen if manually entered, but we still want the parser to do something with it.
		{ TEXT("15:55:22;09.-90"), FTimecode(-15,55,22,9, 0.9f, bDropFrame)},
		{ TEXT("00:55:22;09.-90"), FTimecode(0,-55,22,9, 0.9f, bDropFrame)},
		{ TEXT("00:00:22;09.-90"), FTimecode(0,0,-22,9, 0.9f, bDropFrame)},
		{ TEXT("00:00:00;09.-90"), FTimecode(0,0,0,-9, 0.9f, bDropFrame)},
		{ TEXT("00:00:00;00.-90"), MakeTimecodeNoCheck(0,0,0,0, -0.9f, bDropFrame)},
		{ TEXT("- 00:00:00:00.90"), MakeTimecodeNoCheck(0,0,0,0, -0.9f, bStandard)},
		// High frame number (ex for audio timecodes) supported.
		{ TEXT("15:55:22:43999"), FTimecode(15,55,22,43999, bStandard)},
		// Values out of normal range
		{ TEXT("200:210:220:44000"), MakeTimecodeNoCheck(200,210,220,44000, 0.0f, bStandard)}
	};

	TArray<FTimecodeParserTestEntry> TimecodeParseFailureTests =
	{
		{ TEXT(""), FTimecode()},
		{ TEXT("00:00"), FTimecode()},			// not enough values
		{ TEXT("01:02:1d:25"), FTimecode()},	// value is not a valid 'base10' number
		{ TEXT("00.00:00:00"), FTimecode()},	// wrong separator
		{ TEXT(":00:00:00:00"), FTimecode()},	// doesn't begin with a number
		{ TEXT("...0"), FTimecode()},			// ""
		{ TEXT("00;00:00:00"), FTimecode()},	// drop frame separator at the wrong place (ambiguous)
		{ TEXT("00:00:00:00:00"), FTimecode()},	// wrong subframe separator
	};

	auto ExecuteTimecodeTests = [](const TArray<FTimecodeParserTestEntry>& InTests, bool bInExpectedSuccess)
	{
		for (const FTimecodeParserTestEntry& Test : InTests)
		{
			TOptional<FTimecode> TimecodeOptional = FTimecode::ParseTimecode(*Test.TimecodeString);
			if (TimecodeOptional.IsSet())
			{
				const FTimecode& Timecode = TimecodeOptional.GetValue();
				if (bInExpectedSuccess)
				{
					if (IsSame(Timecode, Test.ExpectedTimecode))
					{
						INFO(FString::Printf(TEXT("Parsing '%s' to parsed timecode '%s' -> OK"), *Test.TimecodeString, *TimecodeToString(Timecode)));
					}
					else
					{
						FAIL_CHECK(FString::Printf(TEXT("Parsing '%s' (parsed timecode '%s') was different than expected: '%s'"),
							*Test.TimecodeString, *TimecodeToString(Timecode), *TimecodeToString(Test.ExpectedTimecode)));
					}
				}
				else
				{
					FAIL_CHECK(FString::Printf(TEXT("Parsing '%s' was expected to failed, but it succeeded (parsed timecode '%s')"), *Test.TimecodeString, *TimecodeToString(Timecode)));
				}
			}
			else
			{
				if (bInExpectedSuccess)
				{
					FAIL_CHECK(FString::Printf(TEXT("Parsing '%s' failed, but was expected to succeed (Expected: '%s'."), *Test.TimecodeString, *TimecodeToString(Test.ExpectedTimecode)));
				}
				else
				{
					INFO(FString::Printf(TEXT("Parsing '%s' failed as expected."), *Test.TimecodeString));
				}
			}
		}
	};

	ExecuteTimecodeTests(TimecodeParseSuccessTests, true /*bInExpectedSuccess*/);
	ExecuteTimecodeTests(TimecodeParseFailureTests, false /*bInExpectedSuccess*/);
}

#endif //WITH_TESTS
