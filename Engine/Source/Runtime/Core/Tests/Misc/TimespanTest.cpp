// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Misc/Timespan.h"

#include "Containers/UnrealString.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FTimespanTest, "System::Core::Time::Timespan", "[Core][Time][SmokeFilter]")
{
	SECTION("Constructors")
	{
		FTimespan ts1_1 = FTimespan(3, 2, 1);
		FTimespan ts1_2 = FTimespan(0, 3, 2, 1);
		FTimespan ts1_3 = FTimespan(0, 3, 2, 1, 0);

		CHECK(ts1_1 == ts1_2);
		CHECK(ts1_1 == ts1_3);
	}

	SECTION("Components")
	{
		FTimespan ts2_1 = FTimespan(1, 2, 3, 4, 123456789);

		CHECK(ts2_1.GetDays() == 1);
		CHECK(ts2_1.GetHours() == 2);
		CHECK(ts2_1.GetMinutes() == 3);
		CHECK(ts2_1.GetSeconds() == 4);
		CHECK(ts2_1.GetFractionMilli() == 123);
		CHECK(ts2_1.GetFractionMicro() == 123456);
		CHECK(ts2_1.GetFractionNano() == 123456700);
	}

	SECTION("Duration")
	{
		FTimespan ts3_1 = FTimespan(1, 2, 3, 4, 123456789);
		FTimespan ts3_2 = FTimespan(-1, -2, -3, -4, -123456789);

		CHECK(ts3_1.GetDuration() == ts3_2.GetDuration());
	}

	SECTION("FromXxx to GetTotalXxx")
	{
		CHECK(FTimespan::FromDays(123).GetTotalDays() == 123.0);
		CHECK(FTimespan::FromHours(123).GetTotalHours() == 123.0);
		CHECK(FTimespan::FromMinutes(123).GetTotalMinutes() == 123.0);
		CHECK(FTimespan::FromSeconds(123).GetTotalSeconds() == 123.0);
		CHECK(FTimespan::FromMilliseconds(123).GetTotalMilliseconds() == 123.0);
		CHECK(FTimespan::FromMicroseconds(123).GetTotalMicroseconds() == 123.0);
	}

	SECTION("ToString")
	{
		FTimespan ts5_1 = FTimespan(1, 2, 3, 4, 123456789);

		CHECK(ts5_1.ToString() == TEXT("+1.02:03:04.123"));
		CHECK(ts5_1.ToString(TEXT("%d.%h:%m:%s.%f")) == TEXT("+1.02:03:04.123"));
		CHECK(ts5_1.ToString(TEXT("%d.%h:%m:%s.%u")) == TEXT("+1.02:03:04.123456"));
		CHECK(ts5_1.ToString(TEXT("%D.%h:%m:%s.%n")) == TEXT("+00000001.02:03:04.123456700"));
	}

	SECTION("Parse Valid")
	{
		FTimespan ts6_t;

		FTimespan ts6_1 = FTimespan(1, 2, 3, 4, 123000000);
		FTimespan ts6_2 = FTimespan(1, 2, 3, 4, 123456000);
		FTimespan ts6_3 = FTimespan(1, 2, 3, 4, 123456700);

		CHECK(FTimespan::Parse(TEXT("+1.02:03:04.123"), ts6_t));
		CHECK(ts6_t == ts6_1);

		CHECK(FTimespan::Parse(TEXT("+1.02:03:04.123456"), ts6_t));
		CHECK(ts6_t == ts6_2);

		CHECK(FTimespan::Parse(TEXT("+1.02:03:04.123456789"), ts6_t));
		CHECK(ts6_t == ts6_3);

		FTimespan ts6_4 = FTimespan(-1, -2, -3, -4, -123000000);
		FTimespan ts6_5 = FTimespan(-1, -2, -3, -4, -123456000);
		FTimespan ts6_6 = FTimespan(-1, -2, -3, -4, -123456700);

		CHECK(FTimespan::Parse(TEXT("-1.02:03:04.123"), ts6_t));
		CHECK(ts6_t == ts6_4);

		CHECK(FTimespan::Parse(TEXT("-1.02:03:04.123456"), ts6_t));
		CHECK(ts6_t == ts6_5);

		CHECK(FTimespan::Parse(TEXT("-1.02:03:04.123456789"), ts6_t));
		CHECK(ts6_t == ts6_6);
	}

	SECTION("Parse Invalid")
	{
		FTimespan ts7_1;

		//CHECK_FALSE(FTimespan::Parse(TEXT("1,02:03:04.005"), ts7_1));
		//CHECK_FALSE(FTimespan::Parse(TEXT("1.1.02:03:04:005"), ts7_1));
		//CHECK_FALSE(FTimespan::Parse(TEXT("04:005"), ts7_1));
	}

	// `From*` converters must return correct value
	// test normal and edge cases for polar conversions (FromMicroseconds() and FromDay()) and just normal case for all others
	SECTION("FromXxx Exact Ticks")
	{
		CHECK(FTimespan::FromMicroseconds(0) == FTimespan(0));
		CHECK(FTimespan::FromMicroseconds(1) == FTimespan(1 * ETimespan::TicksPerMicrosecond));
		CHECK(FTimespan::FromMicroseconds(1.1) == FTimespan(1 * ETimespan::TicksPerMicrosecond + 1));
		CHECK(FTimespan::FromMicroseconds(1.5) == FTimespan(1 * ETimespan::TicksPerMicrosecond + 5));
		CHECK(FTimespan::FromMicroseconds(1.499999999999997) == FTimespan(1 * ETimespan::TicksPerMicrosecond + 5));
		CHECK(FTimespan::FromMicroseconds(1.50000001) == FTimespan(1 * ETimespan::TicksPerMicrosecond + 5));
		CHECK(FTimespan::FromMicroseconds(-1) == FTimespan(-1 * ETimespan::TicksPerMicrosecond));
		CHECK(FTimespan::FromMicroseconds(-1.1) == FTimespan(-1 * ETimespan::TicksPerMicrosecond - 1));
		CHECK(FTimespan::FromMicroseconds(-1.5) == FTimespan(-1 * ETimespan::TicksPerMicrosecond - 5));

		CHECK(FTimespan::FromMilliseconds(1) == FTimespan(1 * ETimespan::TicksPerMillisecond));
		CHECK(FTimespan::FromSeconds(1) == FTimespan(1 * ETimespan::TicksPerSecond));
		CHECK(FTimespan::FromMinutes(1) == FTimespan(1 * ETimespan::TicksPerMinute));
		CHECK(FTimespan::FromHours(1) == FTimespan(1 * ETimespan::TicksPerHour));

		CHECK(FTimespan::FromDays(0) == FTimespan(0));
		CHECK(FTimespan::FromDays(1) == FTimespan(1 * ETimespan::TicksPerDay));
		CHECK(FTimespan::FromDays(1.25) == FTimespan(1 * ETimespan::TicksPerDay + 6 * ETimespan::TicksPerHour));
		CHECK(FTimespan::FromDays(1.5) == FTimespan(1 * ETimespan::TicksPerDay + 12 * ETimespan::TicksPerHour));
		CHECK(FTimespan::FromDays(1.499999999999997) == FTimespan(1 * ETimespan::TicksPerDay + 12 * ETimespan::TicksPerHour));
		CHECK(FTimespan::FromDays(-1) == FTimespan(-1 * ETimespan::TicksPerDay));
		CHECK(FTimespan::FromDays(-1.25) == FTimespan(-1 * ETimespan::TicksPerDay - 6 * ETimespan::TicksPerHour));
		CHECK(FTimespan::FromDays(-1.5) == FTimespan(-1 * ETimespan::TicksPerDay - 12 * ETimespan::TicksPerHour));
	}
}

#endif //WITH_TESTS
