// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Tests/TestHarnessAdapter.h"
#include "Containers/TripleBuffer.h"
#include "Math/RandomStream.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(FTripleBufferTest, "System::Core::Misc::TripleBuffer", "[ApplicationContextMask][SmokeFilter]")
{
	// uninitialized buffer
	{
		TTripleBuffer<int32> Buffer(NoInit);

		CHECK_FALSE_MESSAGE(TEXT("Uninitialized triple buffer must not be dirty"), Buffer.IsDirty());
	}

	// initialized buffer
	{
		TTripleBuffer<int32> Buffer(1);

		CHECK_FALSE_MESSAGE(TEXT("Initialized triple buffer must not be dirty"), Buffer.IsDirty());
		CHECK_EQUALS(TEXT("Initialized triple buffer must have correct read buffer value"), Buffer.Read(), 1);

		Buffer.SwapReadBuffers();

		CHECK_EQUALS(TEXT("Initialized triple buffer must have correct temp buffer value"), Buffer.Read(), 1);

		Buffer.SwapWriteBuffers();

		CHECK_MESSAGE(TEXT("Write buffer swap must set dirty flag"), Buffer.IsDirty());

		Buffer.SwapReadBuffers();

		CHECK_FALSE_MESSAGE(TEXT("Read buffer swap must clear dirty flag"), Buffer.IsDirty());
		CHECK_EQUALS(TEXT("Initialized triple buffer must have correct temp buffer value"), Buffer.Read(), 1);
	}

	// pre-set buffer
	{
		int32 Array[3] = { 1, 2, 3 };
		TTripleBuffer<int32> Buffer(Array);

		int32 Read = Buffer.Read();
		CHECK_EQUALS(TEXT("Pre-set triple buffer must have correct Read buffer value"), Read, 3);

		Buffer.SwapReadBuffers();

		int32 Temp = Buffer.Read();
		CHECK_EQUALS(TEXT("Pre-set triple buffer must have correct Temp buffer value"), Temp, 1);

		Buffer.SwapWriteBuffers();
		Buffer.SwapReadBuffers();

		int32 Write = Buffer.Read();
		CHECK_EQUALS(TEXT("Pre-set triple buffer must have correct Write buffer value"), Write, 2);
	}

	// operations
	{
		TTripleBuffer<int32> Buffer;

		for (int32 Index = 0; Index < 6; ++Index)
		{
			int32& Write = Buffer.GetWriteBuffer(); Write = Index; Buffer.SwapWriteBuffers();
			Buffer.SwapReadBuffers();
			CHECK_EQUALS(*FString::Printf(TEXT("Triple buffer must read correct value (%i)"), Index), Buffer.Read(), Index);
		}

		FRandomStream Rand;
		int32 LastRead = -1;

		for (int32 Index = 0; Index < 100; ++Index)
		{
			int32 Writes = Rand.GetUnsignedInt() % 4;

			while (Writes > 0)
			{
				int32& Write = Buffer.GetWriteBuffer(); Write = Index; Buffer.SwapWriteBuffers();
				--Writes;
			}
			
			int32 Reads = Rand.GetUnsignedInt() % 4;

			while (Reads > 0)
			{
				if (!Buffer.IsDirty())
				{
					break;
				}

				Buffer.SwapReadBuffers();
				int32 Read = Buffer.Read();
				CHECK_MESSAGE(TEXT("Triple buffer must read in increasing order"), Read > LastRead);
				LastRead = Read;
				--Reads;
			}
		}
	}
}

#endif //WITH_TESTS