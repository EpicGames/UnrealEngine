// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreTypes.h"
#include "Templates/UnrealTypeTraits.h"
#include "Containers/UnrealString.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Tests/TestHarnessAdapter.h"
#include "Async/Async.h"
#include "Misc/TypeContainer.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

/* Helpers
 *****************************************************************************/

struct IFruit
{
	virtual ~IFruit() { }
	virtual FString Name() = 0;
};

struct IBerry : public IFruit
{
	virtual ~IBerry() { }
};

struct FBanana : public IFruit
{
	virtual ~FBanana() { }
	virtual FString Name() override { return TEXT("Banana"); }
};

struct FStrawberry : public IBerry
{
	virtual ~FStrawberry() { }
	virtual FString Name() override { return TEXT("Strawberry"); }
};

template<ESPMode Mode = ESPMode::ThreadSafe>
struct ISmoothie
{
	virtual ~ISmoothie() { }
	virtual TSharedRef<IBerry, Mode> GetBerry() = 0;
	virtual TSharedRef<IFruit, Mode> GetFruit() = 0;
};

template<ESPMode Mode = ESPMode::ThreadSafe>
struct TSmoothie : public ISmoothie<Mode>
{
	TSmoothie(TSharedRef<IFruit, Mode> InFruit, TSharedRef<IBerry, Mode> InBerry) : Berry(InBerry), Fruit(InFruit) { }
	virtual ~TSmoothie() { }
	virtual TSharedRef<IBerry, Mode> GetBerry() override { return Berry; }
	virtual TSharedRef<IFruit, Mode> GetFruit() override { return Fruit; }
	TSharedRef<IBerry, Mode> Berry;
	TSharedRef<IFruit, Mode> Fruit;
};

template<ESPMode Mode = ESPMode::ThreadSafe>
struct TTwoSmoothies
{
	TSharedPtr<ISmoothie<Mode>, Mode> One;
	TSharedPtr<ISmoothie<Mode>, Mode> Two;
};


DECLARE_DELEGATE_RetVal(TSharedRef<IBerry>, FBerryFactoryDelegate);
DECLARE_DELEGATE_RetVal(TSharedRef<IFruit>, FFruitFactoryDelegate);
DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<ISmoothie<>>, FSmoothieFactoryDelegate, TSharedRef<IFruit>, TSharedRef<IBerry>);

Expose_TNameOf(IBerry)
Expose_TNameOf(IFruit)
Expose_TNameOf(ISmoothie<>)


/* Tests
 *****************************************************************************/

TEST_CASE_NAMED(FTypeContainerTest, "System::Core::Misc::TypeContainer", "[ApplicationContextMask][EngineFilter]")
{
	// existing instance test
	{
		TSharedRef<IFruit> Fruit = MakeShareable(new FBanana());
		TSharedRef<IBerry> Berry = MakeShareable(new FStrawberry());

		TTypeContainer<> Container;
		{
			Container.RegisterInstance<IFruit>(Fruit);
			Container.RegisterInstance<IBerry>(Berry);
		}

		auto Instance = Container.GetInstance<IFruit>();

		CHECK_EQUALS(TEXT("Correct instance must be returned"), Instance, Fruit);
		CHECK_NOT_EQUALS(TEXT("Incorrect instance must not be returned"), Instance, StaticCastSharedRef<IFruit>(Berry));
	}

	// per instance test
	{
		TTypeContainer<> Container;
		{
			Container.RegisterClass<IFruit, FBanana>(ETypeContainerScope::Instance);
			Container.RegisterClass<IBerry, FStrawberry>(ETypeContainerScope::Instance);
			Container.RegisterClass<ISmoothie<>, TSmoothie<>, IFruit, IBerry>(ETypeContainerScope::Instance); // !!!
		}

		auto Smoothie1 = Container.GetInstance<ISmoothie<>>();
		auto Smoothie2 = Container.GetInstance<ISmoothie<>>();

		CHECK_NOT_EQUALS(TEXT("For per-instances classes, a unique instance must be returned each time"), Smoothie1, Smoothie2);
		CHECK_NOT_EQUALS(TEXT("For per-instances dependencies, a unique instance must be returned each time [1]"), Smoothie1->GetBerry(), Smoothie2->GetBerry());
		CHECK_NOT_EQUALS(TEXT("For per-instances dependencies, a unique instance must be returned each time [2]"), Smoothie1->GetFruit(), Smoothie2->GetFruit());
	}

	// per thread test
	{
		TTypeContainer<ESPMode::ThreadSafe> Container;
		{
			Container.RegisterClass<IFruit, FBanana>(ETypeContainerScope::Thread);
			Container.RegisterClass<IBerry, FStrawberry>(ETypeContainerScope::Instance);
			Container.RegisterClass<ISmoothie<ESPMode::ThreadSafe>, TSmoothie<ESPMode::ThreadSafe>, IFruit, IBerry>(ETypeContainerScope::Thread); // !!!
		}

		TFunction<TTwoSmoothies<ESPMode::ThreadSafe>()> MakeSmoothies = [&]()
		{
			TTwoSmoothies<ESPMode::ThreadSafe> Smoothies;
			Smoothies.One = Container.GetInstance<ISmoothie<ESPMode::ThreadSafe>>();
			Smoothies.Two = Container.GetInstance<ISmoothie<ESPMode::ThreadSafe>>();
			return Smoothies;
		};

		auto Smoothies1 = Async(EAsyncExecution::Thread, MakeSmoothies);
		auto Smoothies2 = Async(EAsyncExecution::Thread, MakeSmoothies);

		auto One1 = Smoothies1.Get().One;
		auto Two1 = Smoothies1.Get().Two;
		auto One2 = Smoothies2.Get().One;
		auto Two2 = Smoothies2.Get().Two;

		CHECK_EQUALS(TEXT("For per-thread classes, the same instance must be returned from the same thread [1]"), One1, Two1);
		CHECK_EQUALS(TEXT("For per-thread classes, the same instance must be returned from the same thread [2]"), One2, Two2);
		CHECK_NOT_EQUALS(TEXT("For per-thread classes, different instances must be returned from different threads [1]"), One1, One2);
		CHECK_NOT_EQUALS(TEXT("For per-thread classes, different instances must be returned from different threads [2]"), Two1, Two2);
	}

	// per process test
	{
		TTypeContainer<ESPMode::ThreadSafe> Container;
		{
			Container.RegisterClass<IFruit, FBanana>(ETypeContainerScope::Thread);
			Container.RegisterClass<IBerry, FStrawberry>(ETypeContainerScope::Instance);
			Container.RegisterClass<ISmoothie<ESPMode::ThreadSafe>, TSmoothie<ESPMode::ThreadSafe>, IFruit, IBerry>(ETypeContainerScope::Process); //!!!
		}

		TFunction<TTwoSmoothies<ESPMode::ThreadSafe>()> MakeSmoothies = [&]()
		{
			TTwoSmoothies<ESPMode::ThreadSafe> Smoothies;
			Smoothies.One = Container.GetInstance<ISmoothie<ESPMode::ThreadSafe>>();
			Smoothies.Two = Container.GetInstance<ISmoothie<ESPMode::ThreadSafe>>();
			return Smoothies;
		};

		auto Smoothies1 = Async(EAsyncExecution::Thread, MakeSmoothies);
		auto Smoothies2 = Async(EAsyncExecution::Thread, MakeSmoothies);

		auto One1 = Smoothies1.Get().One;
		auto Two1 = Smoothies1.Get().Two;
		auto One2 = Smoothies2.Get().One;
		auto Two2 = Smoothies2.Get().Two;

		CHECK_EQUALS(TEXT("For per-process classes, the same instance must be returned from the same thread [1]"), One1, Two1);
		CHECK_EQUALS(TEXT("For per-process classes, the same instance must be returned from the same thread [2]"), One2, Two2);
		CHECK_EQUALS(TEXT("For per-process classes, the same instance must be returned from different threads [1]"), One1, One2);
		CHECK_EQUALS(TEXT("For per-process classes, the same instance must be returned from different threads [1]"), Two1, Two2);
	}

	// factory test
	{
		struct FLocal
		{
			static TSharedRef<IBerry> MakeStrawberry()
			{
				return MakeShareable(new FStrawberry());
			}

			static TSharedRef<ISmoothie<>> MakeSmoothie(TSharedRef<IFruit> Fruit, TSharedRef<IBerry> Berry)
			{
				return MakeShareable(new TSmoothie<>(Fruit, Berry));
			}
		};

		TTypeContainer<> Container;
		{
			Container.RegisterFactory<IBerry>(&FLocal::MakeStrawberry);
			Container.RegisterFactory<IFruit>(
				[]() -> TSharedRef<IFruit> {
					return MakeShareable(new FBanana());
				}
			);
			Container.RegisterFactory<ISmoothie<>, IFruit, IBerry>(&FLocal::MakeSmoothie);
		}

		auto Berry = Container.GetInstance<IBerry>();
		auto Fruit = Container.GetInstance<IFruit>();
		auto Smoothie = Container.GetInstance<ISmoothie<>>();
	}

	// delegate test
	{
		struct FLocal
		{
			static TSharedRef<IBerry> MakeBerry()
			{
				return MakeShareable(new FStrawberry());
			}

			static TSharedRef<IFruit> MakeFruit(bool Banana)
			{
				if (Banana)
				{
					return MakeShareable(new FBanana());
				}

				return MakeShareable(new FStrawberry());
			}

			static TSharedRef<ISmoothie<>> MakeSmoothie(TSharedRef<IFruit> Fruit, TSharedRef<IBerry> Berry)
			{
				return MakeShareable(new TSmoothie<>(Fruit, Berry));
			}
		};

		TTypeContainer<> Container;
		{
			Container.RegisterDelegate<IBerry>(FBerryFactoryDelegate::CreateStatic(&FLocal::MakeBerry));
			Container.RegisterDelegate<IFruit>(FFruitFactoryDelegate::CreateStatic(&FLocal::MakeFruit, true));
			Container.RegisterDelegate<ISmoothie<>, FSmoothieFactoryDelegate, IFruit, IBerry>(FSmoothieFactoryDelegate::CreateStatic(&FLocal::MakeSmoothie));
		}

		auto Fruit = Container.GetInstance<IFruit>();
		auto Smoothie = Container.GetInstance<ISmoothie<>>();
	}

}

#endif //WITH_TESTS
