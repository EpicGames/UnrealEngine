// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Misc/URLRequestFilter.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeExit.h"
#include "Containers/StringView.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"

#include "Tests/TestHarnessAdapter.h"

namespace UE::Core::Private
{

// Adds a config file to GConfig while object is in scope, for ease of testing with ini settings.
// Doesn't add the file if one already exists with the same name, to avoid overwriting it.
class FScopedConfigFile
{
public:
	FScopedConfigFile(const TCHAR* BaseName, const FString& Contents)
	{
		Filename = GConfig->GetConfigFilename(BaseName);
		FConfigFile* Existingfile = GConfig->FindConfigFile(Filename);

		// If the file already exists don't overwrite it
		if (Existingfile != nullptr)
		{
			bWasAdded = false;
		}
		else
		{
			FConfigFile TestConfigFile;
			TestConfigFile.CombineFromBuffer(Contents, Filename);
			GConfig->Add(Filename, TestConfigFile);
			bWasAdded = true;
		}
	}

	~FScopedConfigFile()
	{
		if (bWasAdded)
		{
			GConfig->Remove(Filename);
		}
	}

	// Could be moveable, but not needed for these tests
	FScopedConfigFile(const FScopedConfigFile&) = delete;
	FScopedConfigFile(FScopedConfigFile&&) = delete;

	FScopedConfigFile& operator==(const FScopedConfigFile&) = delete;
	FScopedConfigFile& operator==(FScopedConfigFile&&) = delete;

	FString GetFilename() const { return Filename; }
	bool WasAdded() const { return bWasAdded; }

private:
	FString Filename;
	bool bWasAdded = false;
};


TEST_CASE_NAMED(FMiscURLRequestFilterTest, "System::Core::Misc::URLRequestFilter", "[Core]")
{
	SECTION("Empty filter")
	{
		const FURLRequestFilter Filter{FURLRequestFilter::FRequestMap()};

		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://subdomain.domain.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://subdomain.domain.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("file://C:/somefile.txt")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1:7777")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1:7777/path/to/somewhere")));
	}

	SECTION("Empty scheme")
	{
		FURLRequestFilter::FRequestMap SchemeMap;
		SchemeMap.Add(TEXT(""), TArray<FString>{});

		FURLRequestFilter Filter{SchemeMap};

		// Reject everything
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("://")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW(":///")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https:///")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("://epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://subdomain.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://subdomain.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("file://C:/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1:7777")));
	}

	SECTION("Empty domain")
	{
		FURLRequestFilter::FRequestMap SchemeMap;
		SchemeMap.Add(TEXT("https"), TArray<FString>{TEXT("")});

		FURLRequestFilter Filter{SchemeMap};

		// Reject everything
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("://")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW(":///")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https:///")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("://epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://subdomain.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://subdomain.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("httpsepicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgameshttps")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://subdomain.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://subdomain.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("file://C:/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1:7777")));
	}

	SECTION("Single scheme filter")
	{
		FURLRequestFilter::FRequestMap SchemeMap;
		SchemeMap.Add(TEXT("https"), TArray<FString>{});

		FURLRequestFilter Filter{SchemeMap};

		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://subdomain.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://subdomain.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("httpsepicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgameshttps")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://subdomain.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://subdomain.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("file://C:/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1:7777")));

		// Various Windows file paths

		// Drive absolute
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:/https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:\\https\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:/https/../somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:\\https\\..\\somefile.txt")));

		// Drive relative
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:https\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:https/../somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("C:https\\..\\somefile.txt")));

		// Rooted
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("/https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\https\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("/https/../somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\https\\..\\somefile.txt")));

		// Relative
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("../https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("..\\https\\somefile.txt")));
		
		// UNC aboslute
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//someserver/share/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\someserver\\share\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//https/share/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\https\\share\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//https/https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\https\\https\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//someserver/https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\someserver\\https\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//someserver/https/../somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\someserver\\https\\..\\somefile.txt")));

		// Local device
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//./share/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\.\\share\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//./C:/https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\.\\C:\\https\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//./https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\.\\https\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//./https/../somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\.\\https\\..\\somefile.txt")));

		// Root local device
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//?/share/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\?\\share\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//?/C:/https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\?\\C:\\https\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//?/https/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\?\\https\\somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("//?/https/../somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("\\\\?\\https\\..\\somefile.txt")));
	}

	SECTION("Case-insensitive scheme only")
	{
		FURLRequestFilter::FRequestMap SchemeMap;
		SchemeMap.Add(TEXT("https"), TArray<FString>{});
		SchemeMap.Add(TEXT("HtTpS"), TArray<FString>{});
		SchemeMap.Add(TEXT("HTTPS"), TArray<FString>{});

		FURLRequestFilter Filter{SchemeMap};

		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://subdomain.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("HTTPS://epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("HTTPS://subdomain.epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("hTtPs://epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("hTtPs://subdomain.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("HTTPSepicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("hTtPsepicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgamesHTTPS")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgameshTtPs")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://subdomain.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("file://C:/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1")));
	}

	SECTION("Case-insensitive scheme with domains")
	{
		FURLRequestFilter::FRequestMap SchemeMap;
		SchemeMap.Add(TEXT("https"), TArray<FString>{TEXT("test1.epicgames.com")});
		// Case-insensitive FString key overwrites previous entry
		SchemeMap.Add(TEXT("HtTpS"), TArray<FString>{TEXT("test2.epicgames.com")});

		FURLRequestFilter Filter{SchemeMap};

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://test1.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://test2.epicgames.com")));

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("hTtPs://test1.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("hTtPs://test2.epicgames.com")));

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("HTTPS://test1.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("HTTPS://test2.epicgames.com")));
		
	}

	SECTION("Multiple scheme filter")
	{
		FURLRequestFilter::FRequestMap SchemeMap;
		SchemeMap.Add(TEXT("https"), TArray<FString>{});
		SchemeMap.Add(TEXT("http"), TArray<FString>{});

		FURLRequestFilter Filter{SchemeMap};

		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://subdomain.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://subdomain.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("httpsepicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgameshttps")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("httpepicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgameshttp")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://subdomain.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://subdomain.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("file://C:/somefile.txt")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1/path/to/somewhere")));
	}

	SECTION("Absolute domains")
	{
		FURLRequestFilter::FRequestMap SchemeMap;
		SchemeMap.Add(TEXT("https"), TArray<FString>{TEXT("epicgames.com"), TEXT("unrealengine.com")});
		SchemeMap.Add(TEXT("http"), TArray<FString>{TEXT("test1.epicgames.com"), TEXT("test2.epicgames.com")});

		FURLRequestFilter Filter{SchemeMap};

		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://unrealengine.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://test1.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://test3.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("httpsepicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgameshttps")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("httptest1.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("test2.epicgames.com.https")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unrealengine.com/path/to/somewhere")));

		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://test1.epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://test2.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://unrealengine.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("test1.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("test2.epicgames.com/path/to/somewhere")));

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://unrealengine.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://unrealengine.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://test1.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://test2.epicgames.com/path/to/somewhere")));
	}

	SECTION("Subdomains")
	{
		FURLRequestFilter::FRequestMap SchemeMap;
		SchemeMap.Add(TEXT("https"), TArray<FString>{TEXT(".epicgames.com"), TEXT(".unrealengine.com")});
		SchemeMap.Add(TEXT("http"), TArray<FString>{TEXT(".test1.epicgames.com"), TEXT(".test2.epicgames.com")});

		FURLRequestFilter Filter{SchemeMap};

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://unrealengine.com/path/to/somewhere")));

		// Technically allowed by the URI grammar in RFC 3986
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://.sub.epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://.unrealengine.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://.sub.unrealengine.com")));

		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://test1.epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://test2.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://test1.unrealengine.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://test2.unrealengine.com")));
		
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://sub.test1.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://sub.test2.epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://sub.test1.unrealengine.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://sub.test2.unrealengine.com/path/to/somewhere")));

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgameshttps")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http.test1.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW(".test2.epicgames.com.http")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unrealengine.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW(".epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW(".unrealengine.com")));

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://test1.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("http://test2.epicgames.com/path/to/somewhere")));

		// Technically allowed by the URI grammar in RFC 3986
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://.test1.epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://.sub.test1.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://.test2.epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://.sub.test2.epicgames.com")));

		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://sub1.test1.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://sub2.test1.epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://sub1.test2.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://sub2.test2.epicgames.com/path/to/somewhere")));
		
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://sub1.sub2.test1.epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://sub3.sub4.test1.epicgames.com")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://sub1.sub2.test2.epicgames.com/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("http://sub3.sub4.test2.epicgames.com")));

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("unrealengine.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW(".epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW(".unrealengine.com/path/to/somewhere")));

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://.unrealengine.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://.test1.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://.test2.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://sub1.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://sub1.unrealengine.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://sub1.test1.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://sub1.test2.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://sub1.sub2.epicgames.com/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://sub1.sub2.unrealengine.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://sub1.sub2.test1.epicgames.com")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("wss://sub1.sub2.test2.epicgames.com/path/to/somewhere")));
	}

	SECTION("IP addresses")
	{
		FURLRequestFilter::FRequestMap SchemeMap;
		SchemeMap.Add(TEXT("https"), TArray<FString>{TEXT("127.0.0.1"), TEXT("[::1]")});

		FURLRequestFilter Filter{SchemeMap};

		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://127.0.0.1")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://127.0.0.1/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://127.0.0.1:443")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://127.0.0.1:443/path/to/somewhere")));

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://255.255.255.0")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://255.255.255.0/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://255.255.255.0:443")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://255.255.255.0:443/path/to/somewhere")));

		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://[::1]")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://[::1]/path/to/somewhere")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://[::1]:443")));
		CHECK(Filter.IsRequestAllowed(TEXTVIEW("https://[::1]:443/path/to/somewhere")));

		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://[2001:db8::]")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://[2001:db8::]/path/to/somewhere")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://[2001:db8::]:443")));
		CHECK(!Filter.IsRequestAllowed(TEXTVIEW("https://[2001:db8::]:443/path/to/somewhere")));

	}

	SECTION("Config initialization")
	{
		const FString TestConfigFilename = "TestConfigFile";
		const TCHAR* ConfigContents = TEXT(R"(
[NoScheme]

[NoSchemeSpace ]

[NoSchemeSpaces   ]

[SchemeFilter https]
!AllowedDomains=ClearArray
+AllowedDomains=epicgames.com
+AllowedDomains=.epicgames.com
!DevAllowedDomains=ClearArray
+DevAllowedDomains=docs.unrealengine.com

[SchemeFilter http]
!AllowedDomains=ClearArray
+AllowedDomains=unrealengine.com

[SchemeNoDomain https]
!AllowedDomains=ClearArray
)");

		Private::FScopedConfigFile ScopedFile(TEXT("TestConfigFile"), ConfigContents);
		REQUIRE(ScopedFile.WasAdded());

		// Reject everything
		FURLRequestFilter NoScheme{TEXT("NoScheme"), ScopedFile.GetFilename()};
		CHECK(!NoScheme.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(!NoScheme.IsRequestAllowed(TEXTVIEW("https://unrealengine.com")));
		CHECK(!NoScheme.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(!NoScheme.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!NoScheme.IsRequestAllowed(TEXTVIEW("https://127.0.0.1")));
		CHECK(!NoScheme.IsRequestAllowed(TEXTVIEW("127.0.0.1")));
		CHECK(!NoScheme.IsRequestAllowed(TEXTVIEW("https://[::1]")));
		CHECK(!NoScheme.IsRequestAllowed(TEXTVIEW("[::1]")));

		FURLRequestFilter NoSchemeSpace{TEXT("NoSchemeSpace"), ScopedFile.GetFilename()};
		CHECK(!NoSchemeSpace.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(!NoSchemeSpace.IsRequestAllowed(TEXTVIEW("https://unrealengine.com")));
		CHECK(!NoSchemeSpace.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(!NoSchemeSpace.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!NoSchemeSpace.IsRequestAllowed(TEXTVIEW("https://127.0.0.1")));
		CHECK(!NoSchemeSpace.IsRequestAllowed(TEXTVIEW("127.0.0.1")));
		CHECK(!NoSchemeSpace.IsRequestAllowed(TEXTVIEW("https://[::1]")));
		CHECK(!NoSchemeSpace.IsRequestAllowed(TEXTVIEW("[::1]")));

		FURLRequestFilter NoSchemeSpaces{TEXT("NoSchemeSpaces"), ScopedFile.GetFilename()};
		CHECK(!NoSchemeSpaces.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(!NoSchemeSpaces.IsRequestAllowed(TEXTVIEW("https://unrealengine.com")));
		CHECK(!NoSchemeSpaces.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(!NoSchemeSpaces.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!NoSchemeSpaces.IsRequestAllowed(TEXTVIEW("https://127.0.0.1")));
		CHECK(!NoSchemeSpaces.IsRequestAllowed(TEXTVIEW("127.0.0.1")));
		CHECK(!NoSchemeSpaces.IsRequestAllowed(TEXTVIEW("https://[::1]")));
		CHECK(!NoSchemeSpaces.IsRequestAllowed(TEXTVIEW("[::1]")));

		// Normal filtering
		FURLRequestFilter SchemeFilter{TEXT("SchemeFilter"), ScopedFile.GetFilename()};
		CHECK(SchemeFilter.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(!SchemeFilter.IsRequestAllowed(TEXTVIEW("https://unrealengine.com")));
		CHECK(!SchemeFilter.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(!SchemeFilter.IsRequestAllowed(TEXTVIEW("epicgames.com")));
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
		CHECK(!SchemeFilter.IsRequestAllowed(TEXTVIEW("https://docs.unrealengine.com")));
#else
		CHECK(SchemeFilter.IsRequestAllowed(TEXTVIEW("https://docs.unrealengine.com")));
#endif

		FURLRequestFilter SchemeNoDomain{TEXT("SchemeNoDomain"), ScopedFile.GetFilename()};
		CHECK(SchemeNoDomain.IsRequestAllowed(TEXTVIEW("https://epicgames.com")));
		CHECK(SchemeNoDomain.IsRequestAllowed(TEXTVIEW("https://epicgames.com/path/to/somewhere")));
		CHECK(SchemeNoDomain.IsRequestAllowed(TEXTVIEW("https://subdomain.epicgames.com")));
		CHECK(SchemeNoDomain.IsRequestAllowed(TEXTVIEW("https://subdomain.epicgames.com/path/to/somewhere")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("epicgames.com")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("httpsepicgames.com")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("epicgameshttps")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("epicgames.com/path/to/somewhere")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("http://epicgames.com")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("http://epicgames.com/path/to/somewhere")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("http://subdomain.epicgames.com")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("http://subdomain.epicgames.com/path/to/somewhere")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("file://C:/somefile.txt")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1")));
		CHECK(!SchemeNoDomain.IsRequestAllowed(TEXTVIEW("unreal://127.0.0.1:7777")));
	}
}

}

#endif //WITH_TESTS
