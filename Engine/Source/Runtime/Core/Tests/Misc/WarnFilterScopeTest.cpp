// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"
#include "Tests/WarnFilterScope.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogTesting, Log, All);
DEFINE_LOG_CATEGORY(LogTesting);

TEST_CASE("UE::Testing::WarnFilterScope")
{
	bool bFiltered = false;
	FWarnFilterScope _([&bFiltered](const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category)
		{
			bFiltered = FCString::Strcmp(Message, TEXT("Test Message")) == 0 && Verbosity == ELogVerbosity::Type::Warning && Category == TEXT("LogTesting");
			return bFiltered;
		});
	UE_LOGF(LogTesting, Warning, "Test Message");
	CHECK(bFiltered);
}

#endif
