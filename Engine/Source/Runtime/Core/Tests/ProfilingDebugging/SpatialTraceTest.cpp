// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS
#include "ProfilingDebugging/SpatialTrace.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FSpatialTraceMacroTest, "System::Core::SpatialTrace::Macros", "[ApplicationContextMask][SmokeFilter]")
{
	SECTION("Position logging macro must compile")
	{
		UE_TRACE_SPATIAL_POINT_POS_SPEC_DECLARE(TestPositionSpec, TEXT("MacroTestPosition"));
		UE_TRACE_SPATIAL_POINT_POS_LOG(TestPositionSpec, FVector(1.0, 2.0, 3.0));
	}

	SECTION("Position/direction logging macro must compile")
	{
		UE_TRACE_SPATIAL_POINT_POS_DIR_SPEC_DECLARE(TestPosDirectionSpec, TEXT("MacroTestPosDirection"));
		UE_TRACE_SPATIAL_POINT_POS_DIR_LOG(TestPosDirectionSpec, FVector(1.0, 2.0, 3.0), FVector::ForwardVector);
	}

	SECTION("Position/velocity logging macro must compile")
	{
		UE_TRACE_SPATIAL_POINT_POS_VEL_SPEC_DECLARE(TestPosVelocitySpec, TEXT("MacroTestPosVelocity"));
		UE_TRACE_SPATIAL_POINT_POS_VEL_LOG(TestPosVelocitySpec, FVector(0.0, 0.0, 0.0), FVector(10.0, 0.0, 0.0));
	}

	SECTION("Position/velocity/direction logging macro must compile")
	{
		UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_SPEC_DECLARE(TestPosVelDirSpec, TEXT("MacroTestPosVelDir"));
		UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_LOG(TestPosVelDirSpec, FVector(5.0, 10.0, 15.0), FVector(1.0, 0.0, 0.0), FVector::UpVector);
	}

	SECTION("Inline logging macros must compile")
	{
		UE_TRACE_SPATIAL_POINT_POS_INLINE_LOG(TEXT("InlinePosition"), FVector(100.0, 200.0, 300.0));
		UE_TRACE_SPATIAL_POINT_POS_DIR_INLINE_LOG(TEXT("InlinePosDirection"), FVector(1.0, 1.0, 1.0), FVector::UpVector);
		UE_TRACE_SPATIAL_POINT_POS_VEL_INLINE_LOG(TEXT("InlinePosVelocity"), FVector::ZeroVector, FVector(0.0, 0.0, 9.8));
		UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_INLINE_LOG(TEXT("InlinePosVelDir"), FVector(50.0, 60.0, 70.0), FVector(0.0, 5.0, 0.0), FVector::ForwardVector);
	}
}

#if UE_TRACE_SPATIAL_ENABLED

TEST_CASE_NAMED(FSpatialTraceStringOrViewTest, "System::Core::SpatialTrace::StringOrView", "[ApplicationContextMask][SmokeFilter]")
{
	using FStringOrView = UE::SpatialTrace::Private::FStringOrView;

	SECTION("View construction and access")
	{
		const TCHAR* Literal = TEXT("TestLiteral");
		FStringOrView ViewWrapper = FStringOrView::FromView(Literal);

		TStringView<TCHAR> Retrieved = ViewWrapper.GetView();
		CHECK_MESSAGE(TEXT("StringOrView must correctly store and retrieve string views"), Retrieved.GetData() == Literal);
	}

	SECTION("String construction and access")
	{
		FString OriginalString = "TestString";
		FStringOrView StringWrapper = FStringOrView::FromString(OriginalString);

		TStringView<TCHAR> Retrieved = StringWrapper.GetView();
		CHECK_MESSAGE(TEXT("StringOrView must correctly store and retrieve owned strings"), Retrieved == TStringView<TCHAR>(OriginalString));
	}

	SECTION("Moved string construction")
	{
		FString MovedString = "MovedString";
		const TCHAR* OrigDataPtr = *MovedString;
		FStringOrView MovedWrapper = FStringOrView::FromString(MoveTemp(MovedString));

		TStringView<TCHAR> Retrieved = MovedWrapper.GetView();
		CHECK_MESSAGE(TEXT("StringOrView must correctly handle moved strings"), Retrieved.GetData() == OrigDataPtr);
	}
}

/*
 * The following test contains commented out code which would trigger compilation errors if they were uncommented.
 */
TEST_CASE_NAMED(FSpatialTraceCompileErrorTest, "System::Core::SpatialTrace::CompileErrors", "[ApplicationContextMask][SmokeFilter]")
{
	// Error: No position component.
	/*using FInvalidSpec1 = UE::SpatialTrace::Private::TPointSpec<UE::SpatialTrace::Private::EPointComponentFlags::Velocity>;
	FInvalidSpec1 InvalidSpec1 = FInvalidSpec1::FromLiteral(TEXT("InvalidSpec1"));*/

	// Error: Invalid display name type (not const TCHAR array)
	/*FString NonConstName = "Invalid";
	UE_TRACE_SPATIAL_POINT_POS_SPEC_DECLARE(TestSpec, NonConstName);*/

	// Error: Wrong parameter count
	/*UE_TRACE_SPATIAL_POINT_POS_DIR_SPEC_DECLARE(Spec, TEXT("Test"));
	UE_TRACE_SPATIAL_POINT_POS_LOG(Spec, FVector3d::ZeroVector);
	UE_TRACE_SPATIAL_POINT_POS_VEL_DIR_LOG(Spec, FVector3d::ZeroVector, FVector3d::ForwardVector, FVector3d::UpVector);*/
}

#endif // UE_TRACE_SPATIAL_ENABLED

#endif // WITH_TESTS
