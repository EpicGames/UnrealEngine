// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "Tests/Benchmark.h"
#include "Tasks/Pipe.h"
#include "HAL/Thread.h"
#include "Misc/SpinLock.h"
#include "Misc/ScopeLock.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Async/ParallelFor.h"
#include "Tests/TestHarnessAdapter.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/PlatformManualResetEvent.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Containers/UnrealString.h"
#include "Sanitizer/RaceDetector.h"

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"

#include <atomic>
#include <thread>

#if WITH_TESTS && USING_INSTRUMENTATION

namespace UE { namespace TasksTests
{
	using namespace Tasks;
	using namespace UE::Sanitizer;
	using namespace UE::Sanitizer::RaceDetector;

	// A "fake" spinning wait that makes sure a "Wait" doesn't make the caller thread
	// retract the task. Used to test different scenarios.
	void TestWait(FTask& Task)
	{
		while (!Task.IsCompleted())
		{
			FPlatformProcess::Yield();
		}
	}

	void TestWait(FGraphEventRef& GraphEvent)
	{
		while (!GraphEvent->IsComplete())
		{
			FPlatformProcess::Yield();
		}
	}

	struct FDataRace {
		void* Address;
		FString FirstThreadName;
		FString SecondThreadName;
		FFullLocation FirstLocation;
		FFullLocation SecondLocation;

		bool operator==(const FDataRace& Other) const
		{
			return Address == Other.Address &&
				((FirstThreadName == Other.FirstThreadName && SecondThreadName == Other.SecondThreadName) ||
				(SecondThreadName == Other.FirstThreadName && FirstThreadName == Other.SecondThreadName));
		}
	};

	class FRaceCollectorBase {
	public:
	
		bool IsEmpty()
		{
			return Races.IsEmpty();
		}

		void Reset()
		{
			Races.Reset();
		}

		bool Contains(void* RaceAddress)
		{
			for (auto& Race : Races)
			{
				if (Race.Address == RaceAddress)
				{
					return true;
				}
			}
			return false;
		}

		bool Contains(void* RaceAddress, const FString& FirstTaskName, const FString& SecondTaskName)
		{
			FDataRace Check{ RaceAddress, FirstTaskName, SecondTaskName };
			for (auto& Race : Races)
			{
				if (Race == Check)
					return true;
			}
			return false;
		}

		uint32 NumRacesForAddress(void* RaceAddress)
		{
			uint32 Num = 0;
			for (auto& Race : Races)
			{
				if (Race.Address == RaceAddress)
				{
					Num++;
				}
			}
			return Num;
		}

	protected:
		UE::FSpinLock RaceLock;
		TArray<FDataRace> Races;
	};

	class FThreadRaceCollector : public FRaceCollectorBase {
	public:
		FThreadRaceCollector()
		{
			SetRaceCallbackFn(*this);
		}

		~FThreadRaceCollector()
		{
			ResetRaceCallbackFn();
		}

		void operator()(uint64 RaceAddress, uint32 FirstThreadId, uint32 SecondThreadId, const FFullLocation& FirstLocation, const FFullLocation& SecondLocation)
		{
			UE::TUniqueLock Lock(RaceLock);
			Races.Add(FDataRace{ (void*)RaceAddress, FString::Printf(TEXT("%d"), FirstThreadId), FString::Printf(TEXT("%d"), SecondThreadId), FirstLocation, SecondLocation });
		}
	};

	void Test();
	void TestLockFree(int32 OuterIters = 3);

	TEST_CASE_NAMED(FRaceDetectorTasksWithPrereqTest, "System::Core::Sanitizer::RaceDetector::TasksWithPrereq", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);
		FThreadRaceCollector Collector;

		for (uint32 i = 0; i < 100; ++i)
		{
			Collector.Reset();
			ToggleRaceDetection(true);

			int x = 0, y = 0;

			FGraphEventRef LegacyTask = FFunctionGraphTask::CreateAndDispatchWhenReady([&]
			{
				x = 1;
				y = 1;
			}, TStatId(), nullptr, ENamedThreads::AnyHiPriThreadHiPriTask);

			{
				FGraphEventArray Prereqs;
				Prereqs.Add(LegacyTask);
				LegacyTask = FFunctionGraphTask::CreateAndDispatchWhenReady([&]
				{
					x = 2;
					y = 2;
				}, TStatId(), &Prereqs, ENamedThreads::AnyHiPriThreadHiPriTask);
			}

			LegacyTask->Wait();

			ToggleRaceDetection(false);

			CHECK(!Collector.Contains(&x));
			CHECK(!Collector.Contains(&y));
		}
	}

	TEST_CASE_NAMED(FRaceDetectorStdThreadRacesTest, "System::Core::Sanitizer::RaceDetector::StdThreadRaces", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		int x = 0, y = 0, z = 0, w = 0;
		std::atomic<int> sync = 0;
		std::thread t1{ [&]() {
			x = 1;
			y = 1;
			z = 1;
		} };

		w = 2;

		std::thread t2{ [&]() {
			w = 3;
			z = 3;
		} };

		x = 2;

		t1.join();
		t2.join();

		y = 2;

		ToggleRaceDetection(false);

		CHECK(Collector.Contains(&x));
		CHECK(!Collector.Contains(&y));
		CHECK(Collector.Contains(&z));
		CHECK(!Collector.Contains(&w));
	}

	TEST_CASE_NAMED(FRaceDetectorStdThreadAtomicSyncTest, "System::Core::Sanitizer::RaceDetector::StdThreadAtomicSync", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);
		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		int x = 0, y = 0, result = 0;
		std::atomic<int> sync = 0;
		std::thread t1{ [&]() {
			x = 10;

			int expected = 0;
			if (!sync.compare_exchange_strong(expected, 1))
			{
				result = y;
			}
		} };

		y = 20;

		int expected = 0;
		if (!sync.compare_exchange_strong(expected, 1))
		{
			result = x;
		}

		t1.join();

		ToggleRaceDetection(false);

		CHECK(!Collector.Contains(&x));
		CHECK(!Collector.Contains(&y));
		CHECK(!Collector.Contains(&result));
	}

	TEST_CASE_NAMED(FRaceDetectorStdThreadSynchEventTest, "System::Core::Sanitizer::RaceDetector::StdThreadSynchEvent", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		FEvent* Event = FPlatformProcess::GetSynchEventFromPool(false);
		Event->Reset();

		int x = 0, y = 0, result = 0;

		ToggleFilterDetailedLogOnAddress(&y);
		ToggleGlobalDetailedLog(true);

		std::thread t1{ [&]() {
			x = 10;
			Event->Trigger();
			y = 20;
		} };

		Event->Wait();
		result = x;
		result += y;

		t1.join();

		ToggleFilterDetailedLogOnAddress(nullptr);
		ToggleGlobalDetailedLog(false);

		ToggleRaceDetection(false);

		CHECK(!Collector.Contains(&x));
		CHECK(Collector.Contains(&y));
		CHECK(result); // avoid the result and its operations to be optimized out.
	}

	TEST_CASE_NAMED(FRaceDetectorStdThreadManualResetEventTest, "System::Core::Sanitizer::RaceDetector::StdThreadManualResetEvent", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);
		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		UE::FPlatformManualResetEvent Event;
		Event.Reset();

		int x = 0, y = 0, result = 0;
		std::thread t1{ [&]() {
			x = 10;
			Event.Notify();
			y = 20;
		} };

		Event.Wait();
		result = x;
		result += y;

		t1.join();

		ToggleRaceDetection(false);

		CHECK(!Collector.Contains(&x));
		CHECK(Collector.Contains(&y));
		CHECK(result); // avoid the result and its operations to be optimized out.
	}

	TEST_CASE_NAMED(FRaceDetectorModernTasksTest, "System::Core::Sanitizer::RaceDetector::ModernTasks", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		int x = 0, y = 0;
		FTask TaskA = UE::Tasks::Launch(TEXT("A"), [&]() {
			x = 3;
			y = 3;
		});
		x = 5;
		TestWait(TaskA);

		y = 5;

		ToggleRaceDetection(false);
		CHECK(Collector.Contains(&x));
		CHECK(!Collector.Contains(&y));
	}

	TEST_CASE_NAMED(FRaceDetectorLegacyTasksTest, "System::Core::Sanitizer::RaceDetector::LegacyTasks", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		int x = 0, y = 0;
		FGraphEventRef LegacyTask = FFunctionGraphTask::CreateAndDispatchWhenReady([&]
		{
			x = 3;
			y = 3;
		}, TStatId(), nullptr, ENamedThreads::AnyHiPriThreadHiPriTask);

		x = 5;

		TestWait(LegacyTask);

		y = 5;

		ToggleRaceDetection(false);

		CHECK(Collector.Contains(&x));
		CHECK(!Collector.Contains(&y));
	}

	TEST_CASE_NAMED(FRaceDetectorTwoAsyncThreadRaceTest, "System::Core::Sanitizer::RaceDetector::TwoAsyncThreadRace", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		int x = 0;
		TFuture<void> FutureA = AsyncThread([&x]() {
			x = 1;
		});

		x = 2;

		FutureA.Get();

		ToggleRaceDetection(false);

		CHECK(Collector.Contains(&x));
	}

	TEST_CASE_NAMED(FRaceDetectorAsyncThreadAtomicSyncTest, "System::Core::Sanitizer::RaceDetector::AsyncThreadAtomicSync", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		int x = 0;
		std::atomic<bool> sync = false;
		TFuture<void> FutureA = AsyncThread([&x, &sync]() {
			ToggleRaceDetection(true);
			x = 1;
			sync.store(true);
			ToggleRaceDetection(false);
		});

		FutureA.Get();

		TFuture<void> FutureB = AsyncThread([&x, &sync]() {
			ToggleRaceDetection(true);
			if (sync.load())
			{
				x = 2;
			}
			ToggleRaceDetection(false);
		});


		FutureB.Get();

		CHECK(!Collector.Contains(&x));
	}

	TEST_CASE_NAMED(FRaceDetectorAsyncThreadAtomicAccessTest, "System::Core::Sanitizer::RaceDetector::AsyncThreadAtomicAccess", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		std::atomic<uint32> x = 0;
		TFuture<void> FutureA = AsyncThread([&x]() {
			x = 1;
		});

		TFuture<void> FutureB = AsyncThread([&x]() {
			x = 2;
		});

		FutureA.Get();
		FutureB.Get();

		ToggleRaceDetection(false);

		CHECK(!Collector.Contains(&x));
	}

	TEST_CASE_NAMED(FRaceDetectorAtomicRefMixedAccessRaceTest, "System::Core::Sanitizer::RaceDetector::AtomicRefMixedAccessRace", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);
		FThreadRaceCollector Collector;
		ToggleRaceDetection(true);

		// One thread writes Data directly (non-atomic); the other writes the same memory
		// through atomic_ref. A non-atomic write racing with an atomic write at the same
		// address is undefined under the C++ memory model and must be reported on &Data.
		uint32 Data = 0;

		std::thread WriterA([&Data]() {
			Data = 0xAA;
		});

		std::thread WriterB([&Data]() {
			std::atomic_ref<uint32>(Data).store(0xBB, std::memory_order_relaxed);
		});

		WriterA.join();
		WriterB.join();

		ToggleRaceDetection(false);

		CHECK(Data == 0xAA || Data == 0xBB);
		CHECK(Collector.Contains(&Data));
	}

	TEST_CASE_NAMED(FRaceDetectorAtomicRefSyncTest, "System::Core::Sanitizer::RaceDetector::AtomicRefSync", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);
		FThreadRaceCollector Collector;
		ToggleRaceDetection(true);

		// Producer writes Data, then publishes via release atomic_ref.store on Flag.
		// Consumer waits on acquire atomic_ref.load on Flag, then reads Data. The
		// release/acquire pair must establish happens-before, so the non-atomic
		// access to Data is NOT a race. With the bug, the atomic_ref operations
		// record synchronization at a stack-local instead of &Flag, no
		// happens-before is established for Flag, and Data shows up as a race.
		int32 Data = 0;
		bool Flag = false;

		std::thread Producer([&Data, &Flag]() {
			Data = 42;
			std::atomic_ref<bool>(Flag).store(true, std::memory_order_release);
		});

		while (!std::atomic_ref<bool>(Flag).load(std::memory_order_acquire))
		{
			FPlatformProcess::Yield();
		}
		volatile int32 Sink = Data;
		(void)Sink;

		Producer.join();

		ToggleRaceDetection(false);

		CHECK(!Collector.Contains(&Data));
		CHECK(!Collector.Contains(&Flag));
	}

	TEST_CASE_NAMED(FRaceDetectorAtomicRefFetchAddRaceTest, "System::Core::Sanitizer::RaceDetector::AtomicRefFetchAddRace", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);
		FThreadRaceCollector Collector;
		ToggleRaceDetection(true);

		// Same conflict as the mixed-access test, but exercises the
		// _Atomic_integral<T&, N>::fetch_add path instead of plain store.
		int32 Counter = 0;

		std::thread WriterA([&Counter]() {
			Counter = Counter + 1;
		});

		std::thread WriterB([&Counter]() {
			std::atomic_ref<int32>(Counter).fetch_add(1, std::memory_order_relaxed);
		});

		WriterA.join();
		WriterB.join();

		ToggleRaceDetection(false);
		
		CHECK(Counter > 0 && Counter <= 2);
		CHECK(Collector.Contains(&Counter));
	}

	TEST_CASE_NAMED(FRaceDetectorAtomicRefAtomicAccessTest, "System::Core::Sanitizer::RaceDetector::AtomicRefAtomicAccess", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);
		FThreadRaceCollector Collector;
		ToggleRaceDetection(true);

		// Two threads access the same memory exclusively through atomic_ref.
		// Mirrors FRaceDetectorAsyncThreadAtomicAccessTest but with atomic_ref
		// in place of std::atomic. After the fix, both stores are recorded as
		// atomic accesses to &x, so no race must be reported.
		uint32 x = 0;

		std::thread WriterA([&x]() {
			std::atomic_ref<uint32>(x).store(1, std::memory_order_relaxed);
		});

		std::thread WriterB([&x]() {
			std::atomic_ref<uint32>(x).store(2, std::memory_order_relaxed);
		});

		WriterA.join();
		WriterB.join();

		ToggleRaceDetection(false);

		CHECK(x == 1 || x == 2);
		CHECK(!Collector.Contains(&x));
	}

	TEST_CASE_NAMED(FRaceDetectorVirtualPointerHarmfulTest, "System::Core::Sanitizer::RaceDetector::VirtualPointerHarmful", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		class FBase
		{
		public:
			virtual void Function() {}
			void Done() { Event.Notify(); }
			virtual ~FBase() { Event.Wait(); }
		private:
			UE::FManualResetEvent Event;
		};

		class FDerived : public FBase
		{
		public:
			virtual void Function() {}
			virtual ~FDerived() {}
		};

		FBase* Base = new FDerived; 
		TFuture<void> FutureA = AsyncThread([Base] { 
			Base->Function(); 
			Base->Done(); 
		});

		// This is a race because the function called could be the one of FDerived or FBase
		// depending if the call is made before or after we enter the destructor and the vptr
		// is rewritten to point to the base functions.
		delete Base;
		FutureA.Get();

		ToggleRaceDetection(false);

		// For this test, assume the vptr is stored as the first member of the instance.
		CHECK(Collector.Contains(Base));
	}

	TEST_CASE_NAMED(FRaceDetectorVirtualPointerBenignTest, "System::Core::Sanitizer::RaceDetector::VirtualPointerBenign", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		class FBase
		{
		public:
			virtual void Function() {}
			void Done() { Event.Notify(); }
			virtual ~FBase() { Event.Wait(); }
		private:
			UE::FManualResetEvent Event;
		};

		FBase* Base = new FBase;
		TFuture<void> FutureA = AsyncThread([Base] {
			Base->Function();
			Base->Done();
		});

		// This race is considered benign since the vptr can only point on the base class
		// so the racedetector won't report it.
		delete Base;
		FutureA.Get();

		ToggleRaceDetection(false);

		// For this test, assume the vptr is stored as the first member of the instance.
		CHECK(!Collector.Contains(Base));
	}

	// Validates that SuspendThread/ResumeThread act as acquire/release barriers
	// in the race detector. This is the pattern used by callstack capture code
	// (WindowsPlatformStackWalk.cpp) where a thread is suspended, its context is
	// read, and then it is resumed.
	TEST_CASE_NAMED(FRaceDetectorSuspendThreadBarrierTest, "System::Core::Sanitizer::RaceDetector::SuspendThreadBarrier", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		int32 X = 0, Y = 0;

		std::atomic<bool> WorkerReady{false};
		std::atomic<bool> WorkerCanExit{false};

		class FSuspendTestRunnable : public FRunnable
		{
		public:
			int& X;
			int& Y;
			std::atomic<bool>& bReady;
			std::atomic<bool>& bCanExit;

			FSuspendTestRunnable(int& InX, int& InY, std::atomic<bool>& InReady, std::atomic<bool>& InCanExit)
				: X(InX), Y(InY), bReady(InReady), bCanExit(InCanExit) {}

			virtual uint32 Run() override
			{
				// Write X before we get suspended.
				X = 42;
				bReady.store(true, std::memory_order_relaxed);

				// Spin until the main thread lets us continue (after Resume).
				while (!bCanExit.load(std::memory_order_relaxed))
				{
					FPlatformProcess::Yield();
				}

				// Read Y after being resumed - should see main thread's write
				// with no race because ResumeThread is a release barrier.
				volatile int32 Sink = Y;
				(void)Sink;

				return 0;
			}
		};

		FSuspendTestRunnable Runnable(X, Y, WorkerReady, WorkerCanExit);
		FRunnableThread* WorkerThread = FRunnableThread::Create(&Runnable, TEXT("SuspendBarrierTest"));

		// Wait until the worker has written x.
		while (!WorkerReady.load(std::memory_order_relaxed))
		{
			FPlatformProcess::Yield();
		}

		// Suspend the worker. After this returns, all of the worker's prior
		// writes (including x = 42) are visible to us without a race.
		WorkerThread->Suspend(true);

		// Read x while worker is suspended - no race because SuspendThread
		// establishes an acquire barrier.
		volatile int32 Sink = X;
		(void)Sink;

		// Write y while worker is suspended - will be visible to the worker
		// after ResumeThread (release barrier).
		Y = 99;

		// Resume the worker.
		WorkerThread->Suspend(false);
		WorkerCanExit.store(true, std::memory_order_relaxed);

		WorkerThread->WaitForCompletion();
		delete WorkerThread;

		ToggleRaceDetection(false);

		// Neither access should be flagged as a race.
		CHECK(!Collector.Contains(&X));
		CHECK(!Collector.Contains(&Y));
	}

	struct FTLSReuseBetweenThreads
	{
		int32 Value = 0;
	};

	static thread_local FTLSReuseBetweenThreads TLSReuseBetweenThreads;

	TEST_CASE_NAMED(FRaceDetectorTLSReuseBetweenThreads, "System::Core::Sanitizer::RaceDetector::TLSReuseBetweenThreads", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);

		FThreadRaceCollector Collector;

		ToggleRaceDetection(true);

		// Spawning and shutting down thread repetitively is enough to trigger a few false positives
		// throughout the UE code-base. Also add our own TLS value bump here for completeness.
		std::atomic<bool> bDone = false;
		for (int32 i = 0; i < 100; ++i)
		{
			bDone.store(false, std::memory_order_relaxed);
			AsyncThread(
				[&bDone]()
				{
					TLSReuseBetweenThreads.Value++;

					bDone.store(true, std::memory_order_relaxed);
				});
			
			// Wait until the thread is done to increase our chance of
			// the TLS memory address to be reused for the next spawned thread
			while (!bDone.load(std::memory_order_relaxed))
			{
				FPlatformProcess::Sleep(0);
			}
		}

		ToggleRaceDetection(false);

		// if race detector properly detects TLS addresses being released
		// by a thread, it shouldn't detect any race on the thread_local.
		CHECK(Collector.IsEmpty());
	}

	struct FTestStruct
	{
		int32 Index;
		int32 Constant;
		FTestStruct(int32 InIndex)
			: Index(InIndex)
			, Constant(0xfe05abcd)
		{
		}
	};

	struct FTestRigFIFO
	{
		FLockFreePointerFIFOBase<FTestStruct, PLATFORM_CACHE_LINE_SIZE> Test1;
		FLockFreePointerFIFOBase<FTestStruct, 8> Test2;
		FLockFreePointerFIFOBase<FTestStruct, 8, 1 << 4> Test3;
	};

	struct FTestRigLIFO
	{
		FLockFreePointerListLIFOBase<FTestStruct, PLATFORM_CACHE_LINE_SIZE> Test1;
		FLockFreePointerListLIFOBase<FTestStruct, 8> Test2;
		FLockFreePointerListLIFOBase<FTestStruct, 8, 1 << 4> Test3;
	};


	void Test()
	{
		ToggleRaceDetection(false);
		FThreadRaceCollector Collector;
		ToggleRaceDetection(true);

		TestLockFree();

		ToggleRaceDetection(false);
		Collector.Contains(nullptr);
	}

	void TestLockFree(int32 OuterIters)
	{
		if (!FTaskGraphInterface::IsMultithread())
		{
			UE_LOGF(LogConsoleResponse, Display, "WARNING: TestLockFree disabled for non multi-threading platforms");
			return;
		}

		const int32 NumWorkers = FTaskGraphInterface::Get().GetNumWorkerThreads();
		// If we have too many threads active at once, they become too slow due to contention.  Set a reasonable maximum for how many are required to guarantee correctness of our LockFreePointers.
		const int32 MaxWorkersForTest = 5;
		const int32 MinWorkersForTest = 2; // With less than two threads we're not testing threading at all, so the test is pointless.
		if (NumWorkers < MinWorkersForTest)
		{
			UE_LOGF(LogConsoleResponse, Display, "WARNING: TestLockFree disabled for current machine because of not enough worker threads.  Need %d, have %d.", MinWorkersForTest, NumWorkers);
			return;
		}

		const uint32 NumWorkersForTest = static_cast<uint32>(FMath::Clamp(NumWorkers, MinWorkersForTest, MaxWorkersForTest));
		auto RunWorkersSynchronous = [NumWorkersForTest](const TFunction<void(uint32)>& WorkerTask)
		{
			FGraphEventArray Tasks;
			for (uint32 Index = 0; Index < NumWorkersForTest; Index++)
			{
				TUniqueFunction<void()> WorkerTaskWithIndex{ [Index, &WorkerTask] { WorkerTask(Index); } };
				Tasks.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(WorkerTaskWithIndex), TStatId{}, nullptr, ENamedThreads::AnyNormalThreadHiPriTask));
			}
			FTaskGraphInterface::Get().WaitUntilTasksComplete(MoveTemp(Tasks));
		};

		for (int32 Iter = 0; Iter < OuterIters; Iter++)
		{
			{
				UE_LOGF(LogTemp, Display, "******************************* Iter FIFO %d", Iter);
				FTestRigFIFO Rig;
				for (int32 Index = 0; Index < 1000; Index++)
				{
					Rig.Test1.Push(new FTestStruct(Index));
				}
				TFunction<void(uint32)> Broadcast =
					[&Rig](uint32 WorkerIndex)
				{
					FRandomStream Stream(((int32)WorkerIndex) * 7 + 13);
					for (int32 Index = 0; Index < 100000; Index++)
					{
						if (Stream.FRand() < .03f)
						{
							TArray<FTestStruct*> Items;
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.PopAll(Items);
								}
								else if (r < .66f)
								{
									Rig.Test2.PopAll(Items);
								}
								else
								{
									Rig.Test3.PopAll(Items);
								}
							}
							for (FTestStruct* Item : Items)
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.Push(Item);
								}
								else if (r < .66f)
								{
									Rig.Test2.Push(Item);
								}
								else
								{
									Rig.Test3.Push(Item);
								}
							}
						}
						else
						{
							FTestStruct* Item;
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Item = Rig.Test1.Pop();
								}
								else if (r < .66f)
								{
									Item = Rig.Test2.Pop();
								}
								else
								{
									Item = Rig.Test3.Pop();
								}
							}
							if (Item)
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.Push(Item);
								}
								else if (r < .66f)
								{
									Rig.Test2.Push(Item);
								}
								else
								{
									Rig.Test3.Push(Item);
								}
							}
						}
					}
				};
				RunWorkersSynchronous(Broadcast);

				TArray<FTestStruct*> Items;
				Rig.Test1.PopAll(Items);
				Rig.Test2.PopAll(Items);
				Rig.Test3.PopAll(Items);

				checkf(Items.Num() == 1000, TEXT("Items %d"), Items.Num());

				for (int32 LookFor = 0; LookFor < 1000; LookFor++)
				{
					bool bFound = false;
					for (int32 Index = 0; Index < 1000; Index++)
					{
						if (Items[Index]->Index == LookFor && Items[Index]->Constant == 0xfe05abcd)
						{
							check(!bFound);
							bFound = true;
						}
					}
					check(bFound);
				}
				for (FTestStruct* Item : Items)
				{
					delete Item;
				}

				UE_LOGF(LogTemp, Display, "******************************* Pass FTestRigFIFO");

			}
			{
				UE_LOGF(LogTemp, Display, "******************************* Iter LIFO %d", Iter);
				FTestRigLIFO Rig;
				for (int32 Index = 0; Index < 1000; Index++)
				{
					Rig.Test1.Push(new FTestStruct(Index));
				}
				TFunction<void(uint32)> Broadcast =
					[&Rig](uint32 WorkerIndex)
				{
					FRandomStream Stream(((int32)WorkerIndex) * 7 + 13);
					for (int32 Index = 0; Index < 100000; Index++)
					{
						if (Index % 200000 == 1)
						{
							//UE_LOGF(LogTemp, Log, "%8d iters thread=%d", Index, int32(WorkerIndex));
						}
						if (Stream.FRand() < .03f)
						{
							TArray<FTestStruct*> Items;
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.PopAll(Items);
								}
								else if (r < .66f)
								{
									Rig.Test2.PopAll(Items);
								}
								else
								{
									Rig.Test3.PopAll(Items);
								}
							}
							for (FTestStruct* Item : Items)
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.Push(Item);
								}
								else if (r < .66f)
								{
									Rig.Test2.Push(Item);
								}
								else
								{
									Rig.Test3.Push(Item);
								}
							}
						}
						else
						{
							FTestStruct* Item;
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Item = Rig.Test1.Pop();
								}
								else if (r < .66f)
								{
									Item = Rig.Test2.Pop();
								}
								else
								{
									Item = Rig.Test3.Pop();
								}
							}
							if (Item)
							{
								float r = Stream.FRand();
								if (r < .33f)
								{
									Rig.Test1.Push(Item);
								}
								else if (r < .66f)
								{
									Rig.Test2.Push(Item);
								}
								else
								{
									Rig.Test3.Push(Item);
								}
							}
						}
					}
				};
				RunWorkersSynchronous(Broadcast);

				TArray<FTestStruct*> Items;
				Rig.Test1.PopAll(Items);
				Rig.Test2.PopAll(Items);
				Rig.Test3.PopAll(Items);

				checkf(Items.Num() == 1000, TEXT("Items %d"), Items.Num());

				for (int32 LookFor = 0; LookFor < 1000; LookFor++)
				{
					bool bFound = false;
					for (int32 Index = 0; Index < 1000; Index++)
					{
						if (Items[Index]->Index == LookFor && Items[Index]->Constant == 0xfe05abcd)
						{
							check(!bFound);
							bFound = true;
						}
					}
					check(bFound);
				}
				for (FTestStruct* Item : Items)
				{
					delete Item;
				}

				UE_LOGF(LogTemp, Display, "******************************* Pass FTestRigLIFO");
			}
		}
	}

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"

	// Validates that unmapping a file view (or unloading a DLL) cleans up shadow
	// state so that reuse of the same virtual address range doesn't produce false
	// positive race reports. See UE-368317.
	//
	// Strategy:
	//   1. Map a pagefile-backed view and write to it (creates shadow entries).
	//   2. Unmap the view — our NtUnmapViewOfSection hook should clean shadow.
	//   3. Reclaim the same VA with VirtualAlloc.
	//   4. Write to the fresh allocation from a new thread.
	//   5. Verify no false positive race between the old and new accesses.
	//
	// Steps 1-5 are retried a few times because reclaiming the exact VA is
	// best-effort; the test is inconclusive (not failing) if it never succeeds.
	TEST_CASE_NAMED(FRaceDetectorUnmapViewShadowCleanupTest, "System::Core::Sanitizer::RaceDetector::UnmapViewShadowCleanup", "[ApplicationContextMask][EngineFilter]")
	{
		CHECK(UE::Sanitizer::RaceDetector::Initialize());

		ToggleRaceDetection(false);
		FThreadRaceCollector Collector;

		constexpr SIZE_T MappingSize = 65536;
		constexpr int32 MaxAttempts = 10;
		bool bTestExecuted = false;

		for (int32 Attempt = 0; Attempt < MaxAttempts && !bTestExecuted; ++Attempt)
		{
			Collector.Reset();

			// Create a pagefile-backed file mapping.
			HANDLE FileMapping = CreateFileMappingW(INVALID_HANDLE_VALUE, NULL, PAGE_READWRITE, 0, (DWORD)MappingSize, NULL);
			CHECK(FileMapping != NULL);

			void* MappedView = MapViewOfFile(FileMapping, FILE_MAP_ALL_ACCESS, 0, 0, MappingSize);
			CHECK(MappedView != nullptr);

			int32* MappedPtr = (int32*)MappedView;

			// Write from a background thread with detection ON to create shadow entries.
			ToggleRaceDetection(true);

			std::atomic<bool> Ready = false;
			std::thread WriterA([MappedPtr, &Ready]() {
				*MappedPtr = 42;

				Ready.store(true, std::memory_order_relaxed);
			});

			// Don't wait right now, it would be a barrier and would prevent the race from happening.
			ON_SCOPE_EXIT { WriterA.join(); };

			// Wait until ready without any barrier
			while (!Ready.load(std::memory_order_relaxed))
			{
			}

			// Unmap. Our NtUnmapViewOfSection hook should clean the shadow.
			void* OldAddr = MappedView;
			UnmapViewOfFile(MappedView);
			CloseHandle(FileMapping);

			// Try to reclaim the exact same virtual address.
			void* NewAlloc = VirtualAlloc(OldAddr, MappingSize, MEM_RESERVE | MEM_COMMIT, PAGE_READWRITE);
			if (NewAlloc != OldAddr)
			{
				if (NewAlloc)
				{
					VirtualFree(NewAlloc, 0, MEM_RELEASE);
				}
				continue; // Retry with a fresh mapping.
			}

			bTestExecuted = true;

			int32* NewPtr = (int32*)NewAlloc;
			std::thread WriterB([NewPtr]() {
				*NewPtr = 99;
			});
			WriterB.join();

			ToggleRaceDetection(false);

			check(!Collector.Contains(NewPtr));

			VirtualFree(NewAlloc, 0, MEM_RELEASE);
		}

		if (!bTestExecuted)
		{
			UE_LOG(LogTemp, Warning, TEXT("UnmapViewShadowCleanup: Could not reclaim VA after %d attempts - test inconclusive"), MaxAttempts);
		}
	}
	
#include "Windows/HideWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

}}

#endif // WITH_TESTS
