// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Misc/Base64.h"
#include "Templates/Function.h"
#include "Tests/TestHarnessAdapter.h"

template <size_t InputSize>
struct TBase64Test
{
	const uint8 EncodeInput[InputSize]; // aka DecodeOutput
	const FString EncodeOutput; // aka DecodeInput

	TArray<uint8> GetInputArray() const
	{
		return TArray<uint8>(EncodeInput, InputSize);
	}
};

struct FBase64StringTest
{
	const FString EncodeInput;
	const FString EncodeOutput;

	TArray<uint8> GetInputArray() const
	{
		TArray<uint8> Out;
		Out.AddUninitialized(EncodeInput.Len());

		for (int Ix = 0; Ix < EncodeInput.Len(); ++Ix)
		{
			CHECK((EncodeInput[Ix] & 0xff) == EncodeInput[Ix]);
			Out[Ix] = (uint8)EncodeInput[Ix];
		}
		return Out;
	}
};

template <size_t NumVectors, typename TestType>
void RunRoundTripSuccessfulBase64Tests(const TestType (&TestVectors)[NumVectors], EBase64Mode Mode)
{
	for (size_t Ix = 0; Ix < NumVectors; ++Ix)
	{
		const auto& Test = TestVectors[Ix];
		const TArray<uint8> InputArray = Test.GetInputArray();

		const FString Encoded = FBase64::Encode(InputArray, Mode);
		CHECK(Encoded == Test.EncodeOutput);

		TArray<uint8> DecodeOut;
		CHECK(FBase64::Decode(Test.EncodeOutput, DecodeOut, Mode));
		CHECK(DecodeOut == InputArray);
	}
}

// Base64 encode/decode works for the test vectors provided in the RFC
TEST_CASE_NAMED(FBase64EncodeDecodeRFCTestVectorsTest, "System::Core::Serialization::Base64::encode/decode RFC test vectors", "[Core][base64][SmokeFilter]")
{
	// This test does a good job of verifying the correct padding of various lengths by going through 2 cycles of the padding modulus.
	const FBase64StringTest TestVectors[] =
	{
		{ FString(), FString() },
		{ FString("f"), FString("Zg==") },
		{ FString("fo"), FString("Zm8=") },
		{ FString("foo"), FString("Zm9v") },
		{ FString("foob"), FString("Zm9vYg==") },
		{ FString("fooba"), FString("Zm9vYmE=") },
		{ FString("foobar"), FString("Zm9vYmFy") },
	};
	RunRoundTripSuccessfulBase64Tests(TestVectors, EBase64Mode::Standard);

	// The results are the same for URL safe as these vectors never produce 0x1f or 0x3f
	RunRoundTripSuccessfulBase64Tests(TestVectors, EBase64Mode::UrlSafe);
}

// Standard base64 encode/decode every possible 6 bit input
TEST_CASE_NAMED(FBase64StandartEncodeDecodeEverySixBitInputTest, "System::Core::Serialization::Base64::Standard encode/decode every 6 bit input", "[Core][base64][SmokeFilter]")
{
	const TBase64Test<48> TestVectors[] =
	{
		{ { 0,16,131,16,81,135,32,146,139,48,211,143,65,20,147,81,85,151,97,150,155,113,215,159,130,24,163,146,89,167,162,154,171,178,219,175,195,28,179,211,93,183,227,158,187,243,223,191 }, FString("ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789+/") }
	};

	RunRoundTripSuccessfulBase64Tests(TestVectors, EBase64Mode::Standard);
}

// UrlSafe base64 encode/decode every possible 6 bit input
TEST_CASE_NAMED(FBase64URLSafeTest, "System::Core::Serialization::Base64::URLSafe encode/decode every 6 bit input", "[Core][base64][SmokeFilter]")
{
	const TBase64Test<48> TestVectors[] = 
	{
		{ { 0,16,131,16,81,135,32,146,139,48,211,143,65,20,147,81,85,151,97,150,155,113,215,159,130,24,163,146,89,167,162,154,171,178,219,175,195,28,179,211,93,183,227,158,187,243,223,191 }, FString("ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789-_") }
	};

	RunRoundTripSuccessfulBase64Tests(TestVectors, EBase64Mode::UrlSafe);
}

void RunNegativeDecodeBase64Test(EBase64Mode Mode, TFunction<bool(int)>&& IsInDecodeAlphabet)
{
	TCHAR BadDecodeBuffer[]
	{
		TEXT('A'), TEXT('B'), TEXT('C'), TEXT('D'), TEXT('E'), TEXT('F'), TEXT('G'), TEXT('H'), TEXT('I'), TEXT('J'), TEXT('K'), TEXT('L'), TEXT('M'), TEXT('N'), TEXT('O'), TEXT('P'),
		TEXT('Q'), TEXT('R'), TEXT('S'), TEXT('T'), TEXT('U'), TEXT('V'), TEXT('W'), TEXT('X'), TEXT('Y'), TEXT('Z'), TEXT('a'), TEXT('b'), TEXT('c'), TEXT('d'), TEXT('e'), TEXT('f'),
		TEXT('g'), TEXT('h'), TEXT('i'), TEXT('j'), TEXT('k'), TEXT('l'), TEXT('m'), TEXT('n'), TEXT('o'), TEXT('p'), TEXT('q'), TEXT('r'), TEXT('s'), TEXT('t'), TEXT('u'), TEXT('v'),
		TEXT('w'), TEXT('x'), TEXT('y'), TEXT('z'), TEXT('0'), TEXT('1'), TEXT('2'), TEXT('3'), TEXT('4'), TEXT('5'), TEXT('6'), TEXT('7'), TEXT('8'), TEXT('9'), TEXT('+'), TEXT('/'),
		TEXT('A'), TEXT('=') /* bad bytes will go here */, TEXT('='), TEXT('=')
	};

	const uint32 NumChars = UE_ARRAY_COUNT(BadDecodeBuffer);
	TArray<uint8> DecodeOut;
	DecodeOut.AddZeroed(FBase64::GetMaxDecodedDataSize(NumChars));

	// Test the end of the buffer so we're in an area that is not decoding 4 bytes at a time.
	for (int Ix = 0; Ix <= 256; ++Ix)
	{
		if (IsInDecodeAlphabet(Ix))
		{
			continue;
		}
		BadDecodeBuffer[NumChars - 3] = TCHAR(Ix);
		CHECK_FALSE(FBase64::Decode(BadDecodeBuffer, NumChars, DecodeOut.GetData(), Mode));
	}

	// Now test the beginning of the buffer so we're in an area that *is* decoding 4 bytes at a time.
	for (int Ix = 0; Ix <= 256; ++Ix)
	{
		if (IsInDecodeAlphabet(Ix))
		{
			continue;
		}
		BadDecodeBuffer[0] = TCHAR(Ix);
		// We use NumChars-4 as the length to ignore the end of buffer sequence.
		CHECK_FALSE(FBase64::Decode(BadDecodeBuffer, NumChars - 4, DecodeOut.GetData(), Mode));
	}
}

// Base64 decode should fail on input that is not in the decoding alphabet
TEST_CASE_NAMED(FBase64NegativeTest, "System::Core::Serialization::Base64::Negative", "[Core][base64][SmokeFilter]")
{
	auto IsInStandardAlphabet = [](int Ix)
	{
		return (Ix >= 'A' && Ix <= 'Z')
			|| (Ix >= 'a' && Ix <= 'z')
			|| (Ix >= '0' && Ix <= '9')
			|| Ix == '+'
			|| Ix == '/';
	};

	auto IsInUrlSafeAlphabet = [](int Ix)
	{
		return (Ix >= 'A' && Ix <= 'Z')
			|| (Ix >= 'a' && Ix <= 'z')
			|| (Ix >= '0' && Ix <= '9')
			|| Ix == '-'
			|| Ix == '_';
	};

	RunNegativeDecodeBase64Test(EBase64Mode::Standard, MoveTemp(IsInStandardAlphabet));
	RunNegativeDecodeBase64Test(EBase64Mode::UrlSafe, MoveTemp(IsInUrlSafeAlphabet));
}

#endif //WITH_TESTS
