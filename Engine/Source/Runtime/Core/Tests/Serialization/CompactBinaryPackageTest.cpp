// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Serialization/CompactBinaryPackage.h"

#include "Algo/IsSorted.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/CompactBinaryValidation.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FCompactBinaryAttachmentTest, "System::Core::Serialization::CompactBinary::Attachment", "[Core][CompactBinary][SmokeFilter]")
{
	const auto TestSaveLoadValidate = [](const FCbAttachment& Attachment)
	{
		TCbWriter<256> Writer;
		FBufferArchive WriteAr;
		Attachment.Save(Writer);
		Attachment.Save(WriteAr);
		FCbFieldIterator Fields = Writer.Save();

		CHECK(MakeMemoryView(WriteAr).EqualBytes(Fields.GetOuterBuffer().GetView()));
		CHECK(ValidateCompactBinaryRange(MakeMemoryView(WriteAr), ECbValidateMode::All) == ECbValidateError::None);
		CHECK(ValidateCompactBinaryAttachment(MakeMemoryView(WriteAr), ECbValidateMode::All) == ECbValidateError::None);

		FCbAttachment FromFields;
		CHECK(FromFields.TryLoad(Fields));
		CHECK_FALSE(Fields);
		CHECK(FromFields == Attachment);

		FCbAttachment FromArchive;
		FMemoryReader ReadAr(WriteAr);
		CHECK(FromArchive.TryLoad(ReadAr));
		CHECK(ReadAr.AtEnd());
		CHECK(FromArchive == Attachment);
	};

	SECTION("Empty Attachment")
	{
		FCbAttachment Attachment;
		CHECK(Attachment.IsNull());
		CHECK_FALSE(Attachment);
		CHECK_FALSE(Attachment.AsBinary());
		CHECK_FALSE(Attachment.AsObject());
		CHECK_FALSE(Attachment.IsBinary());
		CHECK_FALSE(Attachment.IsCompressedBinary());
		CHECK_FALSE(Attachment.IsObject());
		CHECK(Attachment.GetHash() == FIoHash::Zero);
	}

	SECTION("Binary Attachment")
	{
		FSharedBuffer Buffer = FSharedBuffer::Clone(MakeMemoryView<uint8>({0, 1, 2, 3}));
		FCbAttachment Attachment(Buffer);
		CHECK_FALSE(Attachment.IsNull());
		CHECK(Attachment);
		CHECK(Attachment.AsBinary() == Buffer);
		CHECK_FALSE(Attachment.AsObject());
		CHECK(Attachment.IsBinary());
		CHECK_FALSE(Attachment.IsCompressedBinary());
		CHECK_FALSE(Attachment.IsObject());
		CHECK(Attachment.GetHash() == FIoHash::HashBuffer(Buffer));
		TestSaveLoadValidate(Attachment);
	}

	SECTION("Compressed Binary Attachment")
	{
		FCompressedBuffer Buffer = FCompressedBuffer::Compress(FSharedBuffer::Clone(MakeMemoryView<uint8>({0, 1, 2, 3})));
		FCbAttachment Attachment(Buffer);
		CHECK_FALSE(Attachment.IsNull());
		CHECK(Attachment);
		CHECK(Attachment.AsCompressedBinary().GetCompressed().ToShared().GetView().EqualBytes(Buffer.GetCompressed().ToShared().GetView()));
		CHECK_FALSE(Attachment.AsObject());
		CHECK_FALSE(Attachment.IsBinary());
		CHECK(Attachment.IsCompressedBinary());
		CHECK_FALSE(Attachment.IsObject());
		CHECK(Attachment.GetHash() == Buffer.GetRawHash());
		TestSaveLoadValidate(Attachment);
	}

	SECTION("Object Attachment")
	{
		FCbWriter Writer;
		Writer.BeginObject();
		Writer << ANSITEXTVIEW("Name") << 42;
		Writer.EndObject();
		FCbObject Object = Writer.Save().AsObject();
		FCbAttachment Attachment(Object);
		CHECK_FALSE(Attachment.IsNull());
		CHECK(Attachment);
		CHECK(Attachment.AsBinary() == FSharedBuffer());
		CHECK(Attachment.AsObject().Equals(Object));
		CHECK_FALSE(Attachment.IsBinary());
		CHECK_FALSE(Attachment.IsCompressedBinary());
		CHECK(Attachment.IsObject());
		CHECK(Attachment.GetHash() == Object.GetHash());
		TestSaveLoadValidate(Attachment);
	}

	SECTION("Binary View")
	{
		const uint8 Value[]{0, 1, 2, 3};
		FSharedBuffer Buffer = FSharedBuffer::MakeView(MakeMemoryView(Value));
		FCbAttachment Attachment(Buffer);
		CHECK_FALSE(Attachment.IsNull());
		CHECK(Attachment);
		CHECK(Attachment.AsBinary() != Buffer);
		CHECK(Attachment.AsBinary().GetView().EqualBytes(Buffer.GetView()));
		CHECK_FALSE(bool(Attachment.AsObject()));
		CHECK(Attachment.IsBinary());
		CHECK_FALSE(Attachment.IsCompressedBinary());
		CHECK_FALSE(Attachment.IsObject());
		CHECK(Attachment.GetHash() == FIoHash::HashBuffer(Buffer));
	}

	SECTION("Object View")
	{
		FCbWriter Writer;
		Writer.BeginObject();
		Writer << ANSITEXTVIEW("Name") << 42;
		Writer.EndObject();
		FCbObject Object = Writer.Save().AsObject();
		FCbObject ObjectView = FCbObject::MakeView(Object);
		FCbAttachment Attachment(ObjectView);
		CHECK_FALSE(Attachment.IsNull());
		CHECK(Attachment);
		CHECK(Attachment.AsObject().Equals(Object));
		CHECK_FALSE(Attachment.IsBinary());
		CHECK_FALSE(Attachment.IsCompressedBinary());
		CHECK(Attachment.IsObject());
		CHECK(Attachment.GetHash() == Object.GetHash());
	}

	SECTION("Binary Load from View")
	{
		const uint8 Value[]{0, 1, 2, 3};
		const FSharedBuffer Buffer = FSharedBuffer::MakeView(MakeMemoryView(Value));
		FCbAttachment Attachment(Buffer);

		FCbWriter Writer;
		Attachment.Save(Writer);
		FCbFieldIterator Fields = Writer.Save();
		FCbFieldIterator FieldsView = FCbFieldIterator::MakeRangeView(FCbFieldViewIterator(Fields));

		CHECK(Attachment.TryLoad(FieldsView));
		CHECK_FALSE(Attachment.IsNull());
		CHECK(Attachment);
		CHECK_FALSE(FieldsView.GetOuterBuffer().GetView().Contains(Attachment.AsBinary().GetView()));
		CHECK(Attachment.AsBinary().GetView().EqualBytes(Buffer.GetView()));
		CHECK_FALSE(Attachment.AsObject());
		CHECK(Attachment.IsBinary());
		CHECK_FALSE(Attachment.IsCompressedBinary());
		CHECK_FALSE(Attachment.IsObject());
		CHECK(Attachment.GetHash() == FIoHash::HashBuffer(MakeMemoryView(Value)));
	}

	SECTION("Compressed Binary Load from View")
	{
		const uint8 Value[]{0, 1, 2, 3};
		FCompressedBuffer Buffer = FCompressedBuffer::Compress(FSharedBuffer::MakeView(MakeMemoryView(Value)));
		FCbAttachment Attachment(Buffer);

		FCbWriter Writer;
		Attachment.Save(Writer);
		FCbFieldIterator Fields = Writer.Save();
		FCbFieldIterator FieldsView = FCbFieldIterator::MakeRangeView(FCbFieldViewIterator(Fields));

		CHECK(Attachment.TryLoad(FieldsView));
		CHECK_FALSE(Attachment.IsNull());
		CHECK(Attachment);
		CHECK_FALSE(FieldsView.GetOuterBuffer().GetView().Contains(Attachment.AsCompressedBinary().GetCompressed().ToShared().GetView()));
		CHECK(Attachment.AsCompressedBinary().GetCompressed().ToShared().GetView().EqualBytes(Buffer.GetCompressed().ToShared().GetView()));
		CHECK_FALSE(Attachment.AsObject());
		CHECK_FALSE(Attachment.IsBinary());
		CHECK(Attachment.IsCompressedBinary());
		CHECK_FALSE(Attachment.IsObject());
		CHECK(Attachment.GetHash() == FIoHash::HashBuffer(MakeMemoryView(Value)));
	}

	SECTION("Object Load from View")
	{
		FCbWriter ValueWriter;
		ValueWriter.BeginObject();
		ValueWriter << ANSITEXTVIEW("Name") << 42;
		ValueWriter.EndObject();
		const FCbObject Value = ValueWriter.Save().AsObject();
		CHECK(ValidateCompactBinaryRange(Value.GetOuterBuffer(), ECbValidateMode::All) == ECbValidateError::None);
		FCbAttachment Attachment(Value);

		FCbWriter Writer;
		Attachment.Save(Writer);
		FCbFieldIterator Fields = Writer.Save();
		FCbFieldIterator FieldsView = FCbFieldIterator::MakeRangeView(FCbFieldViewIterator(Fields));

		CHECK(Attachment.TryLoad(FieldsView));
		FMemoryView View;
		CHECK_FALSE(Attachment.IsNull());
		CHECK(Attachment);
		CHECK(Attachment.AsBinary().GetView().EqualBytes(FMemoryView()));
		CHECK_FALSE((!Attachment.AsObject().TryGetView(View) || FieldsView.GetOuterBuffer().GetView().Contains(View)));
		CHECK_FALSE(Attachment.IsBinary());
		CHECK_FALSE(Attachment.IsCompressedBinary());
		CHECK(Attachment.IsObject());
		CHECK(Attachment.GetHash() == Value.GetHash());
	}

	SECTION("Binary Null")
	{
		const FCbAttachment Attachment(FSharedBuffer{});
		CHECK(Attachment.IsNull());
		CHECK_FALSE(Attachment.IsBinary());
		CHECK_FALSE(Attachment.IsCompressedBinary());
		CHECK_FALSE(Attachment.IsObject());
		CHECK(Attachment.GetHash() == FIoHash::Zero);
	}

	SECTION("Binary Empty")
	{
		const FCbAttachment Attachment(FUniqueBuffer::Alloc(0).MoveToShared());
		CHECK_FALSE(Attachment.IsNull());
		CHECK(Attachment.IsBinary());
		CHECK_FALSE(Attachment.IsCompressedBinary());
		CHECK_FALSE(Attachment.IsObject());
		CHECK(Attachment.GetHash() == FIoHash::HashBuffer(FSharedBuffer{}));
	}

	SECTION("Compressed Binary Empty")
	{
		const FCbAttachment Attachment(FCompressedBuffer::Compress(FUniqueBuffer::Alloc(0).MoveToShared()));
		CHECK_FALSE(Attachment.IsNull());
		CHECK_FALSE(Attachment.IsBinary());
		CHECK(Attachment.IsCompressedBinary());
		CHECK_FALSE(Attachment.IsObject());
		CHECK(Attachment.GetHash() == FIoHash::HashBuffer(FSharedBuffer{}));
	}

	SECTION("Object Empty")
	{
		const FCbAttachment Attachment(FCbObject{});
		CHECK_FALSE(Attachment.IsNull());
		CHECK_FALSE(Attachment.IsBinary());
		CHECK_FALSE(Attachment.IsCompressedBinary());
		CHECK(Attachment.IsObject());
		CHECK(Attachment.GetHash() == FCbObject().GetHash());
	}
}

TEST_CASE_NAMED(FCompactBinaryPackageTest, "System::Core::Serialization::CompactBinary::Package", "[Core][CompactBinary][SmokeFilter]")
{
	const auto TestSaveLoadValidate = [](const FCbPackage& Package)
	{
		TCbWriter<256> Writer;
		FBufferArchive WriteAr;
		Package.Save(Writer);
		Package.Save(WriteAr);
		FCbFieldIterator Fields = Writer.Save();

		CHECK(MakeMemoryView(WriteAr).EqualBytes(Fields.GetOuterBuffer().GetView()));
		CHECK(ValidateCompactBinaryRange(MakeMemoryView(WriteAr), ECbValidateMode::All) == ECbValidateError::None);
		CHECK(ValidateCompactBinaryPackage(MakeMemoryView(WriteAr), ECbValidateMode::All) == ECbValidateError::None);

		FCbPackage FromFields;
		CHECK(FromFields.TryLoad(Fields));
		CHECK_FALSE(Fields);
		CHECK(FromFields == Package);

		FCbPackage FromArchive;
		FMemoryReader ReadAr(WriteAr);
		CHECK(FromArchive.TryLoad(ReadAr));
		CHECK(ReadAr.AtEnd());
		CHECK(FromArchive == Package);
	};

	SECTION("Empty")
	{
		FCbPackage Package;
		CHECK(Package.IsNull());
		CHECK_FALSE(Package);
		CHECK(Package.GetAttachments().Num() == 0);
		TestSaveLoadValidate(Package);
	}

	SECTION("Object Only")
	{
		TCbWriter<256> Writer;
		Writer.BeginObject();
		Writer << "Field" << 42;
		Writer.EndObject();

		const FCbObject Object = Writer.Save().AsObject();
		FCbPackage Package(Object);
		CHECK_FALSE(Package.IsNull());
		CHECK(Package);
		CHECK(Package.GetAttachments().Num() == 0);
		CHECK(Package.GetObject().GetOuterBuffer() == Object.GetOuterBuffer());
		CHECK(Package.GetObject()["Field"].AsInt32() == 42);
		CHECK(Package.GetObjectHash() == Package.GetObject().GetHash());
		TestSaveLoadValidate(Package);
	}

	SECTION("Object View Only")
	{
		TCbWriter<256> Writer;
		Writer.BeginObject();
		Writer << "Field" << 42;
		Writer.EndObject();

		const FCbObject Object = Writer.Save().AsObject();
		FCbPackage Package(FCbObject::MakeView(Object));
		CHECK_FALSE(Package.IsNull());
		CHECK(Package);
		CHECK(Package.GetAttachments().Num() == 0);
		CHECK(Package.GetObject().GetOuterBuffer() != Object.GetOuterBuffer());
		CHECK(Package.GetObject()["Field"].AsInt32() == 42);
		CHECK(Package.GetObjectHash() == Package.GetObject().GetHash());
		TestSaveLoadValidate(Package);
	}

	SECTION("Attachment Only")
	{
		FCbObject Object1;
		{
			TCbWriter<256> Writer;
			Writer.BeginObject();
			Writer << "Field1" << 42;
			Writer.EndObject();
			Object1 = Writer.Save().AsObject();
		}
		FCbObject Object2;
		{
			TCbWriter<256> Writer;
			Writer.BeginObject();
			Writer << "Field2" << 42;
			Writer.EndObject();
			Object2 = Writer.Save().AsObject();
		}

		FCbPackage Package;
		Package.AddAttachment(FCbAttachment(Object1));
		Package.AddAttachment(FCbAttachment(Object2.GetOuterBuffer()));

		CHECK_FALSE(Package.IsNull());
		CHECK(Package);
		CHECK(Package.GetAttachments().Num() == 2);
		CHECK(Package.GetObject().Equals(FCbObject()));
		CHECK(Package.GetObjectHash() == FIoHash::Zero);
		TestSaveLoadValidate(Package);

		const FCbAttachment* const Object1Attachment = Package.FindAttachment(Object1.GetHash());
		const FCbAttachment* const Object2Attachment = Package.FindAttachment(Object2.GetHash());

		CHECK((Object1Attachment && Object1Attachment->AsObject().Equals(Object1)));
		CHECK((Object2Attachment && Object2Attachment->AsBinary() == Object2.GetOuterBuffer()));

		FSharedBuffer Object1ClonedBuffer = FSharedBuffer::Clone(Object1.GetOuterBuffer());
		Package.AddAttachment(FCbAttachment(Object1ClonedBuffer));
		Package.AddAttachment(FCbAttachment(FCbObject::Clone(Object2)));

		CHECK(Package.GetAttachments().Num() == 2);
		CHECK(Package.FindAttachment(Object1.GetHash()) == Object1Attachment);
		CHECK(Package.FindAttachment(Object2.GetHash()) == Object2Attachment);

		CHECK((Object1Attachment && Object1Attachment->AsBinary() == Object1ClonedBuffer));
		CHECK((Object2Attachment && Object2Attachment->AsObject().Equals(Object2)));

		CHECK(Algo::IsSorted(Package.GetAttachments()));
	}

	// Shared Values
	const uint8 Level4Values[]{0, 1, 2, 3};
	FSharedBuffer Level4 = FSharedBuffer::MakeView(MakeMemoryView(Level4Values));
	const FIoHash Level4Hash = FIoHash::HashBuffer(Level4);

	FCbObject Level3;
	{
		TCbWriter<256> Writer;
		Writer.BeginObject();
		Writer.AddBinaryAttachment("Level4", Level4Hash);
		Writer.EndObject();
		Level3 = Writer.Save().AsObject();
	}
	const FIoHash Level3Hash = Level3.GetHash();

	FCbObject Level2;
	{
		TCbWriter<256> Writer;
		Writer.BeginObject();
		Writer.AddObjectAttachment("Level3", Level3Hash);
		Writer.EndObject();
		Level2 = Writer.Save().AsObject();
	}
	const FIoHash Level2Hash = Level2.GetHash();

	FCbObject Level1;
	{
		TCbWriter<256> Writer;
		Writer.BeginObject();
		Writer.AddObjectAttachment("Level2", Level2Hash);
		Writer.EndObject();
		Level1 = Writer.Save().AsObject();
	}
	const FIoHash Level1Hash = Level1.GetHash();

	const auto Resolver = [&Level2, &Level2Hash, &Level3, &Level3Hash, &Level4, &Level4Hash]
		(const FIoHash& Hash) -> FSharedBuffer
		{
			return
				Hash == Level2Hash ? Level2.GetOuterBuffer() :
				Hash == Level3Hash ? Level3.GetOuterBuffer() :
				Hash == Level4Hash ? Level4 :
				FSharedBuffer();
		};

	SECTION("Object + Attachments")
	{
		FCbPackage Package;
		Package.SetObject(Level1, Level1Hash, Resolver);

		CHECK_FALSE(Package.IsNull());
		CHECK(Package);
		CHECK(Package.GetAttachments().Num() == 3);
		CHECK(Package.GetObject().GetOuterBuffer() == Level1.GetOuterBuffer());
		CHECK(Package.GetObjectHash() == Level1Hash);
		TestSaveLoadValidate(Package);

		const FCbAttachment* const Level2Attachment = Package.FindAttachment(Level2Hash);
		const FCbAttachment* const Level3Attachment = Package.FindAttachment(Level3Hash);
		const FCbAttachment* const Level4Attachment = Package.FindAttachment(Level4Hash);
		CHECK((Level2Attachment && Level2Attachment->AsObject().Equals(Level2)));
		CHECK((Level3Attachment && Level3Attachment->AsObject().Equals(Level3)));
		CHECK((
			Level4Attachment &&
			Level4Attachment->AsBinary() != Level4 &&
			Level4Attachment->AsBinary().GetView().EqualBytes(Level4.GetView())));

		CHECK(Algo::IsSorted(Package.GetAttachments()));

		const FCbPackage PackageCopy = Package;
		CHECK(PackageCopy == Package);

		CHECK(Package.RemoveAttachment(Level1Hash) == 0);
		CHECK(Package.RemoveAttachment(Level2Hash) == 1);
		CHECK(Package.RemoveAttachment(Level3Hash) == 1);
		CHECK(Package.RemoveAttachment(Level4Hash) == 1);
		CHECK(Package.RemoveAttachment(Level4Hash) == 0);
		CHECK(Package.GetAttachments().Num() == 0);

		CHECK(PackageCopy != Package);
		Package = PackageCopy;
		CHECK(PackageCopy == Package);
		Package.SetObject(FCbObject());
		CHECK(PackageCopy != Package);
		CHECK(Package.GetObjectHash() == FIoHash::Zero);
	}

	SECTION("Out of Order")
	{
		TCbWriter<384> Writer;
		FCbAttachment Attachment2(Level2, Level2Hash);
		Attachment2.Save(Writer);
		FCbAttachment Attachment4(Level4);
		Attachment4.Save(Writer);
		Writer.AddHash(Level1Hash);
		Writer.AddObject(Level1);
		FCbAttachment Attachment3(Level3, Level3Hash);
		Attachment3.Save(Writer);
		Writer.AddNull();

		FCbFieldIterator Fields = Writer.Save();
		FCbPackage FromFields;
		CHECK(FromFields.TryLoad(Fields));

		const FCbAttachment* const Level2Attachment = FromFields.FindAttachment(Level2Hash);
		const FCbAttachment* const Level3Attachment = FromFields.FindAttachment(Level3Hash);
		const FCbAttachment* const Level4Attachment = FromFields.FindAttachment(Level4Hash);

		CHECK(FromFields.GetObject().Equals(Level1));
		CHECK(FromFields.GetObject().GetOuterBuffer() == Fields.GetOuterBuffer());
		CHECK(FromFields.GetObjectHash() == Level1Hash);

		CHECK((Level2Attachment && Level2Attachment->AsObject().Equals(Level2)));
		CHECK((Level2Attachment && Level2Attachment->GetHash() == Level2Hash));

		CHECK((Level3Attachment && Level3Attachment->AsObject().Equals(Level3)));
		CHECK((Level3Attachment && Level3Attachment->GetHash() == Level3Hash));

		CHECK((Level4Attachment && Level4Attachment->AsBinary().GetView().EqualBytes(Level4.GetView())));
		CHECK((Level4Attachment && Fields.GetOuterBuffer().GetView().Contains(Level4Attachment->AsBinary().GetView())));
		CHECK((Level4Attachment && Level4Attachment->GetHash() == Level4Hash));

		FBufferArchive WriteAr;
		Writer.Save(WriteAr);
		FCbPackage FromArchive;
		FMemoryReader ReadAr(WriteAr);
		CHECK(FromArchive.TryLoad(ReadAr));

		Writer.Reset();
		FromArchive.Save(Writer);
		FCbFieldIterator Saved = Writer.Save();
		FMemoryView View;
		CHECK(Saved.AsHash() == Level1Hash);
		++Saved;
		CHECK(Saved.AsObject().Equals(Level1));
		++Saved;
		CHECK(Saved.AsObjectAttachment() == Level2Hash);
		++Saved;
		CHECK(Saved.AsObject().Equals(Level2));
		++Saved;
		CHECK(Saved.AsObjectAttachment() == Level3Hash);
		++Saved;
		CHECK(Saved.AsObject().Equals(Level3));
		++Saved;
		CHECK(Saved.AsBinaryAttachment() == Level4Hash);
		++Saved;
		FSharedBuffer SavedLevel4Buffer = FSharedBuffer::MakeView(Saved.AsBinaryView());
		CHECK(SavedLevel4Buffer.GetView().EqualBytes(Level4.GetView()));
		++Saved;
		CHECK(Saved.IsNull());
		++Saved;
		CHECK_FALSE(Saved);
	}

	SECTION("Null Attachment")
	{
		const FCbAttachment NullAttachment;
		FCbPackage Package;
		Package.AddAttachment(NullAttachment);
		CHECK(Package.IsNull());
		CHECK_FALSE(Package);
		CHECK(Package.GetAttachments().Num() == 0);
		CHECK_FALSE(Package.FindAttachment(NullAttachment));
	}

	SECTION("Resolve After Merge")
	{
		bool bResolved = false;
		FCbPackage Package;
		Package.AddAttachment(FCbAttachment(Level3.GetOuterBuffer()));
		Package.AddAttachment(FCbAttachment(Level3),
			[&bResolved](const FIoHash& Hash) -> FSharedBuffer
			{
				bResolved = true;
				return FSharedBuffer();
			});
		CHECK(bResolved);
	}
}

TEST_CASE_NAMED(FCompactBinaryInvalidPackageTest, "System::Core::Serialization::CompactBinary::InvalidPackage", "[Core][CompactBinary][SmokeFilter]")
{
	const auto TestLoad = [](std::initializer_list<uint8> RawData, FCbBufferAllocator Allocator = FUniqueBuffer::Alloc)
	{
		const FMemoryView RawView = MakeMemoryView(RawData);
		FCbPackage FromArchive;
		FMemoryReaderView ReadAr(RawView);
		CHECK_FALSE(FromArchive.TryLoad(ReadAr, Allocator));
	};

	const auto AllocFail = [](uint64) -> FUniqueBuffer
	{
		FAIL_CHECK("Allocation is not expected");
		return FUniqueBuffer();
	};

	SECTION("Empty")
	{
		TestLoad({}, AllocFail);
	}

	SECTION("Invalid Initial Field")
	{
		TestLoad({uint8(ECbFieldType::None)});
		TestLoad({uint8(ECbFieldType::Array)});
		TestLoad({uint8(ECbFieldType::UniformArray)});
		TestLoad({uint8(ECbFieldType::Binary)});
		TestLoad({uint8(ECbFieldType::String)});
		TestLoad({uint8(ECbFieldType::IntegerPositive)});
		TestLoad({uint8(ECbFieldType::IntegerNegative)});
		TestLoad({uint8(ECbFieldType::Float32)});
		TestLoad({uint8(ECbFieldType::Float64)});
		TestLoad({uint8(ECbFieldType::BoolFalse)});
		TestLoad({uint8(ECbFieldType::BoolTrue)});
		TestLoad({uint8(ECbFieldType::ObjectAttachment)});
		TestLoad({uint8(ECbFieldType::BinaryAttachment)});
		TestLoad({uint8(ECbFieldType::Uuid)});
		TestLoad({uint8(ECbFieldType::DateTime)});
		TestLoad({uint8(ECbFieldType::TimeSpan)});
		TestLoad({uint8(ECbFieldType::ObjectId)});
		TestLoad({uint8(ECbFieldType::CustomById)});
		TestLoad({uint8(ECbFieldType::CustomByName)});
	}

	SECTION("Size Out Of Bounds")
	{
		TestLoad({uint8(ECbFieldType::Object), 1}, AllocFail);
		TestLoad({uint8(ECbFieldType::Object), 0xff, 0x12, 0x34, 0x56, 0x78, 0x9a, 0xbc, 0xde, 0xf0}, AllocFail);
	}
}

#endif // WITH_TESTS
