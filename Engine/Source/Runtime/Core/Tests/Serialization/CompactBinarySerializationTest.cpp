// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/CompactBinarySerialization.h"
#include "Tests/TestHarnessAdapter.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

#if WITH_TESTS

TEST_CASE_NAMED(FCbMeasureTest, "System::Core::Serialization::MeasureCompactBinary", "[ApplicationContextMask][SmokeFilter]")
{
	const auto TestMeasure = [&](
		const TCHAR* Test,
		std::initializer_list<uint8> Data,
		bool bExpectedValue,
		uint64 ExpectedSize,
		ECbFieldType ExpectedType,
		ECbFieldType ExternalType = ECbFieldType::HasFieldType) -> void
	{
		ECbFieldType ActualType = ECbFieldType(255);
		uint64 ActualSize = ~uint64(0);
		const bool bActualValue = TryMeasureCompactBinary(MakeMemoryView(Data), ActualType, ActualSize, ExternalType);
		CHECK_EQUALS(FString::Printf(TEXT("TryMeasureCompactBinary(%s)"), Test), bActualValue, bExpectedValue);
		CHECK_EQUALS(FString::Printf(TEXT("TryMeasureCompactBinary(%s)->Type"), Test), ActualType, ExpectedType);
		CHECK_EQUALS(FString::Printf(TEXT("TryMeasureCompactBinary(%s)->Size"), Test), ActualSize, ExpectedSize);
		if (ActualSize == ExpectedSize)
		{
			const uint64 MeasureSize = MeasureCompactBinary(MakeMemoryView(Data), ExternalType);
			CHECK_EQUALS(FString::Printf(TEXT("MeasureCompactBinary(%s)"), Test), MeasureSize, bExpectedValue ? ExpectedSize : 0);
		}
	};

	using EType = ECbFieldType;

	TestMeasure(TEXT("Empty"), {}, false, 1, EType::None);

	TestMeasure(TEXT("None"), {uint8(EType::None)}, false, 0, EType::None);
	TestMeasure(TEXT("None, NoType"), {}, false, 0, EType::None, EType::None);
	TestMeasure(TEXT("None, NoType, Name"), {}, false, 0, EType::None, EType::None | EType::HasFieldName);

	TestMeasure(TEXT("Null"), {uint8(EType::Null)}, true, 1, EType::Null);
	TestMeasure(TEXT("Null, NameSize1B"), {uint8(EType::Null | EType::HasFieldName), 30}, true, 32, EType::Null | EType::HasFieldName);
	TestMeasure(TEXT("Null, NameSize2B"), {uint8(EType::Null | EType::HasFieldName), 0x80, 0x80}, true, 131, EType::Null | EType::HasFieldName);
	TestMeasure(TEXT("Null, NameSize2BShort"), {uint8(EType::Null | EType::HasFieldName), 0x80}, false, 3, EType::Null | EType::HasFieldName);
	TestMeasure(TEXT("Null, NameSize3BShort"), {uint8(EType::Null | EType::HasFieldName), 0xc0}, false, 4, EType::Null | EType::HasFieldName);
	TestMeasure(TEXT("Null, MissingName"), {uint8(EType::Null | EType::HasFieldName)}, false, 2, EType::Null | EType::HasFieldName);

	TestMeasure(TEXT("Null, NoType"), {}, true, 0, EType::Null, EType::Null);
	TestMeasure(TEXT("Null, NoType, NameSize1B"), {30}, true, 31, EType::Null | EType::HasFieldName, EType::Null | EType::HasFieldName);
	TestMeasure(TEXT("Null, NoType, NameSize2B"), {0x80, 0x80}, true, 130, EType::Null | EType::HasFieldName, EType::Null | EType::HasFieldName);
	TestMeasure(TEXT("Null, NoType, NameSize2BShort"), {0x80}, false, 2, EType::Null | EType::HasFieldName, EType::Null | EType::HasFieldName);
	TestMeasure(TEXT("Null, NoType, NameSize3BShort"), {0xc0}, false, 3, EType::Null | EType::HasFieldName, EType::Null | EType::HasFieldName);
	TestMeasure(TEXT("Null, NoType, MissingName"), {}, false, 1, EType::Null | EType::HasFieldName, EType::Null | EType::HasFieldName);

	TestMeasure(TEXT("Object, NoSize"), {uint8(EType::Object)}, false, 2, EType::Object);
	TestMeasure(TEXT("Object, Size1B"), {uint8(EType::Object), 30}, true, 32, EType::Object);
	TestMeasure(TEXT("UniformObject, NoSize"), {uint8(EType::UniformObject)}, false, 2, EType::UniformObject);
	TestMeasure(TEXT("UniformObject, Size1B"), {uint8(EType::UniformObject), 30}, true, 32, EType::UniformObject);

	TestMeasure(TEXT("Array, NoSize"), {uint8(EType::Array)}, false, 2, EType::Array);
	TestMeasure(TEXT("Array, Size1B"), {uint8(EType::Array), 30}, true, 32, EType::Array);
	TestMeasure(TEXT("UniformArray, NoSize"), {uint8(EType::UniformArray)}, false, 2, EType::UniformArray);
	TestMeasure(TEXT("UniformArray, Size1B"), {uint8(EType::UniformArray), 30}, true, 32, EType::UniformArray);

	TestMeasure(TEXT("Binary, NoSize"), {uint8(EType::Binary)}, false, 2, EType::Binary);
	TestMeasure(TEXT("Binary, Size1B"), {uint8(EType::Binary), 30}, true, 32, EType::Binary);

	TestMeasure(TEXT("String, NoSize"), {uint8(EType::String)}, false, 2, EType::String);
	TestMeasure(TEXT("String, Size1B"), {uint8(EType::String), 30}, true, 32, EType::String);
	TestMeasure(TEXT("String, Size2B"), {uint8(EType::String), 0x80, 0x80}, true, 131, EType::String);
	TestMeasure(TEXT("String, Size2BShort"), {uint8(EType::String), 0x80}, false, 3, EType::String);
	TestMeasure(TEXT("String, Size3BShort"), {uint8(EType::String), 0xc0}, false, 4, EType::String);

	TestMeasure(TEXT("String, NameNoSize"), {uint8(EType::String | EType::HasFieldName)}, false, 2, EType::String | EType::HasFieldName);
	TestMeasure(TEXT("String, NameSize1BMissingText"), {uint8(EType::String | EType::HasFieldName), 1}, false, 3, EType::String | EType::HasFieldName);
	TestMeasure(TEXT("String, NameSize1B, NoSize"), {uint8(EType::String | EType::HasFieldName), 1, 'A'}, false, 4, EType::String | EType::HasFieldName);
	TestMeasure(TEXT("String, NameSize1B, Size1B"), {uint8(EType::String | EType::HasFieldName), 1, 'A', 30}, true, 34, EType::String | EType::HasFieldName);
	TestMeasure(TEXT("String, NameSize1B, Size2B"), {uint8(EType::String | EType::HasFieldName), 1, 'A', 0x80, 0x80}, true, 133, EType::String | EType::HasFieldName);
	TestMeasure(TEXT("String, NameSize1B, Size2BShort"), {uint8(EType::String | EType::HasFieldName), 1, 'A', 0x80}, false, 5, EType::String | EType::HasFieldName);
	TestMeasure(TEXT("String, NameSize1B, Size3BShort"), {uint8(EType::String | EType::HasFieldName), 1, 'A', 0xc0}, false, 6, EType::String | EType::HasFieldName);
	TestMeasure(TEXT("String, NameSize2BShort"), {uint8(EType::String | EType::HasFieldName), 0x80}, false, 3, EType::String | EType::HasFieldName);
	TestMeasure(TEXT("String, NameSize3BShort"), {uint8(EType::String | EType::HasFieldName), 0xc0}, false, 4, EType::String | EType::HasFieldName);

	TestMeasure(TEXT("IntegerPositive, NoValue"), {uint8(EType::IntegerPositive)}, false, 2, EType::IntegerPositive);
	TestMeasure(TEXT("IntegerPositive, Value1B"), {uint8(EType::IntegerPositive), 0x7f}, true, 2, EType::IntegerPositive);
	TestMeasure(TEXT("IntegerPositive, Value2B"), {uint8(EType::IntegerPositive), 0x80}, true, 3, EType::IntegerPositive);

	TestMeasure(TEXT("IntegerNegative, NoValue"), {uint8(EType::IntegerNegative)}, false, 2, EType::IntegerNegative);
	TestMeasure(TEXT("IntegerNegative, Value1B"), {uint8(EType::IntegerNegative), 0x7f}, true, 2, EType::IntegerNegative);
	TestMeasure(TEXT("IntegerNegative, Value2B"), {uint8(EType::IntegerNegative), 0x80}, true, 3, EType::IntegerNegative);

	TestMeasure(TEXT("Float32"), {uint8(EType::Float32)}, true, 5, EType::Float32);
	TestMeasure(TEXT("Float32, NameSize1B"), {uint8(EType::Float32 | EType::HasFieldName), 30}, true, 36, EType::Float32 | EType::HasFieldName);
	TestMeasure(TEXT("Float32, NameSize2B"), {uint8(EType::Float32 | EType::HasFieldName), 0x80, 0x80}, true, 135, EType::Float32 | EType::HasFieldName);
	TestMeasure(TEXT("Float32, NameSize2BShort"), {uint8(EType::Float32 | EType::HasFieldName), 0x80}, false, 3, EType::Float32 | EType::HasFieldName);
	TestMeasure(TEXT("Float32, MissingName"), {uint8(EType::Float32 | EType::HasFieldName)}, false, 2, EType::Float32 | EType::HasFieldName);

	TestMeasure(TEXT("Float32, NoType"), {}, true, 4, EType::Float32, EType::Float32);
	TestMeasure(TEXT("Float32, NoType, NameSize1B"), {30}, true, 35, EType::Float32 | EType::HasFieldName, EType::Float32 | EType::HasFieldName);
	TestMeasure(TEXT("Float32, NoType, NameSize2B"), {0x80, 0x80}, true, 134, EType::Float32 | EType::HasFieldName, EType::Float32 | EType::HasFieldName);
	TestMeasure(TEXT("Float32, NoType, NameSize2BShort"), {0x80}, false, 2, EType::Float32 | EType::HasFieldName, EType::Float32 | EType::HasFieldName);
	TestMeasure(TEXT("Float32, NoType, MissingName"), {}, false, 1, EType::Float32 | EType::HasFieldName, EType::Float32 | EType::HasFieldName);

	TestMeasure(TEXT("Float64"), {uint8(EType::Float64)}, true, 9, EType::Float64);

	TestMeasure(TEXT("BoolFalse"), {uint8(EType::BoolFalse)}, true, 1, EType::BoolFalse);
	TestMeasure(TEXT("BoolTrue"), {uint8(EType::BoolTrue)}, true, 1, EType::BoolTrue);

	TestMeasure(TEXT("ObjectAttachment"), {uint8(EType::ObjectAttachment)}, true, 21, EType::ObjectAttachment);
	TestMeasure(TEXT("BinaryAttachment"), {uint8(EType::BinaryAttachment)}, true, 21, EType::BinaryAttachment);

	TestMeasure(TEXT("Hash"), {uint8(EType::Hash)}, true, 21, EType::Hash);
	TestMeasure(TEXT("Uuid"), {uint8(EType::Uuid)}, true, 17, EType::Uuid);

	TestMeasure(TEXT("DateTime"), {uint8(EType::DateTime)}, true, 9, EType::DateTime);
	TestMeasure(TEXT("TimeSpan"), {uint8(EType::TimeSpan)}, true, 9, EType::TimeSpan);

	TestMeasure(TEXT("ObjectId"), {uint8(EType::ObjectId)}, true, 13, EType::ObjectId);

	TestMeasure(TEXT("CustomById, Size1B"), {uint8(EType::CustomById), 0x7f}, true, 129, EType::CustomById);
	TestMeasure(TEXT("CustomById, Size2B"), {uint8(EType::CustomById), 0x80, 0x80}, true, 131, EType::CustomById);
	TestMeasure(TEXT("CustomById, Size1BShort"), {uint8(EType::CustomById)}, false, 2, EType::CustomById);
	TestMeasure(TEXT("CustomById, Size2BShort"), {uint8(EType::CustomById), 0x80}, false, 3, EType::CustomById);
	TestMeasure(TEXT("CustomById, Size9BShort"), {uint8(EType::CustomById), 0xff}, false, 10, EType::CustomById);

	TestMeasure(TEXT("CustomByName, Size1B"), {uint8(EType::CustomByName), 0x7f}, true, 129, EType::CustomByName);
	TestMeasure(TEXT("CustomByName, Size2B"), {uint8(EType::CustomByName), 0x80, 0x80}, true, 131, EType::CustomByName);
	TestMeasure(TEXT("CustomByName, Size1BShort"), {uint8(EType::CustomByName)}, false, 2, EType::CustomByName);
	TestMeasure(TEXT("CustomByName, Size2BShort"), {uint8(EType::CustomByName), 0x80}, false, 3, EType::CustomByName);
	TestMeasure(TEXT("CustomByName, Size9BShort"), {uint8(EType::CustomByName), 0xff}, false, 10, EType::CustomByName);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbSaveTest, "System::Core::Serialization::SaveCompactBinary", "[ApplicationContextMask][SmokeFilter]")
{

	const auto TestSave = [&](const TCHAR* Test, auto Value, FMemoryView ExpectedData)
	{
		{
			FBufferArchive WriteAr;
			SaveCompactBinary(WriteAr, Value);
			if (TestEqual(FString::Printf(TEXT("SaveCompactBinary(%s)->Size"), Test), uint64(WriteAr.Num()), ExpectedData.GetSize()) &&
				TestTrue(FString::Printf(TEXT("SaveCompactBinary(%s)->EqualBytes"), Test), ExpectedData.EqualBytes(MakeMemoryView(WriteAr))))
			{
				{
					FMemoryReader ReadAr(WriteAr);
					FCbField Field = LoadCompactBinary(ReadAr);
					CHECK_MESSAGE(FString::Printf(TEXT("LoadCompactBinary(%s)->EqualBytes"), Test), ExpectedData.EqualBytes(Field.GetOuterBuffer().GetView()));
				}
				{
					TArray<uint8, TInlineAllocator<256>> FieldBuffer;
					FMemoryReader ReadAr(WriteAr);
					FCbFieldView FieldView = LoadCompactBinaryView(ReadAr, [&FieldBuffer](uint64 Size)
					{
						FieldBuffer.SetNumUninitialized(IntCastChecked<int32>(Size));
						return MakeMemoryView(FieldBuffer);
					});
					CHECK_MESSAGE(FString::Printf(TEXT("LoadCompactBinaryView(%s)->EqualBytes"), Test), ExpectedData.EqualBytes(MakeMemoryView(FieldBuffer)));
				}
			}
		}
		{
			FBufferArchive WriteAr;
			WriteAr << Value;
			if (TestEqual(FString::Printf(TEXT("Ar << CompactBinary Save(%s)->Size"), Test), uint64(WriteAr.Num()), ExpectedData.GetSize()) &&
				TestTrue(FString::Printf(TEXT("Ar << CompactBinary Save(%s)->EqualBytes"), Test), ExpectedData.EqualBytes(MakeMemoryView(WriteAr))))
			{
				FMemoryReader ReadAr(WriteAr);
				ReadAr << Value;
				CHECK_MESSAGE(FString::Printf(TEXT("Ar << CompactBinary Load(%s)->EqualBytes"), Test), ExpectedData.EqualBytes(Value.GetOuterBuffer().GetView()));
			}
		}
	};

	// Field
	{
		const uint8 Value[] = { uint8(ECbFieldType::IntegerPositive), 42 };
		TestSave(TEXT("Field"), FCbField(FCbFieldView(Value), FSharedBuffer()), MakeMemoryView(Value));
	}
	{
		const uint8 Value[] = { uint8(ECbFieldType::IntegerPositive | ECbFieldType::HasFieldName), 1, 'A', 42 };
		TestSave(TEXT("Field, Name"), FCbField(FCbFieldView(Value), FSharedBuffer()), MakeMemoryView(Value));
	}
	{
		const uint8 Value[] = { 42 };
		TestSave(TEXT("Field, NoType"), FCbField(FCbFieldView(Value, ECbFieldType::IntegerPositive), FSharedBuffer()),
			MakeMemoryView<uint8>({ uint8(ECbFieldType::IntegerPositive), 42 }));
	}
	{
		const uint8 Value[] = { 1, 'I', 42 };
		TestSave(TEXT("Field, NoType, Name"), FCbField(FCbFieldView(Value, ECbFieldType::IntegerPositive | ECbFieldType::HasFieldName), FSharedBuffer()),
			MakeMemoryView<uint8>({ uint8(ECbFieldType::IntegerPositive | ECbFieldType::HasFieldName), 1, 'I', 42 }));
	}

	// Array
	{
		constexpr uint8 IntType = uint8(ECbFieldType::IntegerPositive);
		const uint8 Value[] = { uint8(ECbFieldType::UniformArray), 5, 3, IntType, 1, 2, 3 };
		TestSave(TEXT("Array"), FCbArray(FCbArrayView(Value), FSharedBuffer()),
			MakeMemoryView(Value));
	}
	{
		constexpr uint8 IntType = uint8(ECbFieldType::IntegerPositive);
		const uint8 Value[] = { uint8(ECbFieldType::UniformArray | ECbFieldType::HasFieldName), 1, 'A', 5, 3, IntType, 1, 2, 3 };
		TestSave(TEXT("Array, Name"), FCbArray(FCbArrayView(Value), FSharedBuffer()),
			MakeMemoryView<uint8>({ uint8(ECbFieldType::UniformArray), 5, 3, IntType, 1, 2, 3 }));
	}
	{
		constexpr uint8 IntType = uint8(ECbFieldType::IntegerPositive);
		const uint8 Value[] = { 5, 3, IntType, 1, 2, 3 };
		TestSave(TEXT("Array, NoType"), FCbArray(FCbArrayView(Value, ECbFieldType::UniformArray), FSharedBuffer()),
			MakeMemoryView<uint8>({ uint8(ECbFieldType::UniformArray), 5, 3, IntType, 1, 2, 3 }));
	}
	{
		constexpr uint8 IntType = uint8(ECbFieldType::IntegerPositive);
		const uint8 Value[] = { 1, 'A', 5, 3, IntType, 1, 2, 3 };
		TestSave(TEXT("Array, NoType, Name"), FCbArray(FCbArrayView(Value, ECbFieldType::UniformArray | ECbFieldType::HasFieldName), FSharedBuffer()),
			MakeMemoryView<uint8>({ uint8(ECbFieldType::UniformArray), 5, 3, IntType, 1, 2, 3 }));
	}

	// Object
	{
		constexpr uint8 IntType = uint8(ECbFieldType::HasFieldName | ECbFieldType::IntegerPositive);
		const uint8 Value[] = { uint8(ECbFieldType::UniformObject), 10, IntType, 1, 'A', 1, 1, 'B', 2, 1, 'C', 3 };
		TestSave(TEXT("Object"), FCbObject(FCbObjectView(Value), FSharedBuffer()),
			MakeMemoryView(Value));
	}
	{
		constexpr uint8 IntType = uint8(ECbFieldType::HasFieldName | ECbFieldType::IntegerPositive);
		const uint8 Value[] = { uint8(ECbFieldType::UniformObject | ECbFieldType::HasFieldName), 1, 'O', 10, IntType, 1, 'A', 1, 1, 'B', 2, 1, 'C', 3 };
		TestSave(TEXT("Object, Name"), FCbObject(FCbObjectView(Value), FSharedBuffer()),
			MakeMemoryView<uint8>({ uint8(ECbFieldType::UniformObject), 10, IntType, 1, 'A', 1, 1, 'B', 2, 1, 'C', 3 }));
	}
	{
		constexpr uint8 IntType = uint8(ECbFieldType::HasFieldName | ECbFieldType::IntegerPositive);
		const uint8 Value[] = { 10, IntType, 1, 'A', 1, 1, 'B', 2, 1, 'C', 3 };
		TestSave(TEXT("Object, NoType"), FCbObject(FCbObjectView(Value, ECbFieldType::UniformObject), FSharedBuffer()),
			MakeMemoryView<uint8>({ uint8(ECbFieldType::UniformObject), 10, IntType, 1, 'A', 1, 1, 'B', 2, 1, 'C', 3 }));
	}
	{
		constexpr uint8 IntType = uint8(ECbFieldType::HasFieldName | ECbFieldType::IntegerPositive);
		const uint8 Value[] = { 1, 'O', 10, IntType, 1, 'A', 1, 1, 'B', 2, 1, 'C', 3 };
		TestSave(TEXT("Object, NoType, Name"), FCbObject(FCbObjectView(Value, ECbFieldType::UniformObject | ECbFieldType::HasFieldName), FSharedBuffer()),
			MakeMemoryView<uint8>({ uint8(ECbFieldType::UniformObject), 10, IntType, 1, 'A', 1, 1, 'B', 2, 1, 'C', 3 }));
	}

}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#endif // WITH_TESTS
