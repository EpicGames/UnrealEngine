// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Serialization/CompactBinaryValidation.h"
#include "Tests/TestHarnessAdapter.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/CompactBinaryWriter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(FCbValidateTest, "System::Core::Serialization::ValidateCompactBinary", "[ApplicationContextMask][SmokeFilter]")
{
	auto Validate = [](std::initializer_list<uint8> Data, ECbFieldType Type = ECbFieldType::HasFieldType) -> ECbValidateError
	{
		return ValidateCompactBinary(MakeMemoryView(Data), ECbValidateMode::All, Type);
	};
	auto ValidateMode = [](std::initializer_list<uint8> Data, ECbValidateMode Mode, ECbFieldType Type = ECbFieldType::HasFieldType) -> ECbValidateError
	{
		return ValidateCompactBinary(MakeMemoryView(Data), Mode, Type);
	};

	auto AddName = [](ECbFieldType Type) -> uint8 { return uint8(Type | ECbFieldType::HasFieldName); };

	constexpr uint8 NullNoName = uint8(ECbFieldType::Null);
	constexpr uint8 NullWithName = uint8(ECbFieldType::Null | ECbFieldType::HasFieldName);
	constexpr uint8 IntNoName = uint8(ECbFieldType::IntegerPositive);
	constexpr uint8 IntWithName = uint8(ECbFieldType::IntegerPositive | ECbFieldType::HasFieldName);

	// Test OutOfBounds
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Empty)"), Validate({}), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Empty, Mode)"), ValidateMode({}, ECbValidateMode::None), ECbValidateError::None);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Null)"), Validate({ NullNoName }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Null, Name)"), Validate({ NullWithName, 1, 'N' }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Null, Name)"), Validate({ NullWithName }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Null, Name)"), Validate({ NullWithName, 1 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Null, Name)"), Validate({ NullWithName, 0x80 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Null, Name)"), Validate({ NullWithName, 0x80, 128 }), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Object, Empty)"), Validate({ uint8(ECbFieldType::Object), 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Object, Empty, NoType)"), Validate({ 0 }, ECbFieldType::Object), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Object, Field)"), Validate({ uint8(ECbFieldType::Object), 7, NullWithName, 1, 'N', IntWithName, 1, 'I', 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Object, Field, NoType)"), Validate({ 7, NullWithName, 1, 'N', IntWithName, 1, 'I', 0 }, ECbFieldType::Object), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Object)"), Validate({ uint8(ECbFieldType::Object) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Object, NoType)"), Validate({}, ECbFieldType::Object), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Object)"), Validate({ uint8(ECbFieldType::Object), 1 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Object, NoType)"), Validate({ 1 }, ECbFieldType::Object), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Object, OOB Field)"), Validate({ uint8(ECbFieldType::Object), 3, AddName(ECbFieldType::Float32), 1, 'N' }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Object, OOB Field, NoType)"), Validate({ 3, AddName(ECbFieldType::Float32), 1, 'N' }, ECbFieldType::Object), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, UniformObject, Field)"), Validate({ uint8(ECbFieldType::UniformObject), 3, NullWithName, 1, 'N' }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, UniformObject, Field, NoType)"), Validate({ 3, NullWithName, 1, 'N' }, ECbFieldType::UniformObject), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformObject)"), Validate({ uint8(ECbFieldType::UniformObject) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformObject, NoType)"), Validate({}, ECbFieldType::UniformObject), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformObject)"), Validate({ uint8(ECbFieldType::UniformObject), 1 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformObject, NoType)"), Validate({ 1 }, ECbFieldType::UniformObject), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformObject, OOB Field)"), Validate({ uint8(ECbFieldType::UniformObject), 3, AddName(ECbFieldType::Float32), 1, 'N' }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformObject, OOB Field, NoType)"), Validate({ 3, AddName(ECbFieldType::Float32), 1, 'N' }, ECbFieldType::UniformObject), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Array, Empty)"), Validate({ uint8(ECbFieldType::Array), 1, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Array, Empty, NoType)"), Validate({ 1, 0 }, ECbFieldType::Array), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Array, Field)"), Validate({ uint8(ECbFieldType::Array), 4, 2, NullNoName, IntNoName, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Array, Field, NoType)"), Validate({ 4, 2, NullNoName, IntNoName, 0 }, ECbFieldType::Array), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Array)"), Validate({ uint8(ECbFieldType::Array) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Array, NoType)"), Validate({}, ECbFieldType::Array), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Array)"), Validate({ uint8(ECbFieldType::Array), 1 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Array, NoType)"), Validate({ 1 }, ECbFieldType::Array), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Array, OOB Field)"), Validate({ uint8(ECbFieldType::Array), 2, 1, uint8(ECbFieldType::Float32) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Array, OOB Field, NoType)"), Validate({ 2, 1, uint8(ECbFieldType::Float32) }, ECbFieldType::Array), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, UniformArray, Field)"), Validate({ uint8(ECbFieldType::UniformArray), 3, 1, IntNoName, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, UniformArray, Field, NoType)"), Validate({ 3, 1, IntNoName, 0 }, ECbFieldType::UniformArray), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformArray)"), Validate({ uint8(ECbFieldType::UniformArray) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformArray, NoType)"), Validate({}, ECbFieldType::UniformArray), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformArray)"), Validate({ uint8(ECbFieldType::UniformArray), 1 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformArray, NoType)"), Validate({ 1 }, ECbFieldType::UniformArray), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformArray, OOB Field)"), Validate({ uint8(ECbFieldType::UniformArray), 2, 1, uint8(ECbFieldType::Float32) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, UniformArray, OOB Field, NoType)"), Validate({ 2, 1, uint8(ECbFieldType::Float32) }, ECbFieldType::UniformArray), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Binary, Empty)"), Validate({ uint8(ECbFieldType::Binary), 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Binary, Empty, NoType)"), Validate({ 0 }, ECbFieldType::Binary), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Binary, Field)"), Validate({ uint8(ECbFieldType::Binary), 1, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Binary, Field, NoType)"), Validate({ 1, 0 }, ECbFieldType::Binary), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Binary)"), Validate({ uint8(ECbFieldType::Binary) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Binary, NoType)"), Validate({}, ECbFieldType::Binary), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Binary)"), Validate({ uint8(ECbFieldType::Binary), 1 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Binary, NoType)"), Validate({ 1 }, ECbFieldType::Binary), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, String, Empty)"), Validate({ uint8(ECbFieldType::String), 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, String, Empty, NoType)"), Validate({ 0 }, ECbFieldType::String), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, String, Field)"), Validate({ uint8(ECbFieldType::String), 1, 'S' }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, String, Field, NoType)"), Validate({ 1, 'S' }, ECbFieldType::String), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, String)"), Validate({ uint8(ECbFieldType::String) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, String, NoType)"), Validate({}, ECbFieldType::String), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, String)"), Validate({ uint8(ECbFieldType::String), 1 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, String, NoType)"), Validate({ 1 }, ECbFieldType::String), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, IntegerPositive, 1-byte)"), Validate({ uint8(ECbFieldType::IntegerPositive), 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, IntegerPositive, 1-byte, NoType)"), Validate({ 0 }, ECbFieldType::IntegerPositive), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, IntegerPositive, 2-byte)"), Validate({ uint8(ECbFieldType::IntegerPositive), 0x80, 0x80 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, IntegerPositive, 2-byte, NoType)"), Validate({ 0x80, 0x80 }, ECbFieldType::IntegerPositive), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerPositive, 1-byte)"), Validate({ uint8(ECbFieldType::IntegerPositive) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerPositive, 1-byte, NoType)"), Validate({}, ECbFieldType::IntegerPositive), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerPositive, 2-byte)"), Validate({ uint8(ECbFieldType::IntegerPositive), 0x80 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerPositive, 2-byte, NoType)"), Validate({ 0x80 }, ECbFieldType::IntegerPositive), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerPositive, 9-byte)"), Validate({ uint8(ECbFieldType::IntegerPositive), 0xff, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerPositive, 9-byte, NoType)"), Validate({ 0xff, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::IntegerPositive), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, IntegerNegative, 1-byte)"), Validate({ uint8(ECbFieldType::IntegerNegative), 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, IntegerNegative, 1-byte, NoType)"), Validate({ 0 }, ECbFieldType::IntegerNegative), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, IntegerNegative, 2-byte)"), Validate({ uint8(ECbFieldType::IntegerNegative), 0x80, 0x80 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, IntegerNegative, 2-byte, NoType)"), Validate({ 0x80, 0x80 }, ECbFieldType::IntegerNegative), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerNegative, 1-byte)"), Validate({ uint8(ECbFieldType::IntegerNegative) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerNegative, 1-byte, NoType)"), Validate({}, ECbFieldType::IntegerNegative), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerNegative, 2-byte)"), Validate({ uint8(ECbFieldType::IntegerNegative), 0x80 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerNegative, 2-byte, NoType)"), Validate({ 0x80 }, ECbFieldType::IntegerNegative), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerNegative, 9-byte)"), Validate({ uint8(ECbFieldType::IntegerNegative), 0xff, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, IntegerNegative, 9-byte, NoType)"), Validate({ 0xff, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::IntegerNegative), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Float32)"), Validate({ uint8(ECbFieldType::Float32), 0, 0, 0, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Float32, NoType)"), Validate({ 0, 0, 0, 0 }, ECbFieldType::Float32), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Float32)"), Validate({ uint8(ECbFieldType::Float32), 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Float32, NoType)"), Validate({ 0, 0, 0 }, ECbFieldType::Float32), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Float64)"), Validate({ uint8(ECbFieldType::Float64), 0x3f, 0xff, 0xff, 0xff, 0xf0, 0x00, 0x00, 0x00 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Float64, NoType)"), Validate({ 0x3f, 0xff, 0xff, 0xff, 0xf0, 0x00, 0x00, 0x00 }, ECbFieldType::Float64), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Float64)"), Validate({ uint8(ECbFieldType::Float64), 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Float64, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::Float64), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, BoolFalse)"), Validate({ uint8(ECbFieldType::BoolFalse) }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, BoolTrue)"), Validate({ uint8(ECbFieldType::BoolTrue) }), ECbValidateError::None);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, ObjectAttachment)"), Validate({ uint8(ECbFieldType::ObjectAttachment), 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, ObjectAttachment, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::ObjectAttachment), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, ObjectAttachment)"), Validate({ uint8(ECbFieldType::ObjectAttachment), 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, ObjectAttachment, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::ObjectAttachment), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, BinaryAttachment)"), Validate({ uint8(ECbFieldType::BinaryAttachment), 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, BinaryAttachment, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::BinaryAttachment), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, BinaryAttachment)"), Validate({ uint8(ECbFieldType::BinaryAttachment), 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, BinaryAttachment, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::BinaryAttachment), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Hash)"), Validate({ uint8(ECbFieldType::Hash), 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Hash, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::Hash), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Hash)"), Validate({ uint8(ECbFieldType::Hash), 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Hash, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::Hash), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Uuid)"), Validate({ uint8(ECbFieldType::Uuid), 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, Uuid, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::Uuid), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Uuid)"), Validate({ uint8(ECbFieldType::Uuid), 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, Uuid, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::Uuid), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, DateTime)"), Validate({ uint8(ECbFieldType::DateTime), 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, DateTime, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::DateTime), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, DateTime)"), Validate({ uint8(ECbFieldType::DateTime), 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, DateTime, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::DateTime), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, TimeSpan)"), Validate({ uint8(ECbFieldType::TimeSpan), 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, TimeSpan, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::TimeSpan), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, TimeSpan)"), Validate({ uint8(ECbFieldType::TimeSpan), 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, TimeSpan, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::TimeSpan), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, ObjectId)"), Validate({ uint8(ECbFieldType::ObjectId), 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, ObjectId, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::ObjectId), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, ObjectId)"), Validate({ uint8(ECbFieldType::ObjectId), 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, ObjectId, NoType)"), Validate({ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }, ECbFieldType::ObjectId), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomById, Empty)"), Validate({ uint8(ECbFieldType::CustomById), 1, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomById, Empty, NoType)"), Validate({ 1, 0 }, ECbFieldType::CustomById), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomById, Value)"), Validate({ uint8(ECbFieldType::CustomById), 2, 0, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomById, Value, NoType)"), Validate({ 2, 0, 0 }, ECbFieldType::CustomById), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomById, MaxTypeId)"), Validate({ uint8(ECbFieldType::CustomById), 9, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomById, MaxTypeId, NoType)"), Validate({ 9, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff }, ECbFieldType::CustomById), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomById, MaxTypeId, Value)"), Validate({ uint8(ECbFieldType::CustomById), 10, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomById, MaxTypeId, Value, NoType)"), Validate({ 10, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0 }, ECbFieldType::CustomById), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomById, NoSize)"), Validate({ uint8(ECbFieldType::CustomById) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomById, NoSize, NoType)"), Validate({}, ECbFieldType::CustomById), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomById, NoTypeId)"), Validate({ uint8(ECbFieldType::CustomById), 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomById, NoTypeId, NoType)"), Validate({ 0 }, ECbFieldType::CustomById), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomById, ShortTypeId)"), Validate({ uint8(ECbFieldType::CustomById), 1, 0x80 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomById, ShortTypeId, NoType)"), Validate({ 1, 0x80 }, ECbFieldType::CustomById), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomById, NoValue)"), Validate({ uint8(ECbFieldType::CustomById), 2, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomById, NoValue, NoType)"), Validate({ 2, 0 }, ECbFieldType::CustomById), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomById, ShortValue)"), Validate({ uint8(ECbFieldType::CustomById), 3, 0, 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomById, ShortValue, NoType)"), Validate({ 3, 0, 0 }, ECbFieldType::CustomById), ECbValidateError::OutOfBounds);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomByName, Empty)"), Validate({ uint8(ECbFieldType::CustomByName), 2, 1, 'A' }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomByName, Empty, NoType)"), Validate({ 2, 1, 'A' }, ECbFieldType::CustomByName), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomByName, Value)"), Validate({ uint8(ECbFieldType::CustomByName), 3, 1, 'A', 0 }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Valid, CustomByName, Value, NoType)"), Validate({ 3, 1, 'A', 0 }, ECbFieldType::CustomByName), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, NoSize)"), Validate({ uint8(ECbFieldType::CustomByName) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, NoSize, NoType)"), Validate({}, ECbFieldType::CustomByName), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, NoTypeSize)"), Validate({ uint8(ECbFieldType::CustomByName), 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, NoTypeSize, NoType)"), Validate({ 0 }, ECbFieldType::CustomByName), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, ShortTypeSize)"), Validate({ uint8(ECbFieldType::CustomByName), 1, 0x80 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, ShortTypeSize, NoType)"), Validate({ 1, 0x80 }, ECbFieldType::CustomByName), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, ShortTypeName)"), Validate({ uint8(ECbFieldType::CustomByName), 1, 1 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, ShortTypeName, NoType)"), Validate({ 1, 1 }, ECbFieldType::CustomByName), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, NoValue)"), Validate({ uint8(ECbFieldType::CustomByName), 3, 1, 'A' }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, NoValue, NoType)"), Validate({ 3, 1, 'A' }, ECbFieldType::CustomByName), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, ShortValue)"), Validate({ uint8(ECbFieldType::CustomByName), 4, 1, 'A', 0 }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(OutOfBounds, CustomByName, ShortValue, NoType)"), Validate({ 4, 1, 'A', 0 }, ECbFieldType::CustomByName), ECbValidateError::OutOfBounds);

	// Test InvalidType
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidType, Unknown)"), Validate({ uint8(ECbFieldType::ObjectId) + 1 }), ECbValidateError::InvalidType);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidType, Unknown)"), Validate({}, ECbFieldType(uint8(ECbFieldType::ObjectId) + 1)), ECbValidateError::InvalidType);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidType, HasFieldType)"), Validate({ uint8(ECbFieldType::Null | ECbFieldType::HasFieldType) }), ECbValidateError::InvalidType);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidType, ZeroSizeField)"), Validate({}, ECbFieldType::Null), ECbValidateError::InvalidType);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidType, ZeroSizeField, BoolFalse)"), Validate({}, ECbFieldType::BoolFalse), ECbValidateError::InvalidType);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidType, ZeroSizeField, BoolTrue)"), Validate({}, ECbFieldType::BoolTrue), ECbValidateError::InvalidType);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidType, ZeroSizeField, Array)"), Validate({ uint8(ECbFieldType::UniformArray), 2, 2, NullNoName }), ECbValidateError::InvalidType);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidType, ZeroSizeField, Object)"), Validate({ uint8(ECbFieldType::UniformObject), 2, NullNoName, 0 }), ECbValidateError::InvalidType);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidType, CustomByName, EmptyTypeName)"), Validate({ uint8(ECbFieldType::CustomByName), 1, 0 }), ECbValidateError::InvalidType);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidType, CustomByName, EmptyTypeName, NoType)"), Validate({ 1, 0 }, ECbFieldType::CustomByName), ECbValidateError::InvalidType);

	// Test DuplicateName
	CHECK_EQUALS(TEXT("ValidateCompactBinary(DuplicateName)"), Validate({ uint8(ECbFieldType::UniformObject), 7, NullWithName, 1, 'A', 1, 'B', 1, 'A' }), ECbValidateError::DuplicateName);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(DuplicateName, CaseSensitive)"), Validate({ uint8(ECbFieldType::UniformObject), 7, NullWithName, 1, 'A', 1, 'B', 1, 'a' }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(DuplicateName, Mode)"), ValidateMode({ uint8(ECbFieldType::UniformObject), 7, NullWithName, 1, 'A', 1, 'B', 1, 'A' }, ECbValidateMode::All & ~ECbValidateMode::Names), ECbValidateError::None);

	// Test MissingName
	CHECK_EQUALS(TEXT("ValidateCompactBinary(MissingName)"), Validate({ uint8(ECbFieldType::Object), 3, NullNoName, IntNoName, 0 }), ECbValidateError::MissingName);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(MissingName, Uniform)"), Validate({ uint8(ECbFieldType::UniformObject), 3, IntNoName, 0, 0 }), ECbValidateError::MissingName);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(MissingName, Mode)"), ValidateMode({ uint8(ECbFieldType::Object), 3, NullNoName, IntNoName, 0 }, ECbValidateMode::All & ~ECbValidateMode::Names), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(MissingName, Uniform, Mode)"), ValidateMode({ uint8(ECbFieldType::UniformObject), 3, IntNoName, 0, 0 }, ECbValidateMode::All & ~ECbValidateMode::Names), ECbValidateError::None);

	// Test ArrayName
	CHECK_EQUALS(TEXT("ValidateCompactBinary(ArrayName)"), Validate({ uint8(ECbFieldType::Array), 5, 2, NullNoName, NullWithName, 1, 'F' }), ECbValidateError::ArrayName);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(ArrayName, Uniform)"), Validate({ uint8(ECbFieldType::UniformArray), 4, 1, NullWithName, 1, 'F' }), ECbValidateError::ArrayName);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(ArrayName, Mode)"), ValidateMode({ uint8(ECbFieldType::Array), 5, 2, NullNoName, NullWithName, 1, 'F' }, ECbValidateMode::All & ~ECbValidateMode::Names), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(ArrayName, Uniform, Mode)"), ValidateMode({ uint8(ECbFieldType::UniformArray), 4, 1, NullWithName, 1, 'F' }, ECbValidateMode::All & ~ECbValidateMode::Names), ECbValidateError::None);

	// Test InvalidString
	// Not tested or implemented yet because the engine does not provide enough UTF-8 functionality.

	// Test InvalidInteger
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, NameSize)"), Validate({ NullWithName, 0x80, 1, 'N' }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, ObjectSize)"), Validate({ uint8(ECbFieldType::Object), 0xc0, 0, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, ArraySize)"), Validate({ uint8(ECbFieldType::Array), 0xe0, 0, 0, 1, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, ArrayCount)"), Validate({ uint8(ECbFieldType::Array), 5, 0xf0, 0, 0, 0, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, BinarySize)"), Validate({ uint8(ECbFieldType::Binary), 0xf8, 0, 0, 0, 0, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, StringSize)"), Validate({ uint8(ECbFieldType::String), 0xfc, 0, 0, 0, 0, 0, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, IntegerPositive)"), Validate({ uint8(ECbFieldType::IntegerPositive), 0xfe, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, IntegerNegative)"), Validate({ uint8(ECbFieldType::IntegerNegative), 0xff, 0, 0, 0, 0, 0, 0, 0, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, ArraySize)"), Validate({ uint8(ECbFieldType::Array), 0x80, 1, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, ArrayCount)"), Validate({ uint8(ECbFieldType::Array), 3, 0xc0, 0, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, ObjectSize)"), Validate({ uint8(ECbFieldType::Object), 0xe0, 0, 0, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, CustomById)"), Validate({ uint8(ECbFieldType::CustomById), 2, 0x80, 0 }), ECbValidateError::InvalidInteger);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, CustomByName)"), Validate({ uint8(ECbFieldType::CustomByName), 3, 0x80, 1, 'A' }), ECbValidateError::InvalidInteger);

	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, NameSize, Mode)"), ValidateMode({ NullWithName, 0x80, 1, 'N' }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, ArraySize, Mode)"), ValidateMode({ uint8(ECbFieldType::Array), 0xc0, 0, 1, 0 }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, ObjectSize, Mode)"), ValidateMode({ uint8(ECbFieldType::Object), 0xe0, 0, 0, 0 }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, CustomById, Mode)"), ValidateMode({ uint8(ECbFieldType::CustomById), 2, 0x80, 0 }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidInteger, CustomByName, Mode)"), ValidateMode({ uint8(ECbFieldType::CustomByName), 3, 0x80, 1, 'A' }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None);

	// Test InvalidFloat
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidFloat, MaxSignificant+1)"), Validate({ uint8(ECbFieldType::Float64), 0x3f, 0xff, 0xff, 0xff, 0xf0, 0x00, 0x00, 0x00 }), ECbValidateError::None); // 1.9999999403953552
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidFloat, MaxExponent+1)"), Validate({ uint8(ECbFieldType::Float64), 0x47, 0xff, 0xff, 0xff, 0xe0, 0x00, 0x00, 0x00 }), ECbValidateError::None); // 6.8056469327705771e38
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidFloat, MaxSignificand)"), Validate({ uint8(ECbFieldType::Float64), 0x3f, 0xff, 0xff, 0xff, 0xe0, 0x00, 0x00, 0x00 }), ECbValidateError::InvalidFloat); // 1.9999998807907104
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidFloat, MaxExponent)"), Validate({ uint8(ECbFieldType::Float64), 0x47, 0xef, 0xff, 0xff, 0xe0, 0x00, 0x00, 0x00 }), ECbValidateError::InvalidFloat); // 3.4028234663852886e38
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidFloat, MaxSignificand, Mode)"), ValidateMode({ uint8(ECbFieldType::Float64), 0x3f, 0xff, 0xff, 0xff, 0xe0, 0x00, 0x00, 0x00 }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None); // 1.9999998807907104
	CHECK_EQUALS(TEXT("ValidateCompactBinary(InvalidFloat, MaxExponent, Mode)"), ValidateMode({ uint8(ECbFieldType::Float64), 0x47, 0xef, 0xff, 0xff, 0xe0, 0x00, 0x00, 0x00 }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None); // 3.4028234663852886e38

	// Test NonUniformObject
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformObject)"), Validate({ uint8(ECbFieldType::Object), 3, NullWithName, 1, 'A' }), ECbValidateError::NonUniformObject);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformObject)"), Validate({ uint8(ECbFieldType::Object), 6, NullWithName, 1, 'A', NullWithName, 1, 'B' }), ECbValidateError::NonUniformObject);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformObject, Mode)"), ValidateMode({ uint8(ECbFieldType::Object), 3, NullWithName, 1, 'A' }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformObject, Mode)"), ValidateMode({ uint8(ECbFieldType::Object), 6, NullWithName, 1, 'A', NullWithName, 1, 'B' }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None);

	// Test NonUniformArray
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformArray)"), Validate({ uint8(ECbFieldType::Array), 3, 1, IntNoName, 0 }), ECbValidateError::NonUniformArray);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformArray)"), Validate({ uint8(ECbFieldType::Array), 5, 2, IntNoName, 1, IntNoName, 2 }), ECbValidateError::NonUniformArray);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformArray, Null)"), Validate({ uint8(ECbFieldType::Array), 3, 2, NullNoName, NullNoName }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformArray, Bool)"), Validate({ uint8(ECbFieldType::Array), 3, 2, uint8(ECbFieldType::BoolFalse), uint8(ECbFieldType::BoolFalse) }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformArray, Bool)"), Validate({ uint8(ECbFieldType::Array), 3, 2, uint8(ECbFieldType::BoolTrue), uint8(ECbFieldType::BoolTrue) }), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformArray, Mode)"), ValidateMode({ uint8(ECbFieldType::Array), 3, 1, IntNoName, 0 }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(NonUniformArray, Mode)"), ValidateMode({ uint8(ECbFieldType::Array), 5, 2, IntNoName, 1, IntNoName, 2 }, ECbValidateMode::All & ~ECbValidateMode::Format), ECbValidateError::None);

	// Test Padding
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Padding)"), Validate({ NullNoName, 0 }), ECbValidateError::Padding);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Padding)"), Validate({ uint8(ECbFieldType::Array), 1, 0, 0 }), ECbValidateError::Padding);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Padding)"), Validate({ uint8(ECbFieldType::Object), 0, 0 }), ECbValidateError::Padding);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Padding, Mode)"), ValidateMode({ NullNoName, 0 }, ECbValidateMode::All & ~ECbValidateMode::Padding), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Padding, Mode)"), ValidateMode({ uint8(ECbFieldType::Array), 1, 0, 0 }, ECbValidateMode::All & ~ECbValidateMode::Padding), ECbValidateError::None);
	CHECK_EQUALS(TEXT("ValidateCompactBinary(Padding, Mode)"), ValidateMode({ uint8(ECbFieldType::Object), 0, 0 }, ECbValidateMode::All & ~ECbValidateMode::Padding), ECbValidateError::None);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbValidateRangeTest, "System::Core::Serialization::ValidateCompactBinaryRange", "[ApplicationContextMask][SmokeFilter]")
{
	auto Validate = [](std::initializer_list<uint8> Data, ECbValidateMode Mode = ECbValidateMode::All) -> ECbValidateError
	{
		return ValidateCompactBinaryRange(MakeMemoryView(Data), Mode);
	};
	// Test Empty
	CHECK_EQUALS(TEXT("ValidateCompactBinaryRange(Empty)"), Validate({}), ECbValidateError::None);

	// Test Valid
	CHECK_EQUALS(TEXT("ValidateCompactBinaryRange(Null x2)"), Validate({ uint8(ECbFieldType::Null), uint8(ECbFieldType::Null) }), ECbValidateError::None);

	// Test Padding
	CHECK_EQUALS(TEXT("ValidateCompactBinaryRange(Padding InvalidType)"), Validate({ uint8(ECbFieldType::Null), 0 }), ECbValidateError::InvalidType);
	CHECK_EQUALS(TEXT("ValidateCompactBinaryRange(Padding OutOfBounds)"), Validate({ uint8(ECbFieldType::Null), uint8(ECbFieldType::Binary) }), ECbValidateError::OutOfBounds);
	CHECK_EQUALS(TEXT("ValidateCompactBinaryRange(Padding OutOfBounds, Mode)"), Validate({ uint8(ECbFieldType::Null), uint8(ECbFieldType::Binary) }, ECbValidateMode::None), ECbValidateError::None);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbValidateAttachmentTest, "System::Core::Serialization::ValidateCompactBinaryAttachment", "[ApplicationContextMask][SmokeFilter]")
{
	auto Validate = [](std::initializer_list<uint8> Data, ECbValidateMode Mode = ECbValidateMode::All) -> ECbValidateError
	{
		return ValidateCompactBinaryAttachment(MakeMemoryView(Data), Mode);
	};

	const uint8 BinaryValue[] = { 0, 1, 2, 3 };
	const FMemoryView BinaryView = MakeMemoryView(BinaryValue);

	// Test Null
	{
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Null)"), Validate({ uint8(ECbFieldType::Binary), 0 }, ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Null, Padding)"), Validate({ uint8(ECbFieldType::Binary), 0, 0 }, ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::Padding);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Null, Padding, Mode)"), Validate({ uint8(ECbFieldType::Binary), 0, 0 }, ECbValidateMode::All & ~(ECbValidateMode::Padding | ECbValidateMode::Package)), ECbValidateError::None);
	}

	// Test Binary
	{
		FBufferArchive Buffer;
		FCbAttachment(FSharedBuffer::MakeView(MakeMemoryView<uint8>({ 0 }))).Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Binary)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::None);
		++Buffer[2];
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Binary, InvalidHash)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::InvalidPackageHash);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Binary, InvalidHash, Mode)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::PackageHash), ECbValidateError::None);
		--Buffer[2];
		Buffer.AddZeroed(1);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Binary, Padding)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::Padding);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Binary, Padding, Mode)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Padding), ECbValidateError::None);
		Buffer.Pop(EAllowShrinking::No);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Binary, OutOfBoundsValue)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer).Left(1), ECbValidateMode::All), ECbValidateError::OutOfBounds);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Binary, OutOfBoundsHash)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer).Left(4), ECbValidateMode::All), ECbValidateError::OutOfBounds);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Binary, OutOfBoundsHash, Mode)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer).Left(4), ECbValidateMode::None), ECbValidateError::None);
	}

	// Test Object
	{
		TCbWriter<256> Writer;
		Writer.BeginObject();
		Writer.AddNull(ANSITEXTVIEW("N"));
		Writer.EndObject();
		FBufferArchive Buffer;
		FCbAttachment(Writer.Save().AsObject()).Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Object)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::None);
		++Buffer[2];
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Object, InvalidHash)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::InvalidPackageHash);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Object, InvalidHash, Mode)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::PackageHash), ECbValidateError::None);
		--Buffer[2];
		Buffer.AddZeroed(1);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Object, Padding)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::Padding);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Object, Padding, Mode)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Padding), ECbValidateError::None);
		Buffer.Pop(EAllowShrinking::No);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Object, OutOfBoundsValue)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer).Left(1), ECbValidateMode::All), ECbValidateError::OutOfBounds);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Object, OutOfBoundsHash)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer).Left(8), ECbValidateMode::All), ECbValidateError::OutOfBounds);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Object, OutOfBoundsHash, Mode)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer).Left(8), ECbValidateMode::None), ECbValidateError::None);
	}

	// Test Empty Object
	{
		FBufferArchive Buffer;
		FCbFieldIterator::MakeSingle(FCbObject().AsField()).CopyRangeTo(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Object)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::None);
	}

	// Test InvalidPackageFormat
	{
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Empty)"), Validate({}), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(Empty, Mode)"), Validate({}, ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		FBufferArchive Buffer;
		FCbWriter Writer;
		Writer.AddBinaryAttachment(FIoHash::HashBuffer(BinaryView));
		Writer.SetName(ANSITEXTVIEW("Name")).AddBinary(BinaryView);
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(NameOnValue)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(NameOnValue, Mode)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		FBufferArchive Buffer;
		FCbWriter Writer;
		Writer.SetName(ANSITEXTVIEW("Name")).AddBinaryAttachment(FIoHash::HashBuffer(BinaryView));
		Writer.AddBinary(BinaryView);
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(NameOnHash)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryAttachment(NameOnHash, Mode)"), ValidateCompactBinaryAttachment(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbValidatePackageTest, "System::Core::Serialization::ValidateCompactBinaryPackage", "[ApplicationContextMask][SmokeFilter]")
{
	const uint8 BinaryValue[] = { 0, 1, 2, 3 };
	const FMemoryView BinaryView = MakeMemoryView(BinaryValue);
	const FSharedBuffer BinaryBuffer = FSharedBuffer::MakeView(BinaryView);
	FIoHash BinaryBufferHash = FIoHash::HashBuffer(BinaryBuffer);
	const FCompressedBuffer CompressedBinaryBuffer = FCompressedBuffer::Compress(FSharedBuffer::MakeView(BinaryView));
	
	TCbWriter<32> CommonRootObjectWriter;
	CommonRootObjectWriter.BeginObject();
	CommonRootObjectWriter << "Field" << 42;
	CommonRootObjectWriter.EndObject();
	FCbObject CommonRootObject(CommonRootObjectWriter.Save().AsObject());
	FIoHash CommonRootObjectHash = CommonRootObject.GetHash();

	// Test Null
	{
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Null)"), ValidateCompactBinaryPackage(MakeMemoryView<uint8>({ uint8(ECbFieldType::Null) }), ECbValidateMode::All), ECbValidateError::None);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Null, Padding)"), ValidateCompactBinaryPackage(MakeMemoryView<uint8>({ uint8(ECbFieldType::Null), 0 }), ECbValidateMode::All), ECbValidateError::Padding);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Null, Padding, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView<uint8>({ uint8(ECbFieldType::Null), 0 }), ECbValidateMode::All & ~ECbValidateMode::Padding), ECbValidateError::None);
	}

	// Test Object
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddHash(CommonRootObjectHash);
		Writer.AddObject(CommonRootObject);
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::None);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, MissingHash)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).LeftChop(34), ECbValidateMode::All), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, MissingHash, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).LeftChop(34), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, MissingNull)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).LeftChop(1), ECbValidateMode::All), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, MissingNull, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).LeftChop(1), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
		++Buffer.Last(1);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, InvalidHash)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::InvalidPackageHash);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, InvalidHash, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::PackageHash), ECbValidateError::None);
		--Buffer.Last(1);
		Buffer.AddZeroed(1);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, Padding)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::Padding);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, Padding, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Padding), ECbValidateError::None);
		Buffer.Pop(EAllowShrinking::No);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, OutOfBoundsValue)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).Left(1), ECbValidateMode::All), ECbValidateError::OutOfBounds);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, OutOfBoundsHash)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).Left(11), ECbValidateMode::All), ECbValidateError::OutOfBounds);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object, OutOfBoundsHash, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).Left(11), ECbValidateMode::None), ECbValidateError::None);
	}

	// Test Object + Attachment
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddHash(CommonRootObjectHash);
		Writer.AddObject(CommonRootObject);
		const uint64 AttachmentOffset = Writer.GetSaveSize();
		Writer.AddBinaryAttachment(BinaryBufferHash);
		Writer.AddBinary(BinaryBuffer);
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+Attachment)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::None);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+Attachment, MissingNull)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).LeftChop(1), ECbValidateMode::All), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+Attachment, MissingNull, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).LeftChop(1), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
		++Buffer[2];
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+Attachment, InvalidHash)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::InvalidPackageHash);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+Attachment, InvalidHash, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::PackageHash), ECbValidateError::None);
		--Buffer[2];
		Buffer.AddZeroed(1);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+Attachment, Padding)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::Padding);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+Attachment, Padding, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Padding), ECbValidateError::None);
		Buffer.Pop(EAllowShrinking::No);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+Attachment, OutOfBoundsValue)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).Left(AttachmentOffset + 1), ECbValidateMode::All), ECbValidateError::OutOfBounds);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+Attachment, OutOfBoundsHash)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).Left(AttachmentOffset + 7), ECbValidateMode::All), ECbValidateError::OutOfBounds);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+Attachment, OutOfBoundsHash, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).Left(AttachmentOffset + 7), ECbValidateMode::None), ECbValidateError::None);
	}

	// Test Object + Compressed Binary Attachment
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddHash(CommonRootObjectHash);
		Writer.AddObject(CommonRootObject);
		const uint64 AttachmentOffset = Writer.GetSaveSize();
		Writer.AddBinary(CompressedBinaryBuffer.GetCompressed());
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+CompressedBinaryAttachment)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::None);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+CompressedBinaryAttachment, MissingNull)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).LeftChop(1), ECbValidateMode::All), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+CompressedBinaryAttachment, MissingNull, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).LeftChop(1), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
		++Buffer[2];
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+CompressedBinaryAttachment, InvalidHash)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::InvalidPackageHash);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+CompressedBinaryAttachment, InvalidHash, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::PackageHash), ECbValidateError::None);
		--Buffer[2];
		Buffer.AddZeroed(1);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+CompressedBinaryAttachment, Padding)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::Padding);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+CompressedBinaryAttachment, Padding, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Padding), ECbValidateError::None);
		Buffer.Pop(EAllowShrinking::No);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+CompressedBinaryAttachment, OutOfBoundsValue)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).Left(AttachmentOffset + 1), ECbValidateMode::All), ECbValidateError::OutOfBounds);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+CompressedBinaryAttachment, OutOfBoundsHash)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).Left(AttachmentOffset + 7), ECbValidateMode::All), ECbValidateError::OutOfBounds);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Object+CompressedBinaryAttachment, OutOfBoundsHash, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer).Left(AttachmentOffset + 7), ECbValidateMode::None), ECbValidateError::None);
	}

	// Test InvalidPackageFormat
	{
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Empty)"), ValidateCompactBinaryPackage(FMemoryView(), ECbValidateMode::All), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(Empty, Mode)"), ValidateCompactBinaryPackage(FMemoryView(), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddHash(CommonRootObjectHash);
		Writer.SetName(ANSITEXTVIEW("Name"));
		Writer.AddObject(CommonRootObject);
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(NameOnObject)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(NameOnObject, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.SetName(ANSITEXTVIEW("Name"));
		Writer.AddHash(CommonRootObjectHash);
		Writer.AddObject(CommonRootObject);
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(NameOnHash)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(NameOnHash, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddHash(CommonRootObjectHash);
		Writer.AddObject(CommonRootObject);
		Writer.AddBinaryAttachment(BinaryBufferHash);
		Writer.SetName(ANSITEXTVIEW("Name")).AddBinary(BinaryBuffer);
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(NameOnAttachment)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::InvalidPackageFormat);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(NameOnAttachment, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddHash(CommonRootObjectHash);
		Writer.AddObject(CommonRootObject);
		Writer.AddHash(CommonRootObjectHash);
		Writer.AddObject(CommonRootObject);
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(MultipleObjects)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::MultiplePackageObjects);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(MultipleObjects, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddBinaryAttachment(BinaryBufferHash);
		Writer.AddBinary(BinaryBuffer);
		Writer.AddBinaryAttachment(BinaryBufferHash);
		Writer.AddBinary(BinaryBuffer);
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(DuplicateAttachments)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::DuplicateAttachments);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(DuplicateAttachments, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddBinary(CompressedBinaryBuffer.GetCompressed());
		Writer.AddBinary(CompressedBinaryBuffer.GetCompressed());
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(DuplicateCompressedAttachments)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::DuplicateAttachments);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(DuplicateCompressedAttachments, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddBinaryAttachment(BinaryBufferHash);
		Writer.AddBinary(BinaryBuffer);
		Writer.AddBinary(CompressedBinaryBuffer.GetCompressed());
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(DuplicateMixedAttachments)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::DuplicateAttachments);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(DuplicateMixedAttachments, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		TCbWriter<32> RootObjectWriter;
		RootObjectWriter.BeginObject();
		RootObjectWriter.EndObject();
		FCbObject NullRootObject(RootObjectWriter.Save().AsObject());
		FIoHash NullRootObjectHash = NullRootObject.GetHash();

		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddHash(NullRootObjectHash);
		Writer.AddObject(NullRootObject);
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(NullObject)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::NullPackageObject);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(NullObject, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
	{
		FBufferArchive Buffer;
		TCbWriter<256> Writer;
		Writer.AddBinaryAttachment(FIoHash::HashBuffer(FMemoryView()));
		Writer.AddBinary(FMemoryView());
		Writer.AddNull();
		Writer.Save(Buffer);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(NullAttachment)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::NullPackageAttachment);
		CHECK_EQUALS(TEXT("ValidateCompactBinaryPackage(NullAttachment, Mode)"), ValidateCompactBinaryPackage(MakeMemoryView(Buffer), ECbValidateMode::All & ~ECbValidateMode::Package), ECbValidateError::None);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#endif // WITH_TESTS
