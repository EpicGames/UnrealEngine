// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Serialization/CompactBinaryWriter.h"
#include "IO/IoHash.h"
#include "Tests/TestHarnessAdapter.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/Timespan.h"
#include "Misc/StringBuilder.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/CompactBinaryValidation.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(FCbWriterObjectTest, "System::Core::Serialization::CbWriter::Object", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	// Test Empty Object
	{
		Writer.Reset();
		Writer.BeginObject();
		Writer.EndObject();
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Object, Empty) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_MESSAGE(TEXT("FCbWriter(Object, Empty).IsObject()"), Field.IsObject());
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Object, Empty).AsObjectView()"), Field.AsObjectView().CreateViewIterator().HasValue());
		}
	}

	// Test Named Empty Object
	{
		Writer.Reset();
		Writer.SetName(ANSITEXTVIEW("Object"));
		Writer.BeginObject();
		Writer.EndObject();
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Object, Empty, Name) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_MESSAGE(TEXT("FCbWriter(Object, Empty, Name).IsObject()"), Field.IsObject());
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Object, Empty, Name).AsObjectView()"), Field.AsObjectView().CreateViewIterator().HasValue());
		}
	}

	// Test Basic Object
	{
		Writer.Reset();
		Writer.BeginObject();
		Writer.SetName(ANSITEXTVIEW("Integer")).AddInteger(0);
		Writer.SetName(ANSITEXTVIEW("Float")).AddFloat(0.0f);
		Writer.EndObject();
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Object, Basic) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_MESSAGE(TEXT("FCbWriter(Object, Basic).IsObject()"), Field.IsObject());
			FCbObjectView Object = Field.AsObjectView();
			CHECK_MESSAGE(TEXT("FCbWriter(Object, Basic).AsObjectView()[Integer]"), Object[ANSITEXTVIEW("Integer")].IsInteger());
			CHECK_MESSAGE(TEXT("FCbWriter(Object, Basic).AsObjectView()[Float]"), Object[ANSITEXTVIEW("Float")].IsFloat());
		}
	}

	// Test Uniform Object
	{
		Writer.Reset();
		Writer.BeginObject();
		Writer.SetName(ANSITEXTVIEW("Field1")).AddInteger(0);
		Writer.SetName(ANSITEXTVIEW("Field2")).AddInteger(1);
		Writer.EndObject();
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Object, Uniform) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_MESSAGE(TEXT("FCbWriter(Object, Uniform).IsObject()"), Field.IsObject());
			FCbObjectView Object = Field.AsObjectView();
			CHECK_MESSAGE(TEXT("FCbWriter(Object, Uniform).AsObjectView()[Field1]"), Object[ANSITEXTVIEW("Field1")].IsInteger());
			CHECK_MESSAGE(TEXT("FCbWriter(Object, Uniform).AsObjectView()[Field2]"), Object[ANSITEXTVIEW("Field2")].IsInteger());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterArrayTest, "System::Core::Serialization::CbWriter::Array", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	// Test Empty Array
	{
		Writer.Reset();
		Writer.BeginArray();
		Writer.EndArray();
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Array, Empty) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_MESSAGE(TEXT("FCbWriter(Array, Empty).IsArray()"), Field.IsArray());
			CHECK_EQUALS(TEXT("FCbWriter(Array, Empty).AsArrayView()"), Field.AsArrayView().Num(), uint64(0));
		}
	}

	// Test Named Empty Array
	{
		Writer.Reset();
		Writer.SetName(ANSITEXTVIEW("Array"));
		Writer.BeginArray();
		Writer.EndArray();
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Array, Empty, Name) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_MESSAGE(TEXT("FCbWriter(Array, Empty, Name).IsArray()"), Field.IsArray());
			CHECK_EQUALS(TEXT("FCbWriter(Array, Empty, Name).AsArrayView()"), Field.AsArrayView().Num(), uint64(0));
		}
	}

	// Test Basic Array
	{
		Writer.Reset();
		Writer.BeginArray();
		Writer.AddInteger(0);
		Writer.AddFloat(0.0f);
		Writer.EndArray();
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Array, Basic) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_MESSAGE(TEXT("FCbWriter(Array, Basic).IsArray()"), Field.IsArray());
			FCbFieldViewIterator Iterator = Field.AsArrayView().CreateViewIterator();
			CHECK_MESSAGE(TEXT("FCbWriter(Array, Basic).AsArrayView()[Integer]"), Iterator.IsInteger());
			++Iterator;
			CHECK_MESSAGE(TEXT("FCbWriter(Array, Basic).AsArrayView()[Float]"), Iterator.IsFloat());
			++Iterator;
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Array, Basic).AsArrayView()[End]"), Iterator.HasValue());
		}
	}

	// Test Uniform Array
	{
		Writer.Reset();
		Writer.BeginArray();
		Writer.AddInteger(0);
		Writer.AddInteger(1);
		Writer.EndArray();
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Array, Uniform) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_MESSAGE(TEXT("FCbWriter(Array, Uniform).IsArray()"), Field.IsArray());
			FCbFieldViewIterator Iterator = Field.AsArrayView().CreateViewIterator();
			CHECK_MESSAGE(TEXT("FCbWriter(Array, Basic).AsArrayView()[Field1]"), Iterator.IsInteger());
			++Iterator;
			CHECK_MESSAGE(TEXT("FCbWriter(Array, Basic).AsArrayView()[Field2]"), Iterator.IsInteger());
			++Iterator;
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Array, Basic).AsArrayView()[End]"), Iterator.HasValue());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterNullTest, "System::Core::Serialization::CbWriter::Null", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	// Test Null
	{
		Writer.Reset();
		Writer.AddNull();
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Null) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Null).HasName()"), Field.HasName());
			CHECK_MESSAGE(TEXT("FCbWriter(Null).IsNull()"), Field.IsNull());
		}
	}

	// Test Null with Name
	{
		Writer.Reset();
		Writer.SetName(ANSITEXTVIEW("Null"));
		Writer.AddNull();
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Null, Name) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_EQUALS(TEXT("FCbWriter(Null, Name).GetName()"), Field.GetName(), UTF8TEXTVIEW("Null"));
			CHECK_MESSAGE(TEXT("FCbWriter(Null, Name).HasName()"), Field.HasName());
			CHECK_MESSAGE(TEXT("FCbWriter(Null, Name).IsNull()"), Field.IsNull());
		}
	}

	// Test Null Array/Object Uniformity
	{
		Writer.Reset();

		Writer.BeginArray();
		Writer.AddNull();
		Writer.AddNull();
		Writer.AddNull();
		Writer.EndArray();

		Writer.BeginObject();
		Writer.SetName(ANSITEXTVIEW("N1")).AddNull();
		Writer.SetName(ANSITEXTVIEW("N2")).AddNull();
		Writer.SetName(ANSITEXTVIEW("N3")).AddNull();
		Writer.EndObject();

		FCbFieldIterator Fields = Writer.Save();
		CHECK_EQUALS(TEXT("FCbWriter(Null, Uniform) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None);
	}

	// Test Null with Save(Buffer)
	{
		Writer.Reset();
		constexpr int NullCount = 3;
		for (int Index = 0; Index < NullCount; ++Index)
		{
			Writer.AddNull();
		}
		uint8 Buffer[NullCount]{};
		FCbFieldViewIterator Fields = Writer.Save(MakeMemoryView(Buffer));
		if (TestEqual(TEXT("FCbWriter(Null, Memory) Validate"), ValidateCompactBinaryRange(MakeMemoryView(Buffer), ECbValidateMode::All), ECbValidateError::None))
		{
			for (int Index = 0; Index < NullCount; ++Index)
			{
				CHECK_MESSAGE(TEXT("FCbWriter(Null, Memory) IsNull"), Fields.IsNull());
				++Fields;
			}
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Null, Memory) HasValue"), Fields.HasValue());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


template <typename T, typename Size>
void TestEqual(const TCHAR* What, TArrayView<T, Size> Actual, TArrayView<T, Size> Expected)
{
	CHECK_MESSAGE(What, Actual.Num() == Expected.Num() && CompareItems(Actual.GetData(), Expected.GetData(), Actual.Num()));
}

TEST_CASE_NAMED(FCbWriterBinaryTest, "System::Core::Serialization::CbWriter::Binary", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	// Test Empty Binary
	{
		Writer.Reset();
		Writer.AddBinary(nullptr, 0);
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Binary, Empty) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Binary, Empty).HasName()"), Field.HasName());
			CHECK_MESSAGE(TEXT("FCbWriter(Binary, Empty).IsBinary()"), Field.IsBinary());
			CHECK_MESSAGE(TEXT("FCbWriter(Binary, Empty).AsBinaryView()"), Field.AsBinaryView().IsEmpty());
		}
	}

	// Test Basic Binary
	{
		Writer.Reset();
		const uint8 BinaryValue[] = { 1, 2, 3, 4, 5, 6 };
		Writer.SetName(ANSITEXTVIEW("Binary"));
		Writer.AddBinary(BinaryValue, sizeof(BinaryValue));
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Binary, Array) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_EQUALS(TEXT("FCbWriter(Binary, Array).GetName()"), Field.GetName(), UTF8TEXTVIEW("Binary"));
			CHECK_MESSAGE(TEXT("FCbWriter(Binary, Array).HasName()"), Field.HasName());
			CHECK_MESSAGE(TEXT("FCbWriter(Binary, Array).IsBinary()"), Field.IsBinary());
			CHECK_MESSAGE(TEXT("FCbWriter(Binary, Array).AsBinaryView()"), Field.AsBinaryView().EqualBytes(MakeMemoryView(BinaryValue)));
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterStringTest, "System::Core::Serialization::CbWriter::String", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	// Test Empty Strings
	{
		Writer.Reset();
		Writer.AddString(FAnsiStringView());
		Writer.AddString(FWideStringView());
		FCbFieldIterator Fields = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(String, Empty) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			for (FCbFieldView Field : Fields)
			{
				CHECK_FALSE_MESSAGE(TEXT("FCbWriter(String, Empty).HasName()"), Field.HasName());
				CHECK_MESSAGE(TEXT("FCbWriter(String, Empty).IsString()"), Field.IsString());
				CHECK_MESSAGE(TEXT("FCbWriter(String, Empty).AsString()"), Field.AsString().IsEmpty());
			}
		}
	}

	// Test Basic Strings
	{
		Writer.Reset();
		Writer.SetName(ANSITEXTVIEW("String")).AddString(ANSITEXTVIEW("Value"));
		Writer.SetName(ANSITEXTVIEW("String")).AddString(TEXTVIEW("Value"));
		FCbFieldIterator Fields = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(String, Basic) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			for (FCbFieldView Field : Fields)
			{
				CHECK_EQUALS(TEXT("FCbWriter(String, Basic).GetName()"), Field.GetName(), UTF8TEXTVIEW("String"));
				CHECK_MESSAGE(TEXT("FCbWriter(String, Basic).HasName()"), Field.HasName());
				CHECK_MESSAGE(TEXT("FCbWriter(String, Basic).IsString()"), Field.IsString());
				CHECK_MESSAGE(TEXT("FCbWriter(String, Basic).AsString()"), Field.AsString() == UTF8TEXTVIEW("Value"));
			}
		}
	}

	// Test Long Strings
	{
		Writer.Reset();
		constexpr int DotCount = 256;
		TUtf8StringBuilder<DotCount + 1> Dots;
		for (int Index = 0; Index < DotCount; ++Index)
		{
			Dots << '.';
		}
		Writer.AddString(Dots);
		Writer.AddString(FString::ChrN(DotCount, TEXT('.')));
		FCbFieldIterator Fields = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(String, Long) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			for (FCbFieldView Field : Fields)
			{
				CHECK_EQUALS(TEXT("FCbWriter(String, Long).AsString()"), Field.AsString(), Dots.ToView());
			}
		}
	}

	// Test Non-ASCII String
	{
		Writer.Reset();
		WIDECHAR Value[2] = { 0xd83d, 0xde00 };
		Writer.AddString(ANSITEXTVIEW("\xf0\x9f\x98\x80"));
		Writer.AddString(FWideStringView(Value, UE_ARRAY_COUNT(Value)));
		FCbFieldIterator Fields = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(String, Unicode) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			for (FCbFieldView Field : Fields)
			{
				CHECK_EQUALS(TEXT("FCbWriter(String, Unicode).AsString()"), Field.AsString(), ANSITEXTVIEW("\xf0\x9f\x98\x80"));
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterIntegerTest, "System::Core::Serialization::CbWriter::Integer", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	auto TestInt32 = [&](int32 Value)
	{
		FCbWriter Writer;
		Writer.Reset();
		Writer.AddInteger(Value);
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Integer, Int32) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_EQUALS(TEXT("FCbWriter(Integer, Int32) Value"), Field.AsInt32(), Value);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Integer, Int32) Error"), Field.HasError());
		}
	};

	auto TestUInt32 = [&](uint32 Value)
	{
		FCbWriter Writer;
		Writer.Reset();
		Writer.AddInteger(Value);
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Integer, UInt32) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_EQUALS(TEXT("FCbWriter(Integer, UInt32) Value"), Field.AsUInt32(), Value);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Integer, UInt32) Error"), Field.HasError());
		}
	};

	auto TestInt64 = [&](int64 Value)
	{
		FCbWriter Writer;
		Writer.Reset();
		Writer.AddInteger(Value);
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Integer, Int64) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_EQUALS(TEXT("FCbWriter(Integer, Int64) Value"), Field.AsInt64(), Value);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Integer, Int64) Error"), Field.HasError());
		}
	};

	auto TestUInt64 = [&](uint64 Value)
	{
		FCbWriter Writer;
		Writer.Reset();
		Writer.AddInteger(Value);
		FCbField Field = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Integer, UInt64) Validate"), ValidateCompactBinary(Field.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_EQUALS(TEXT("FCbWriter(Integer, UInt64) Value"), Field.AsUInt64(), Value);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Integer, UInt64) Error"), Field.HasError());
		}
	};

	TestUInt32(uint32(0x00));
	TestUInt32(uint32(0x7f));
	TestUInt32(uint32(0x80));
	TestUInt32(uint32(0xff));
	TestUInt32(uint32(0x0100));
	TestUInt32(uint32(0x7fff));
	TestUInt32(uint32(0x8000));
	TestUInt32(uint32(0xffff));
	TestUInt32(uint32(0x0001'0000));
	TestUInt32(uint32(0x7fff'ffff));
	TestUInt32(uint32(0x8000'0000));
	TestUInt32(uint32(0xffff'ffff));

	TestUInt64(uint64(0x0000'0001'0000'0000));
	TestUInt64(uint64(0x7fff'ffff'ffff'ffff));
	TestUInt64(uint64(0x8000'0000'0000'0000));
	TestUInt64(uint64(0xffff'ffff'ffff'ffff));

	TestInt32(int32(0x01));
	TestInt32(int32(0x80));
	TestInt32(int32(0x81));
	TestInt32(int32(0x8000));
	TestInt32(int32(0x8001));
	TestInt32(int32(0x7fff'ffff));
	TestInt32(int32(0x8000'0000));
	TestInt32(int32(0x8000'0001));

	TestInt64(int64(0x0000'0001'0000'0000));
	TestInt64(int64(0x8000'0000'0000'0000));
	TestInt64(int64(0x7fff'ffff'ffff'ffff));
	TestInt64(int64(0x8000'0000'0000'0001));
	TestInt64(int64(0xffff'ffff'ffff'ffff));
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterFloatTest, "System::Core::Serialization::CbWriter::Float", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	// Test Float32
	{
		Writer.Reset();
		constexpr float Values[] = { 0.0f, 1.0f, -1.0f, UE_PI };
		for (float Value : Values)
		{
			Writer.AddFloat(Value);
		}
		FCbFieldIterator Fields = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Float, Single) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			const float* CheckValue = Values;
			for (FCbFieldView Field : Fields)
			{
				CHECK_EQUALS(TEXT("FCbWriter(Float, Single).AsFloat()"), Field.AsFloat(), *CheckValue++);
				CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Float, Single) Error"), Field.HasError());
			}
		}
	}

	// Test Float64
	{
		Writer.Reset();
		constexpr double Values[] = { 0.0f, 1.0f, -1.0f, UE_PI, 1.9999998807907104, 1.9999999403953552, 3.4028234663852886e38, 6.8056469327705771e38 };
		for (double Value : Values)
		{
			Writer.AddFloat(Value);
		}
		FCbFieldIterator Fields = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Float, Double) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			const double* CheckValue = Values;
			for (FCbFieldView Field : Fields)
			{
				CHECK_EQUALS(TEXT("FCbWriter(Float, Double).AsDouble()"), Field.AsDouble(), *CheckValue++);
				CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Float, Double) Error"), Field.HasError());
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterBoolTest, "System::Core::Serialization::CbWriter::Bool", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	// Test Bool Values
	{
		Writer.AddBool(true);
		Writer.AddBool(false);

		FCbFieldIterator Fields = Writer.Save();
		if (TestEqual(TEXT("FCbWriter(Bool) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
		{
			CHECK_MESSAGE(TEXT("FCbWriter(Bool).AsBool()"), Fields.AsBool());
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Bool) Error"), Fields.HasError());
			++Fields;
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Bool).AsBool()"), Fields.AsBool());
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Bool) Error"), Fields.HasError());
		}
	}

	// Test Bool Array/Object Uniformity
	{
		Writer.Reset();

		Writer.BeginArray();
		Writer.AddBool(false);
		Writer.AddBool(false);
		Writer.AddBool(false);
		Writer.EndArray();

		Writer.BeginObject();
		Writer.SetName(ANSITEXTVIEW("B1")).AddBool(false);
		Writer.SetName(ANSITEXTVIEW("B2")).AddBool(false);
		Writer.SetName(ANSITEXTVIEW("B3")).AddBool(false);
		Writer.EndObject();

		FCbFieldIterator Fields = Writer.Save();
		CHECK_EQUALS(TEXT("FCbWriter(Bool, Uniform) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterObjectAttachmentTest, "System::Core::Serialization::CbWriter::ObjectAttachment", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	const FIoHash::ByteArray ZeroBytes{};
	const FIoHash::ByteArray SequentialBytes{1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20};

	const FIoHash Values[] = { FIoHash(ZeroBytes), FIoHash(SequentialBytes) };
	for (const FIoHash& Value : Values)
	{
		Writer.AddObjectAttachment(Value);
	}

	FCbFieldIterator Fields = Writer.Save();
	if (TestEqual(TEXT("FCbWriter(ObjectAttachment) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
	{
		const FIoHash* CheckValue = Values;
		for (FCbFieldView Field : Fields)
		{
			CHECK_EQUALS(TEXT("FCbWriter(ObjectAttachment).AsObjectAttachment()"), Field.AsObjectAttachment(), *CheckValue++);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(ObjectAttachment) Error"), Field.HasError());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterBinaryAttachmentTest, "System::Core::Serialization::CbWriter::BinaryAttachment", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	const FIoHash::ByteArray ZeroBytes{};
	const FIoHash::ByteArray SequentialBytes{1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20};

	const FIoHash Values[] = { FIoHash(ZeroBytes), FIoHash(SequentialBytes) };
	for (const FIoHash& Value : Values)
	{
		Writer.AddBinaryAttachment(Value);
	}

	FCbFieldIterator Fields = Writer.Save();
	if (TestEqual(TEXT("FCbWriter(BinaryAttachment) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
	{
		const FIoHash* CheckValue = Values;
		for (FCbFieldView Field : Fields)
		{
			CHECK_EQUALS(TEXT("FCbWriter(BinaryAttachment).AsBinaryAttachment()"), Field.AsBinaryAttachment(), *CheckValue++);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(BinaryAttachment) Error"), Field.HasError());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterHashTest, "System::Core::Serialization::CbWriter::Hash", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	const FIoHash::ByteArray ZeroBytes{};
	const FIoHash::ByteArray SequentialBytes{1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20};

	const FIoHash Values[] = { FIoHash(ZeroBytes), FIoHash(SequentialBytes) };
	for (const FIoHash& Value : Values)
	{
		Writer.AddHash(Value);
	}

	FCbFieldIterator Fields = Writer.Save();
	if (TestEqual(TEXT("FCbWriter(Hash) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
	{
		const FIoHash* CheckValue = Values;
		for (FCbFieldView Field : Fields)
		{
			CHECK_EQUALS(TEXT("FCbWriter(Hash).AsHash()"), Field.AsHash(), *CheckValue++);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Hash) Error"), Field.HasError());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterUuidTest, "System::Core::Serialization::CbWriter::Uuid", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	const FGuid Values[] = { FGuid(), FGuid::NewGuid() };
	for (const FGuid& Value : Values)
	{
		Writer.AddUuid(Value);
	}

	FCbFieldIterator Fields = Writer.Save();
	if (TestEqual(TEXT("FCbWriter(Uuid) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
	{
		const FGuid* CheckValue = Values;
		for (FCbFieldView Field : Fields)
		{
			CHECK_EQUALS(TEXT("FCbWriter(Uuid).AsUuid()"), Field.AsUuid(), *CheckValue++);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(Uuid) Error"), Field.HasError());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterDateTimeTest, "System::Core::Serialization::CbWriter::DateTime", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	const FDateTime Values[] = { FDateTime(0), FDateTime(2020, 5, 13, 15, 10) };
	for (FDateTime Value : Values)
	{
		Writer.AddDateTime(Value);
	}

	FCbFieldIterator Fields = Writer.Save();
	if (TestEqual(TEXT("FCbWriter(DateTime) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
	{
		const FDateTime* CheckValue = Values;
		for (FCbFieldView Field : Fields)
		{
			CHECK_EQUALS(TEXT("FCbWriter(DateTime).AsDateTime()"), Field.AsDateTime(), *CheckValue++);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(DateTime) Error"), Field.HasError());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterTimeSpanTest, "System::Core::Serialization::CbWriter::TimeSpan", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	const FTimespan Values[] = { FTimespan(0), FTimespan(1, 2, 4, 8) };
	for (FTimespan Value : Values)
	{
		Writer.AddTimeSpan(Value);
	}

	FCbFieldIterator Fields = Writer.Save();
	if (TestEqual(TEXT("FCbWriter(TimeSpan) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
	{
		const FTimespan* CheckValue = Values;
		for (FCbFieldView Field : Fields)
		{
			CHECK_EQUALS(TEXT("FCbWriter(TimeSpan).AsTimeSpan()"), Field.AsTimeSpan(), *CheckValue++);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(TimeSpan) Error"), Field.HasError());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterObjectIdTest, "System::Core::Serialization::CbWriter::ObjectId", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	const FCbObjectId Values[] = { FCbObjectId(), FCbObjectId(MakeMemoryView<uint8>({1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12})) };
	for (const FCbObjectId& Value : Values)
	{
		Writer.AddObjectId(Value);
	}

	FCbFieldIterator Fields = Writer.Save();
	if (TestEqual(TEXT("FCbWriter(ObjectId) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
	{
		const FCbObjectId* CheckValue = Values;
		for (FCbFieldView Field : Fields)
		{
			CHECK_EQUALS(TEXT("FCbWriter(ObjectId).AsObjectId()"), Field.AsObjectId(), *CheckValue++);
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(ObjectId) Error"), Field.HasError());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterCustomByIdTest, "System::Core::Serialization::CbWriter::CustomById", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	struct FCustomValue
	{
		uint64 Type;
		TArray<uint8, TInlineAllocator<16>> Bytes;
	};
	const FCustomValue Values[] =
	{
		{ 1, {1, 2, 3} },
		{ MAX_uint64, {4, 5, 6} },
	};

	for (const FCustomValue& Value : Values)
	{
		Writer.AddCustom(Value.Type, MakeMemoryView(Value.Bytes));
	}

	FCbFieldIterator Fields = Writer.Save();
	if (TestEqual(TEXT("FCbWriter(CustomById) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
	{
		const FCustomValue* CheckValue = Values;
		for (FCbFieldView Field : Fields)
		{
			CHECK_MESSAGE(TEXT("FCbWriter(CustomById).AsCustom()"), Field.AsCustom(CheckValue->Type).EqualBytes(MakeMemoryView(CheckValue->Bytes)));
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(CustomById) Error"), Field.HasError());
			++CheckValue;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterCustomByNameTest, "System::Core::Serialization::CbWriter::CustomByName", "[ApplicationContextMask][SmokeFilter]")
{
	TCbWriter<256> Writer;

	struct FCustomValue
	{
		FUtf8StringView Type;
		TArray<uint8, TInlineAllocator<16>> Bytes;
	};
	const FCustomValue Values[] =
	{
		{ UTF8TEXTVIEW("Type1"), {1, 2, 3} },
		{ UTF8TEXTVIEW("Type2"), {4, 5, 6} },
	};

	for (const FCustomValue& Value : Values)
	{
		Writer.AddCustom(Value.Type, MakeMemoryView(Value.Bytes));
	}

	FCbFieldIterator Fields = Writer.Save();
	if (TestEqual(TEXT("FCbWriter(CustomByName) Validate"), ValidateCompactBinaryRange(Fields.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None))
	{
		const FCustomValue* CheckValue = Values;
		for (FCbFieldView Field : Fields)
		{
			CHECK_MESSAGE(TEXT("FCbWriter(CustomByName).AsCustom()"), Field.AsCustom(CheckValue->Type).EqualBytes(MakeMemoryView(CheckValue->Bytes)));
			CHECK_FALSE_MESSAGE(TEXT("FCbWriter(CustomByName) Error"), Field.HasError());
			++CheckValue;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterComplexTest, "System::Core::Serialization::CbWriter::Complex", "[ApplicationContextMask][SmokeFilter]")
{
	FCbObject Object;
	FBufferArchive Archive;
	{
		FCbWriter Writer;
		Writer.BeginObject();

		const uint8 LocalField[] = { uint8(ECbFieldType::IntegerPositive | ECbFieldType::HasFieldName), 1, 'I', 42 };
		Writer.AddField(ANSITEXTVIEW("FieldViewCopy"), FCbFieldView(LocalField));
		Writer.AddField(ANSITEXTVIEW("FieldCopy"), FCbField(FSharedBuffer::Clone(MakeMemoryView(LocalField))));

		const uint8 LocalObject[] = { uint8(ECbFieldType::Object | ECbFieldType::HasFieldName), 1, 'O', 7,
			uint8(ECbFieldType::IntegerPositive | ECbFieldType::HasFieldName), 1, 'I', 42,
			uint8(ECbFieldType::Null | ECbFieldType::HasFieldName), 1, 'N' };
		Writer.AddObject(ANSITEXTVIEW("ObjectViewCopy"), FCbObjectView(LocalObject));
		Writer.AddObject(ANSITEXTVIEW("ObjectCopy"), FCbObject(FSharedBuffer::Clone(MakeMemoryView(LocalObject))));

		const uint8 LocalArray[] = { uint8(ECbFieldType::UniformArray | ECbFieldType::HasFieldName), 1, 'A', 4, 2,
			uint8(ECbFieldType::IntegerPositive), 42, 21 };
		Writer.AddArray(ANSITEXTVIEW("ArrayViewCopy"), FCbArrayView(LocalArray));
		Writer.AddArray(ANSITEXTVIEW("ArrayCopy"), FCbArray(FSharedBuffer::Clone(MakeMemoryView(LocalArray))));

		Writer.AddNull(ANSITEXTVIEW("Null"));

		Writer.BeginObject(ANSITEXTVIEW("Binary"));
		{
			Writer.AddBinary(ANSITEXTVIEW("Empty"), FMemoryView());
			Writer.AddBinary(ANSITEXTVIEW("Value"), MakeMemoryView("BinaryValue"));
			Writer.AddBinary(ANSITEXTVIEW("LargeViewValue"), MakeMemoryView(FString::ChrN(256, TEXT('.'))));
			Writer.AddBinary(ANSITEXTVIEW("LargeValue"), FSharedBuffer::Clone(MakeMemoryView(FString::ChrN(256, TEXT('!')))));
		}
		Writer.EndObject();

		Writer.BeginObject(ANSITEXTVIEW("Strings"));
		{
			Writer.AddString(ANSITEXTVIEW("AnsiString"), ANSITEXTVIEW("AnsiValue"));
			Writer.AddString(ANSITEXTVIEW("WideString"), FString::ChrN(256, TEXT('.')));
			Writer.AddString(ANSITEXTVIEW("EmptyAnsiString"), FAnsiStringView());
			Writer.AddString(ANSITEXTVIEW("EmptyWideString"), FWideStringView());
			Writer.AddString(UTF8TEXTVIEW("EmptyUtf8String"), FUtf8StringView());
			Writer.AddString("AnsiStringLiteral", "AnsiValue");
			Writer.AddString("WideStringLiteral", TEXT("AnsiValue"));
		}
		Writer.EndObject();

		Writer.BeginArray(ANSITEXTVIEW("Integers"));
		{
			Writer.AddInteger(int32(-1));
			Writer.AddInteger(int64(-1));
			Writer.AddInteger(uint32(1));
			Writer.AddInteger(uint64(1));
			Writer.AddInteger(MIN_int32);
			Writer.AddInteger(MAX_int32);
			Writer.AddInteger(MAX_uint32);
			Writer.AddInteger(MIN_int64);
			Writer.AddInteger(MAX_int64);
			Writer.AddInteger(MAX_uint64);
		}
		Writer.EndArray();

		Writer.BeginArray(ANSITEXTVIEW("UniformIntegers"));
		{
			Writer.AddInteger(0);
			Writer.AddInteger(MAX_int32);
			Writer.AddInteger(MAX_uint32);
			Writer.AddInteger(MAX_int64);
			Writer.AddInteger(MAX_uint64);
		}
		Writer.EndArray();

		Writer.AddFloat(ANSITEXTVIEW("Float32"), 1.0f);
		Writer.AddFloat(ANSITEXTVIEW("Float64as32"), 2.0);
		Writer.AddFloat(ANSITEXTVIEW("Float64"), 3.0e100);

		Writer.AddBool(ANSITEXTVIEW("False"), false);
		Writer.AddBool(ANSITEXTVIEW("True"), true);

		Writer.AddObjectAttachment(ANSITEXTVIEW("ObjectAttachment"), FIoHash());
		Writer.AddBinaryAttachment(ANSITEXTVIEW("BinaryAttachment"), FIoHash());
		Writer.AddAttachment(ANSITEXTVIEW("Attachment"), FCbAttachment());

		Writer.AddHash(ANSITEXTVIEW("Hash"), FIoHash());
		Writer.AddUuid(ANSITEXTVIEW("Uuid"), FGuid());

		Writer.AddDateTimeTicks(ANSITEXTVIEW("DateTimeZero"), 0);
		Writer.AddDateTime(ANSITEXTVIEW("DateTime2020"), FDateTime(2020, 5, 13, 15, 10));

		Writer.AddTimeSpanTicks(ANSITEXTVIEW("TimeSpanZero"), 0);
		Writer.AddTimeSpan(ANSITEXTVIEW("TimeSpan"), FTimespan(1, 2, 4, 8));

		Writer.AddObjectId(ANSITEXTVIEW("ObjectIdZero"), FCbObjectId());
		Writer.AddObjectId(ANSITEXTVIEW("ObjectId"), FCbObjectId(MakeMemoryView<uint8>({1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12})));

		Writer.BeginObject(ANSITEXTVIEW("NestedObjects"));
		{
			Writer.BeginObject(ANSITEXTVIEW("Empty"));
			Writer.EndObject();

			Writer.BeginObject(ANSITEXTVIEW("Null"));
			Writer.AddNull(ANSITEXTVIEW("Null"));
			Writer.EndObject();
		}
		Writer.EndObject();

		Writer.BeginArray(ANSITEXTVIEW("NestedArrays"));
		{
			Writer.BeginArray();
			Writer.EndArray();

			Writer.BeginArray();
			Writer.AddNull();
			Writer.AddNull();
			Writer.AddNull();
			Writer.EndArray();

			Writer.BeginArray();
			Writer.AddBool(false);
			Writer.AddBool(false);
			Writer.AddBool(false);
			Writer.EndArray();

			Writer.BeginArray();
			Writer.AddBool(true);
			Writer.AddBool(true);
			Writer.AddBool(true);
			Writer.EndArray();
		}
		Writer.EndArray();

		Writer.BeginArray(ANSITEXTVIEW("ArrayOfObjects"));
		{
			Writer.BeginObject();
			Writer.EndObject();

			Writer.BeginObject();
			Writer.AddNull(ANSITEXTVIEW("Null"));
			Writer.EndObject();
		}
		Writer.EndArray();

		Writer.BeginArray(ANSITEXTVIEW("LargeArray"));
		for (int Index = 0; Index < 256; ++Index)
		{
			Writer.AddInteger(Index - 128);
		}
		Writer.EndArray();

		Writer.BeginArray(ANSITEXTVIEW("LargeUniformArray"));
		for (int Index = 0; Index < 256; ++Index)
		{
			Writer.AddInteger(Index);
		}
		Writer.EndArray();

		Writer.BeginArray(ANSITEXTVIEW("NestedUniformArray"));
		for (int Index = 0; Index < 16; ++Index)
		{
			Writer.BeginArray();
			for (int Value = 0; Value < 4; ++Value)
			{
				Writer.AddInteger(Value);
			}
			Writer.EndArray();
		}
		Writer.EndArray();

		Writer.EndObject();
		Object = Writer.Save().AsObject();

		Writer.Save(Archive);
		CHECK_EQUALS(TEXT("FCbWriter(Complex).Save(Ar)->Num()"), uint64(Archive.Num()), Writer.GetSaveSize());
	}

	CHECK_EQUALS(TEXT("FCbWriter(Complex).Save()->Validate"),
		ValidateCompactBinary(Object.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None);

	CHECK_EQUALS(TEXT("FCbWriter(Complex).Save(Ar)->Validate"),
		ValidateCompactBinary(MakeMemoryView(Archive), ECbValidateMode::All), ECbValidateError::None);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterOwnedReadOnlyTest, "System::Core::Serialization::CbWriter::OwnedReadOnly", "[ApplicationContextMask][SmokeFilter]")
{
	FCbWriter Writer;
	Writer.BeginObject();
	Writer.EndObject();
	FCbObject Object = Writer.Save().AsObject();
	CHECK_MESSAGE(TEXT("FCbWriter().Save().IsOwned()"), Object.IsOwned());
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterStreamTest, "System::Core::Serialization::CbWriter::Stream", "[ApplicationContextMask][SmokeFilter]")
{
	FCbObject Object;
	{
		FCbWriter Writer;
		Writer.BeginObject();

		const uint8 LocalField[] = { uint8(ECbFieldType::IntegerPositive | ECbFieldType::HasFieldName), 1, 'I', 42 };
		Writer << ANSITEXTVIEW("FieldCopy") << FCbFieldView(LocalField);

		const uint8 LocalObject[] = { uint8(ECbFieldType::Object | ECbFieldType::HasFieldName), 1, 'O', 7,
			uint8(ECbFieldType::IntegerPositive | ECbFieldType::HasFieldName), 1, 'I', 42,
			uint8(ECbFieldType::Null | ECbFieldType::HasFieldName), 1, 'N' };
		Writer << ANSITEXTVIEW("ObjectCopy") << FCbObjectView(LocalObject);

		const uint8 LocalArray[] = { uint8(ECbFieldType::UniformArray | ECbFieldType::HasFieldName), 1, 'A', 4, 2,
			uint8(ECbFieldType::IntegerPositive), 42, 21 };
		Writer << ANSITEXTVIEW("ArrayCopy") << FCbArrayView(LocalArray);

		Writer << ANSITEXTVIEW("Null") << nullptr;

		Writer << ANSITEXTVIEW("Strings");
		Writer.BeginObject();
		Writer
			<< ANSITEXTVIEW("AnsiString") << ANSITEXTVIEW("AnsiValue")
			<< ANSITEXTVIEW("AnsiStringLiteral") << "AnsiValue"
			<< ANSITEXTVIEW("WideString") << TEXTVIEW("WideValue")
			<< ANSITEXTVIEW("WideStringLiteral") << TEXT("WideValue");
		Writer.EndObject();

		Writer << ANSITEXTVIEW("Integers");
		Writer.BeginArray();
		Writer << int32(-1) << int64(-1) << uint32(1) << uint64(1);
		Writer << SSIZE_T(-1) << SIZE_T(-1) << PTRINT(-1) << UPTRINT(-1);
		Writer.EndArray();

		Writer << ANSITEXTVIEW("Float32") << 1.0f;
		Writer << ANSITEXTVIEW("Float64") << 2.0;

		Writer << ANSITEXTVIEW("False") << false << ANSITEXTVIEW("True") << true;

		Writer << ANSITEXTVIEW("Attachment") << FCbAttachment();

		Writer << ANSITEXTVIEW("Hash") << FIoHash();
		Writer << ANSITEXTVIEW("Uuid") << FGuid();

		Writer << ANSITEXTVIEW("DateTime") << FDateTime(2020, 5, 13, 15, 10);
		Writer << ANSITEXTVIEW("TimeSpan") << FTimespan(1, 2, 4, 8);

		Writer << ANSITEXTVIEW("ObjectIdZero") << FCbObjectId();
		Writer << ANSITEXTVIEW("ObjectId") << FCbObjectId(MakeMemoryView<uint8>({1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12}));

		Writer << "LiteralName" << nullptr;

		Writer.EndObject();
		Object = Writer.Save().AsObject();
	}

	CHECK_EQUALS(TEXT("FCbWriter(Stream) Validate"), ValidateCompactBinary(Object.GetOuterBuffer(), ECbValidateMode::All), ECbValidateError::None);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FCbWriterStateTest, "System::Core::Serialization::CbWriter::State", "[ApplicationContextMask][SmokeFilter]")
{
	FCbWriter Writer;

	// Assert on saving an empty writer.
	//uint8 EmptyField[1];
	//Writer.Reset();
	//Writer.Save(MakeMemoryView(EmptyField));

	// Assert on under-sized save buffer.
	//uint8 ZeroFieldSmall[1];
	//Writer.Reset();
	//Writer.AddInteger(0);
	//Writer.Save(MakeMemoryView(ZeroFieldSmall));

	// Assert on over-sized save buffer.
	//uint8 ZeroFieldLarge[3];
	//Writer.Reset();
	//Writer.AddInteger(0);
	//Writer.Save(MakeMemoryView(ZeroFieldLarge));

	// Assert on empty name.
	//Writer.SetName(ANSITEXTVIEW(""));

	// Assert on name after name.
	//Writer.SetName(ANSITEXTVIEW("Field")).SetName(ANSITEXTVIEW("Field"));

	// Assert on missing name.
	//Writer.BeginObject();
	//Writer.AddNull();
	//Writer.EndObject();

	// Assert on name in array.
	//Writer.BeginArray();
	//Writer.SetName(ANSITEXTVIEW("Field"));
	//Writer.EndArray();

	// Assert on save in object.
	//uint8 InvalidObject[1];
	//Writer.Reset();
	//Writer.BeginObject();
	//Writer.Save(MakeMemoryView(InvalidObject));
	//Writer.EndObject();

	// Assert on save in array.
	//uint8 InvalidArray[1];
	//Writer.Reset();
	//Writer.BeginArray();
	//Writer.Save(MakeMemoryView(InvalidArray));
	//Writer.EndArray();

	// Assert on object end with no begin.
	//Writer.EndObject();

	// Assert on array end with no begin.
	//Writer.EndArray();

	// Assert on object end after name with no value.
	//Writer.BeginObject();
	//Writer.SetName(ANSITEXTVIEW("Field"));
	//Writer.EndObject();

	// Assert on writing a field with no value.
	//Writer.Field(FCbFieldView());
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#endif // WITH_TESTS
