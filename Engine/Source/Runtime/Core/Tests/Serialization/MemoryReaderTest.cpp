// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Tests/TestHarnessAdapter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#if WITH_TESTS

TEST_CASE_NAMED(FMemoryReaderTest, "System::Core::Serialization::MemoryReader", "[ApplicationContextMask][SmokeFilter]")
{
	// Test reading uint64 byte swapped, ensuring that bytes really get swapped.
	{
		const uint64 WrittenValue  = 0x8877665544332211ull;
		const uint64 ExpectedValue = 0x1122334455667788ull;

		TArray<uint8> Bytes;
		Bytes.AddUninitialized(sizeof(uint64));
		*reinterpret_cast<uint64*>(Bytes.GetData()) = WrittenValue;

		FMemoryReader Reader(Bytes);
		Reader.SetByteSwapping(true);
		uint64 ReadValue;
		Reader << ReadValue;
		CHECK_MESSAGE("Test reading uint64 in swapped byte order.", ReadValue == ExpectedValue);
	}

	// Write all supported types and read them back. (Assuming that MemoryWriterTest tests don't fail).
	{
		// Keeps the 'official' test value.
		const uint8  TestValueU8   = 0x12;
		const int8   TestValueS8   = 0x34;
		const uint16 TestValueU16  = 0x1122;
		const int16  TestValueS16  = 0x3344;
		const uint32 TestValueU32  = 0x11223344;
		const int32  TestValueS32  = 0x55667788;
		const uint64 TestValueU64  = 0x1122334455667788;
		const int64  TestValueS64  = 0x99AABBCCDDEEFF00;
		const float  TestValueF    = 128.5f;
		const double TestValueD    = 256.5;
		const bool   TestValueB    = true;
		const WIDECHAR TestValueCh = 0xF2;
		const FString TestAnsiStr  = "Joe";
		const FString TestUtf16Str = TEXT("\uC11C\uC6B8\uC0AC\uB78C"); // Must contains at least one non-ANSI codepoint to ensure UTF16 is used.

		// The values to write.
		TArray<uint8> Bytes;
		FMemoryWriter Writer(Bytes);
		uint8  ValueU8   = TestValueU8;
		int8   ValueS8   = TestValueS8;
		uint16 ValueU16  = TestValueU16;
		int16  ValueS16  = TestValueS16;
		uint32 ValueU32  = TestValueU32;
		int32  ValueS32  = TestValueS32;
		uint64 ValueU64  = TestValueU64;
		int64  ValueS64  = TestValueS64;
		float  ValueF    = TestValueF;
		double ValueD    = TestValueD;
		bool   ValueB    = TestValueB;
		WIDECHAR ValueCh = TestValueCh;
		FString AnsiStr  = TestAnsiStr;
		FString Utf16Str = TestUtf16Str;

		// Try to not align the values in the buffer, so that the test would fail (on platforms not supporting misaligned read) if the reader tries to cast and dereference a pointer type with misaligned address. Doing like: *(uint64*)Array.GetData();
		Writer << ValueU8;
		Writer << ValueU16;
		Writer << ValueU32;
		Writer << ValueU64;
		Writer << ValueS8;
		Writer << ValueS16;
		Writer << ValueS32;
		Writer << ValueS64;
		Writer << ValueF;
		Writer << ValueD;
		Writer << ValueB;
		Writer << ValueCh;
		Writer << AnsiStr;
		Writer << Utf16Str;

		// Write again, but write the bytes swapped.
		Writer.SetByteSwapping(true);

		Writer << ValueU8;
		Writer << ValueU16;
		Writer << ValueU32;
		Writer << ValueU64;
		Writer << ValueS8;
		Writer << ValueS16;
		Writer << ValueS32;
		Writer << ValueS64;
		Writer << ValueF;
		Writer << ValueD;
		Writer << ValueB;
		Writer << ValueCh;
		Writer << AnsiStr;
		Writer << Utf16Str;

		// The value to reads without swapping the the value swapped back to this platform endianness.
		uint8  ReadValueU8,  ReadValueSwapU8;
		int8   ReadValueS8,  ReadValueSwapS8;
		uint16 ReadValueU16, ReadValueSwapU16;
		int16  ReadValueS16, ReadValueSwapS16;
		uint32 ReadValueU32, ReadValueSwapU32;
		int32  ReadValueS32, ReadValueSwapS32;
		uint64 ReadValueU64, ReadValueSwapU64;
		int64  ReadValueS64, ReadValueSwapS64;
		float  ReadValueF,   ReadValueSwapF;
		double ReadValueD,   ReadValueSwapD;
		bool   ReadValueB,   ReadValueSwapB;
		WIDECHAR ReadValueCh, ReadValueSwapCh;
		FString ReadAnsiStr,  ReadSwapAnsiStr;
		FString ReadUtf16Str, ReadSwapUtf16Str;

		// Read the first set of values written in this platform endianness.
		FMemoryReader Reader(Bytes);
		Reader << ReadValueU8;
		Reader << ReadValueU16;
		Reader << ReadValueU32;
		Reader << ReadValueU64;
		Reader << ReadValueS8;
		Reader << ReadValueS16;
		Reader << ReadValueS32;
		Reader << ReadValueS64;
		Reader << ReadValueF;
		Reader << ReadValueD;
		Reader << ReadValueB;
		Reader << ReadValueCh;
		Reader << ReadAnsiStr;
		Reader << ReadUtf16Str;

		// Read the swapped values, swapping them back to their original value (Assuming that MemoryWriterTest tests don't fail)
		Reader.SetByteSwapping(true);
		Reader << ReadValueSwapU8;
		Reader << ReadValueSwapU16;
		Reader << ReadValueSwapU32;
		Reader << ReadValueSwapU64;
		Reader << ReadValueSwapS8;
		Reader << ReadValueSwapS16;
		Reader << ReadValueSwapS32;
		Reader << ReadValueSwapS64;
		Reader << ReadValueSwapF;
		Reader << ReadValueSwapD;
		Reader << ReadValueSwapB;
		Reader << ReadValueSwapCh;
		Reader << ReadSwapAnsiStr;
		Reader << ReadSwapUtf16Str;

		// Validate that the value read are the expected ones.
		CHECK_MESSAGE("Test reading 'u8' from byte stream",             ReadValueU8      == TestValueU8);
		CHECK_MESSAGE("Test reading 'u16' from byte stream",            ReadValueU16     == TestValueU16);
		CHECK_MESSAGE("Test reading 'u32' from byte stream",            ReadValueU32     == TestValueU32);
		CHECK_MESSAGE("Test reading 'u64' from byte stream",            ReadValueU64     == TestValueU64);
		CHECK_MESSAGE("Test reading 's8' from byte stream",             ReadValueS8      == TestValueS8);
		CHECK_MESSAGE("Test reading 's16' from byte stream",            ReadValueS16     == TestValueS16);
		CHECK_MESSAGE("Test reading 's32' from byte stream",            ReadValueS32     == TestValueS32);
		CHECK_MESSAGE("Test reading 's64' from byte stream",            ReadValueS64     == TestValueS64);
		CHECK_MESSAGE("Test reading 'foat' from byte stream",           ReadValueF       == TestValueF);
		CHECK_MESSAGE("Test reading 'double' from byte stream",         ReadValueD       == TestValueD);
		CHECK_MESSAGE("Test reading 'bool' from byte stream",           ReadValueB       == TestValueB);
		CHECK_MESSAGE("Test reading 'wchar' from byte stream",          ReadValueCh      == TestValueCh);
		CHECK_MESSAGE("Test reading 'ansi str' from byte stream",       ReadAnsiStr      == TestAnsiStr);
		CHECK_MESSAGE("Test reading 'utf16 str' from byte stream",      ReadUtf16Str     == TestUtf16Str);
		CHECK_MESSAGE("Test reading 'u8-swapped' from byte stream",     ReadValueSwapU8  == TestValueU8);
		CHECK_MESSAGE("Test reading 'u16-swapped' from byte stream",    ReadValueSwapU16 == TestValueU16);
		CHECK_MESSAGE("Test reading 'u32-swapped' from byte stream",    ReadValueSwapU32 == TestValueU32);
		CHECK_MESSAGE("Test reading 'u64-swapped' from byte stream",    ReadValueSwapU64 == TestValueU64);
		CHECK_MESSAGE("Test reading 's8-swapped' from byte stream",     ReadValueSwapS8  == TestValueS8);
		CHECK_MESSAGE("Test reading 's16'-swapped' from byte stream",   ReadValueSwapS16 == TestValueS16);
		CHECK_MESSAGE("Test reading 's32-swapped' from byte stream",    ReadValueSwapS32 == TestValueS32);
		CHECK_MESSAGE("Test reading 's64-swapped' from byte stream",    ReadValueSwapS64 == TestValueS64);
		CHECK_MESSAGE("Test reading 'float-swapped' from byte stream",  ReadValueSwapF   == TestValueF);
		CHECK_MESSAGE("Test reading 'double-swapped' from byte stream", ReadValueSwapD   == TestValueD);
		CHECK_MESSAGE("Test reading 'bool-swapped' from byte stream",   ReadValueSwapB   == TestValueB);
		CHECK_MESSAGE("Test reading 'wchar-swapped' from byte stream",  ReadValueSwapCh  == TestValueCh);
		CHECK_MESSAGE("Test reading 'ansi-swapped' from byte stream",   ReadSwapAnsiStr  == TestAnsiStr);
		CHECK_MESSAGE("Test reading 'utf16-swapped' from byte stream",  ReadSwapUtf16Str == TestUtf16Str);
	}
}

#endif // WITH_TESTS
