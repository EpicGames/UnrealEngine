// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Tests/TestHarnessAdapter.h"
#include "Misc/ByteSwap.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

#if WITH_TESTS

namespace MemoryWriterTestUtil
{
template <typename T> struct TIsBoolean { enum { Value = false }; };
template <> struct TIsBoolean<bool> { enum { Value = true }; };

class MemoryWriterTester
{
public:
	MemoryWriterTester()
		: Writer(Bytes)
	{
	}

	template<typename T>
	void TestWritePlatformByteOrder(T Value)
	{
		const T Original = Value;
		Writer << Value;
		CheckWrittenByteCount<T>(Bytes.Num());
		checkf(Value == Original, TEXT("The writer unexpectedly modified the input value")); // Ensure the written value wasn't changed during the write. (Regression check when writing byte swapped)
		checkf(FMemory::Memcmp(Bytes.GetData(), &Original, sizeof(Original)) == 0, TEXT("The written value doesn't match the expected one."));
	}

	template<typename T>
	void TestWriteSwappedByteOrder(T Value, T Swapped)
	{
		const T Original = Value;
		Writer.SetByteSwapping(true);
		Writer << Value;
		CheckWrittenByteCount<T>(Bytes.Num());
		checkf(Value == Original, TEXT("The writer unexpectedly modified the input value")); // Ensure the written value wasn't changed during the write. (Regression check when writing byte swapped)
		checkf(FMemory::Memcmp(Bytes.GetData(), &Swapped, sizeof(Swapped)) == 0, TEXT("The written value doesn't match the swapped value."));
	}

	template<typename T>
	void CheckWrittenByteCount(int32 WrittenCount)
	{
		if (TIsBoolean<T>::Value)
		{
			// Boolean are written as 4 bytes integer.
			checkf(WrittenCount == sizeof(int32), TEXT("Unexpected number of bytes written by the writer"));
		}
		else
		{
			checkf(WrittenCount == sizeof(T), TEXT("Unexpected number of bytes written by the writer"));
		}
	}

public:
	TArray<uint8> Bytes;
	FMemoryWriter Writer;
};

}

TEST_CASE_NAMED(FMemoryWriterTest, "System::Core::Serialization::MemoryWriter", "[ApplicationContextMask][SmokeFilter]")
{
	// Keeps the 'official' test value as const, to prevent overwriting them.
	const uint8  TestValueU8   = 0x12;
	const int8   TestValueS8   = 0x34;
	const uint16 TestValueU16  = 0x1122;
	const int16  TestValueS16  = 0x3344;
	const uint32 TestValueU32  = 0x11223344;
	const int32  TestValueS32  = 0x55667788;
	const uint64 TestValueU64  = 0x1122334455667788;
	const int64  TestValueS64  = 0x99AABBCCDDEEFF00;
	const float  TestValueF    = 128.5f;
	const double TestValueD    = 256.5;
	const bool   TestValueB    = true;
	const WIDECHAR TestValueCh = 0xF2;

	// Platform endianness tests.
	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueU8);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueS8);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueU16);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueS16);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueU32);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueS32);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueU64);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueS64);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueF);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueD);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWritePlatformByteOrder(TestValueB);
	}

	// Non Platform endianness tests. (Byte swapping)
	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueU8, TestValueU8);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueS8, TestValueS8);
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueU16, ByteSwap(TestValueU16));
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueS16, ByteSwap(TestValueS16));
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueU32, ByteSwap(TestValueU32));
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueS32, ByteSwap(TestValueS32));
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueU64, ByteSwap(TestValueU64));
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueS64, ByteSwap(TestValueS64));
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueF, ByteSwap(TestValueF));
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueD, ByteSwap(TestValueD));
	}

	{
		MemoryWriterTestUtil::MemoryWriterTester Tester;
		Tester.TestWriteSwappedByteOrder(TestValueB, TestValueB);
	}

	// ANSI String
	{
		const TCHAR* RawStr = TEXT("Joe"); // Only contains ANSI chars -> should be serialized as an ANSI string (1 byte per char)
		const char* RawStrBytes = "Joe";

		const FString TestString = RawStr;
		int32 CharCount = TestString.Len() + 1; // Include \0.
		int32 StrByteCount = CharCount;
		int32 CharCountInBuffer = CharCount; // A positive count let the deserializer know this an ANSI string rather than a UTF16 string.
		int32 CharCountInBufferSwapped = ByteSwap(CharCountInBuffer);

		// Platform endianness
		{
			TArray<uint8> Bytes;
			FMemoryWriter Writer(Bytes);

			FString WriteValue = TestString;
			Writer << WriteValue;
			check(WriteValue == TestString); // Ensure the original value did not change (as side effect).
			check(FMemory::Memcmp(Bytes.GetData(), &CharCountInBuffer, sizeof(int32)) == 0); // Check the count encoded as the first 4 bytes.
			check(FMemory::Memcmp(Bytes.GetData() + sizeof(int32), RawStrBytes, StrByteCount) == 0); // Check the string content.
		}
	
		// Swapped endianness
		{
			TArray<uint8> Bytes;
			FMemoryWriter Writer(Bytes);
			Writer.SetByteSwapping(true); // Swap bytes when writing.

			FString WriteValue = TestString;
			Writer << WriteValue;
			check(WriteValue == TestString); // Ensure the original value did not change (as side effect).
			check(FMemory::Memcmp(Bytes.GetData(), &CharCountInBufferSwapped, sizeof(int32)) == 0); // Check the count encoded as the first 4 bytes.
			check(FMemory::Memcmp(Bytes.GetData() + sizeof(int32), RawStrBytes, StrByteCount) == 0); // Check the string content.
		}
	}

	// UTF16 Strings
	{
		// Must not contain at least one non-ANSI char otherwise, the serialization finds out and serialize and ANSI string (1-byte per char).
		const TCHAR RawStr[] = {TEXT('\u0404'), TEXT('\u0400'), TEXT('\uC0AC'), TEXT('\u0000')};
		const uint8* RawStrBytes = reinterpret_cast<const uint8*>(RawStr);

		const TCHAR RawStrSwapped[] = { ByteSwap(static_cast<uint16>(RawStr[0])), ByteSwap(static_cast<uint16>(RawStr[1])), ByteSwap(static_cast<uint16>(RawStr[2])), ByteSwap(static_cast<uint16>(RawStr[3]))};
		const uint8* RawStrBytesSwapped = reinterpret_cast<const uint8*>(RawStrSwapped);

		const FString TestString = RawStr;
		int32 CharCount = TestString.Len() + 1; // Include \0.
		int32 StrByteCount = CharCount * sizeof(TCHAR);
		int32 CharCountInBuffer = -CharCount; // A negative count let the deserializer know this an UTF16 string rather than a ANSI string.
		int32 CharCountInBufferSwapped = ByteSwap(CharCountInBuffer);

		// Platform endianness
		{
			TArray<uint8> Bytes;
			FMemoryWriter Writer(Bytes);

			FString WriteValue = TestString;
			Writer << WriteValue;
			check(WriteValue == TestString); // Ensure the original value did not change (as side effect).
			check(FMemory::Memcmp(Bytes.GetData(), &CharCountInBuffer, sizeof(int32)) == 0); // Check the count encoded as the first 4 bytes.
			check(FMemory::Memcmp(Bytes.GetData() + sizeof(int32), RawStrBytes, StrByteCount) == 0); // Check the string content.
		}
	
		// Swapped endianness
		{
			TArray<uint8> Bytes;
			FMemoryWriter Writer(Bytes);
			Writer.SetByteSwapping(true); // Swap bytes when writing.

			FString WriteValue = TestString;
			Writer << WriteValue;
			check(WriteValue == TestString); // Ensure the original value did not change (as side effect).
			check(FMemory::Memcmp(Bytes.GetData(), &CharCountInBufferSwapped, sizeof(int32)) == 0); // Check the count encoded as the first 4 bytes.
			check(FMemory::Memcmp(Bytes.GetData() + sizeof(int32), RawStrBytesSwapped, StrByteCount) == 0); // Check the string content.
		}
	}
}

#endif // WITH_TESTS
