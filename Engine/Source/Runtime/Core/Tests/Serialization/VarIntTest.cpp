// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Serialization/VarInt.h"

#include "Serialization/BufferReader.h"
#include "Serialization/BufferWriter.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

TEST_CASE("Core::Serialization::VarInt::Measure", "[Core][Serialization][Smoke]")
{
	SECTION("Test MeasureVarInt at signed 32-bit encoding boundaries.")
	{
		const auto [Value, Size] = GENERATE(table<int32, uint32>(
		{
			{0x0000'0000, 1},
			{0x0000'0001, 1}, {0x0000'003f, 1},
			{0x0000'0040, 2}, {0x0000'1fff, 2},
			{0x0000'2000, 3}, {0x000f'ffff, 3},
			{0x0010'0000, 4}, {0x07ff'ffff, 4},
			{0x0800'0000, 5}, {0x7fff'ffff, 5},
			{0xffff'ffff, 1}, {0xffff'ffc0, 1}, // -0x0000'0001, -0x0000'0040
			{0xffff'ffbf, 2}, {0xffff'e000, 2}, // -0x0000'0041, -0x0000'2000
			{0xffff'dfff, 3}, {0xfff0'0000, 3}, // -0x0000'2001, -0x0010'0000
			{0xffef'ffff, 4}, {0xf800'0000, 4}, // -0x0010'0001, -0x0800'0000
			{0xf7ff'ffff, 5}, {0x8000'0000, 5}, // -0x0800'0001, -0x8000'0000
		}));
		CAPTURE(Value, Size);
		CHECK(MeasureVarInt(Value) == Size);
	}

	SECTION("Test MeasureVarUInt at unsigned 32-bit encoding boundaries.")
	{
		const auto [Value, Size] = GENERATE(table<uint32, uint32>(
		{
			{0x0000'0000, 1},
			{0x0000'0001, 1}, {0x0000'007f, 1},
			{0x0000'0080, 2}, {0x0000'3fff, 2},
			{0x0000'4000, 3}, {0x001f'ffff, 3},
			{0x0020'0000, 4}, {0x0fff'ffff, 4},
			{0x1000'0000, 5}, {0xffff'ffff, 5},
		}));
		CAPTURE(Value, Size);
		CHECK(MeasureVarUInt(Value) == Size);
	}

	SECTION("Test MeasureVarInt at signed 64-bit encoding boundaries.")
	{
		const auto [Value, Size] = GENERATE(table<int64, uint32>(
		{
			{0x0000'0000'0000'0000, 1},
			{0x0000'0000'0000'0001, 1}, {0x0000'0000'0000'003f, 1},
			{0x0000'0000'0000'0040, 2}, {0x0000'0000'0000'1fff, 2},
			{0x0000'0000'0000'2000, 3}, {0x0000'0000'000f'ffff, 3},
			{0x0000'0000'0010'0000, 4}, {0x0000'0000'07ff'ffff, 4},
			{0x0000'0000'0800'0000, 5}, {0x0000'0003'ffff'ffff, 5},
			{0x0000'0004'0000'0000, 6}, {0x0000'01ff'ffff'ffff, 6},
			{0x0000'0200'0000'0000, 7}, {0x0000'ffff'ffff'ffff, 7},
			{0x0001'0000'0000'0000, 8}, {0x007f'ffff'ffff'ffff, 8},
			{0x0080'0000'0000'0000, 9}, {0x7fff'ffff'ffff'ffff, 9},
			{0xffff'ffff'ffff'ffff, 1}, {0xffff'ffff'ffff'ffc0, 1}, // -0x0000'0000'0000'0001, -0x0000'0000'0000'0040
			{0xffff'ffff'ffff'ffbf, 2}, {0xffff'ffff'ffff'e000, 2}, // -0x0000'0000'0000'0041, -0x0000'0000'0000'2000
			{0xffff'ffff'ffff'dfff, 3}, {0xffff'ffff'fff0'0000, 3}, // -0x0000'0000'0000'2001, -0x0000'0000'0010'0000
			{0xffff'ffff'ffef'ffff, 4}, {0xffff'ffff'f800'0000, 4}, // -0x0000'0000'0010'0001, -0x0000'0000'0800'0000
			{0xffff'ffff'f7ff'ffff, 5}, {0xffff'fffc'0000'0000, 5}, // -0x0000'0000'0800'0001, -0x0000'0004'0000'0000
			{0xffff'fffb'ffff'ffff, 6}, {0xffff'fe00'0000'0000, 6}, // -0x0000'0004'0000'0001, -0x0000'0200'0000'0000
			{0xffff'fdff'ffff'ffff, 7}, {0xffff'0000'0000'0000, 7}, // -0x0000'0200'0000'0001, -0x0001'0000'0000'0000
			{0xfffe'ffff'ffff'ffff, 8}, {0xff80'0000'0000'0000, 8}, // -0x0001'0000'0000'0001, -0x0080'0000'0000'0000
			{0xff7f'ffff'ffff'ffff, 9}, {0x8000'0000'0000'0000, 9}, // -0x0080'0000'0000'0001, -0x8000'0000'0000'0000
		}));
		CAPTURE(Value, Size);
		CHECK(MeasureVarInt(Value) == Size);
	}

	SECTION("Test MeasureVarUInt at unsigned 64-bit encoding boundaries.")
	{
		const auto [Value, Size] = GENERATE(table<uint64, uint32>(
		{
			{0x0000'0000'0000'0000, 1},
			{0x0000'0000'0000'0001, 1}, {0x0000'0000'0000'007f, 1},
			{0x0000'0000'0000'0080, 2}, {0x0000'0000'0000'3fff, 2},
			{0x0000'0000'0000'4000, 3}, {0x0000'0000'001f'ffff, 3},
			{0x0000'0000'0020'0000, 4}, {0x0000'0000'0fff'ffff, 4},
			{0x0000'0000'1000'0000, 5}, {0x0000'0007'ffff'ffff, 5},
			{0x0000'0008'0000'0000, 6}, {0x0000'03ff'ffff'ffff, 6},
			{0x0000'0400'0000'0000, 7}, {0x0001'ffff'ffff'ffff, 7},
			{0x0002'0000'0000'0000, 8}, {0x00ff'ffff'ffff'ffff, 8},
			{0x0100'0000'0000'0000, 9}, {0xffff'ffff'ffff'ffff, 9},
		}));
		CAPTURE(Value, Size);
		CHECK(MeasureVarUInt(Value) == Size);
	}

	SECTION("Test MeasureVarInt at encoding boundaries.")
	{
		const auto [FirstByte, Size] = GENERATE(table<uint8, uint32>(
		{
			{uint8(0b0000'0000), 1}, {uint8(0b0111'1111), 1},
			{uint8(0b1000'0000), 2}, {uint8(0b1011'1111), 2},
			{uint8(0b1100'0000), 3}, {uint8(0b1101'1111), 3},
			{uint8(0b1110'0000), 4}, {uint8(0b1110'1111), 4},
			{uint8(0b1111'0000), 5}, {uint8(0b1111'0111), 5},
			{uint8(0b1111'1000), 6}, {uint8(0b1111'1011), 6},
			{uint8(0b1111'1100), 7}, {uint8(0b1111'1101), 7},
			{uint8(0b1111'1110), 8},
			{uint8(0b1111'1111), 9},
		}));
		CAPTURE(FirstByte, Size);
		CHECK(MeasureVarInt(&FirstByte) == Size);
	}

	SECTION("Test MeasureVarUInt at encoding boundaries.")
	{
		const auto [FirstByte, Size] = GENERATE(table<uint8, uint32>(
		{
			{uint8(0b0000'0000), 1}, {uint8(0b0111'1111), 1},
			{uint8(0b1000'0000), 2}, {uint8(0b1011'1111), 2},
			{uint8(0b1100'0000), 3}, {uint8(0b1101'1111), 3},
			{uint8(0b1110'0000), 4}, {uint8(0b1110'1111), 4},
			{uint8(0b1111'0000), 5}, {uint8(0b1111'0111), 5},
			{uint8(0b1111'1000), 6}, {uint8(0b1111'1011), 6},
			{uint8(0b1111'1100), 7}, {uint8(0b1111'1101), 7},
			{uint8(0b1111'1110), 8},
			{uint8(0b1111'1111), 9},
		}));
		CAPTURE(FirstByte, Size);
		CHECK(MeasureVarUInt(&FirstByte) == Size);
	}
}

TEST_CASE("Core::Serialization::VarInt::Serialize", "[Core][Serialization][Smoke]")
{
	SECTION("Test Read/WriteVarInt at signed 32-bit encoding boundaries.")
	{
		const int32 Value = GENERATE(as<int32>{},
			0x0000'0000,
			0x0000'0001, 0x0000'003f,
			0x0000'0040, 0x0000'1fff,
			0x0000'2000, 0x000f'ffff,
			0x0010'0000, 0x07ff'ffff,
			0x0800'0000, 0x7fff'ffff,
			0xffff'ffff, 0xffff'ffc0,  // -0x0000'0001, -0x0000'0040
			0xffff'ffbf, 0xffff'e000,  // -0x0000'0041, -0x0000'2000
			0xffff'dfff, 0xfff0'0000,  // -0x0000'2001, -0x0010'0000
			0xffef'ffff, 0xf800'0000,  // -0x0010'0001, -0x0800'0000
			0xf7ff'ffff, 0x8000'0000); // -0x0800'0001, -0x8000'0000
		CAPTURE(Value);

		constexpr int32 BufferSize = 5;

		uint8 Buffer[BufferSize];
		const uint32 WriteByteCount = WriteVarInt(Value, Buffer);
		CHECK(WriteByteCount <= BufferSize);
		uint32 ReadByteCount = 0;
		CHECK(ReadVarInt(Buffer, ReadByteCount) == Value);
		CHECK(ReadByteCount == WriteByteCount);

		uint8 ArBuffer[BufferSize];
		FBufferWriter WriteAr(ArBuffer, BufferSize);
		WriteVarIntToArchive(WriteAr, Value);
		CHECK(WriteAr.Tell() == WriteByteCount);
		FBufferReader ReadAr(ArBuffer, BufferSize, /*bFreeOnClose*/ false);
		CHECK(ReadVarIntFromArchive(ReadAr) == Value);
		CHECK(ReadAr.Tell() == ReadByteCount);
	}

	SECTION("Test Read/WriteVarUInt at unsigned 32-bit encoding boundaries.")
	{
		const uint32 Value = GENERATE(as<uint32>{},
			0x0000'0000, 0x0000'007f,
			0x0000'0080, 0x0000'3fff,
			0x0000'4000, 0x0000'7fff,
			0x0000'8000, 0x0000'ffff,
			0x001f'ffff, 0x0020'0000,
			0x0fff'ffff, 0x1000'0000,
			0xffff'ffff);
		CAPTURE(Value);

		constexpr int32 BufferSize = 5;

		uint8 Buffer[BufferSize];
		const uint32 WriteByteCount = WriteVarUInt(Value, Buffer);
		CHECK(WriteByteCount <= BufferSize);
		uint32 ReadByteCount = 0;
		CHECK(ReadVarUInt(Buffer, ReadByteCount) == Value);
		CHECK(ReadByteCount == WriteByteCount);

		uint8 ArBuffer[BufferSize];
		FBufferWriter WriteAr(ArBuffer, BufferSize);
		WriteVarUIntToArchive(WriteAr, Value);
		CHECK(WriteAr.Tell() == WriteByteCount);
		FBufferReader ReadAr(ArBuffer, BufferSize, /*bFreeOnClose*/ false);
		CHECK(ReadVarUIntFromArchive(ReadAr) == Value);
		CHECK(ReadAr.Tell() == ReadByteCount);
	}

	SECTION("Test Read/WriteVarInt at signed 64-bit encoding boundaries.")
	{
		const int64 Value = GENERATE(as<int64>{},
			0x0000'0000'0000'0000,
			0x0000'0000'0000'0001, 0x0000'0000'0000'003f,
			0x0000'0000'0000'0040, 0x0000'0000'0000'1fff,
			0x0000'0000'0000'2000, 0x0000'0000'000f'ffff,
			0x0000'0000'0010'0000, 0x0000'0000'07ff'ffff,
			0x0000'0000'0800'0000, 0x0000'0003'ffff'ffff,
			0x0000'0004'0000'0000, 0x0000'01ff'ffff'ffff,
			0x0000'0200'0000'0000, 0x0000'ffff'ffff'ffff,
			0x0001'0000'0000'0000, 0x007f'ffff'ffff'ffff,
			0x0080'0000'0000'0000, 0x7fff'ffff'ffff'ffff,
			0xffff'ffff'ffff'ffff, 0xffff'ffff'ffff'ffc0,  // -0x0000'0000'0000'0001, -0x0000'0000'0000'0040
			0xffff'ffff'ffff'ffbf, 0xffff'ffff'ffff'e000,  // -0x0000'0000'0000'0041, -0x0000'0000'0000'2000
			0xffff'ffff'ffff'dfff, 0xffff'ffff'fff0'0000,  // -0x0000'0000'0000'2001, -0x0000'0000'0010'0000
			0xffff'ffff'ffef'ffff, 0xffff'ffff'f800'0000,  // -0x0000'0000'0010'0001, -0x0000'0000'0800'0000
			0xffff'ffff'f7ff'ffff, 0xffff'fffc'0000'0000,  // -0x0000'0000'0800'0001, -0x0000'0004'0000'0000
			0xffff'fffb'ffff'ffff, 0xffff'fe00'0000'0000,  // -0x0000'0004'0000'0001, -0x0000'0200'0000'0000
			0xffff'fdff'ffff'ffff, 0xffff'0000'0000'0000,  // -0x0000'0200'0000'0001, -0x0001'0000'0000'0000
			0xfffe'ffff'ffff'ffff, 0xff80'0000'0000'0000,  // -0x0001'0000'0000'0001, -0x0080'0000'0000'0000
			0xff7f'ffff'ffff'ffff, 0x8000'0000'0000'0000); // -0x0080'0000'0000'0001, -0x8000'0000'0000'0000
		CAPTURE(Value);

		constexpr int32 BufferSize = 9;

		uint8 Buffer[BufferSize];
		const uint32 WriteByteCount = WriteVarInt(Value, Buffer);
		CHECK(WriteByteCount <= BufferSize);
		uint32 ReadByteCount = 0;
		CHECK(ReadVarInt(Buffer, ReadByteCount) == Value);
		CHECK(ReadByteCount == WriteByteCount);

		uint8 ArBuffer[BufferSize];
		FBufferWriter WriteAr(ArBuffer, BufferSize);
		int64 WriteValue = Value;
		SerializeVarInt(WriteAr, WriteValue);
		CHECK(WriteValue == Value);
		CHECK(WriteAr.Tell() == WriteByteCount);
		FBufferReader ReadAr(ArBuffer, BufferSize, /*bFreeOnClose*/ false);
		int64 ReadValue;
		SerializeVarInt(ReadAr, ReadValue);
		CHECK(ReadValue == Value);
		CHECK(ReadAr.Tell() == ReadByteCount);
	}

	SECTION("Test Read/WriteVarUInt at unsigned 64-bit encoding boundaries.")
	{
		const uint64 Value = GENERATE(as<uint64>{},
			0x0000'0000'0000'0000, 0x0000'0000'0000'007f,
			0x0000'0000'0000'0080, 0x0000'0000'0000'3fff,
			0x0000'0000'0000'4000, 0x0000'0000'0000'7fff,
			0x0000'0000'0000'8000, 0x0000'0000'0000'ffff,
			0x0000'0000'001f'ffff, 0x0000'0000'0020'0000,
			0x0000'0000'0fff'ffff, 0x0000'0000'1000'0000,
			0x0000'0000'7fff'ffff, 0x0000'0000'8000'0000,
			0x0000'0000'ffff'ffff, 0x0000'0007'ffff'ffff,
			0x0000'0008'0000'0000, 0x0000'03ff'ffff'ffff,
			0x0000'0400'0000'0000, 0x0001'ffff'ffff'ffff,
			0x0002'0000'0000'0000, 0x00ff'ffff'ffff'ffff,
			0x0100'0000'0000'0000, 0xffff'ffff'ffff'ffff);
		CAPTURE(Value);

		constexpr int32 BufferSize = 9;

		uint8 Buffer[BufferSize];
		const uint32 WriteByteCount = WriteVarUInt(Value, Buffer);
		CHECK(WriteByteCount <= BufferSize);
		uint32 ReadByteCount = 0;
		CHECK(ReadVarUInt(Buffer, ReadByteCount) == Value);
		CHECK(ReadByteCount == WriteByteCount);

		uint8 ArBuffer[BufferSize];
		FBufferWriter WriteAr(ArBuffer, BufferSize);
		uint64 WriteValue = Value;
		SerializeVarUInt(WriteAr, WriteValue);
		CHECK(WriteValue == Value);
		CHECK(WriteAr.Tell() == WriteByteCount);
		FBufferReader ReadAr(ArBuffer, BufferSize, /*bFreeOnClose*/ false);
		uint64 ReadValue;
		SerializeVarUInt(ReadAr, ReadValue);
		CHECK(ReadValue == Value);
		CHECK(ReadAr.Tell() == ReadByteCount);
	}
}

#endif