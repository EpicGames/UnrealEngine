// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "String/CaseConversion.h"
#include "Containers/StringView.h"
#include "Misc/StringBuilder.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::String
{

TEST_CASE_NAMED(FStringUpperCaseTest, "System::Core::String::UpperCase", "[Core][String][SmokeFilter]")
{
	SECTION("UpperCase with ANSICHAR")
	{
		CHECK(WriteToAnsiString<128>(UpperCase(ANSITEXTVIEW("no uppercase"))) == ANSITEXTVIEW("NO UPPERCASE"));
		CHECK(WriteToAnsiString<128>(UpperCase(ANSITEXTVIEW("Mixed CASE"))) == ANSITEXTVIEW("MIXED CASE"));
		CHECK(WriteToAnsiString<128>(UpperCase(ANSITEXTVIEW("ALL UPPERCASE"))) == ANSITEXTVIEW("ALL UPPERCASE"));
		CHECK(WriteToAnsiString<128>(UpperCase(ANSITEXTVIEW("  spaces   and\ttabs\n"))) == ANSITEXTVIEW("  SPACES   AND\tTABS\n"));
		CHECK(WriteToAnsiString<128>(UpperCase(ANSITEXTVIEW(""))) == ANSITEXTVIEW(""));
		CHECK(WriteToAnsiString<128>(UpperCase(ANSITEXTVIEW("a"))) == ANSITEXTVIEW("A"));
		CHECK(WriteToAnsiString<128>(UpperCase(ANSITEXTVIEW("Z"))) == ANSITEXTVIEW("Z"));
		CHECK(WriteToAnsiString<128>(UpperCase(ANSITEXTVIEW("1234!@#$"))) == ANSITEXTVIEW("1234!@#$"));
	}

	SECTION("UpperCase with WIDECHAR")
	{
		CHECK(WriteToWideString<128>(UpperCase(WIDETEXTVIEW("no uppercase"))) == WIDETEXTVIEW("NO UPPERCASE"));
		CHECK(WriteToWideString<128>(UpperCase(WIDETEXTVIEW("Mixed CASE"))) == WIDETEXTVIEW("MIXED CASE"));
		CHECK(WriteToWideString<128>(UpperCase(WIDETEXTVIEW("ALL UPPERCASE"))) == WIDETEXTVIEW("ALL UPPERCASE"));
		CHECK(WriteToWideString<128>(UpperCase(WIDETEXTVIEW("  spaces   and\ttabs\n"))) == WIDETEXTVIEW("  SPACES   AND\tTABS\n"));
		CHECK(WriteToWideString<128>(UpperCase(WIDETEXTVIEW(""))) == WIDETEXTVIEW(""));
		CHECK(WriteToWideString<128>(UpperCase(WIDETEXTVIEW("a"))) == WIDETEXTVIEW("A"));
		CHECK(WriteToWideString<128>(UpperCase(WIDETEXTVIEW("Z"))) == WIDETEXTVIEW("Z"));
		CHECK(WriteToWideString<128>(UpperCase(WIDETEXTVIEW("1234!@#$"))) == WIDETEXTVIEW("1234!@#$"));
	}

	SECTION("UpperCase with UTF8CHAR")
	{
		CHECK(WriteToUtf8String<128>(UpperCase(UTF8TEXTVIEW("no uppercase"))) == UTF8TEXTVIEW("NO UPPERCASE"));
		CHECK(WriteToUtf8String<128>(UpperCase(UTF8TEXTVIEW("Mixed CASE"))) == UTF8TEXTVIEW("MIXED CASE"));
		CHECK(WriteToUtf8String<128>(UpperCase(UTF8TEXTVIEW("ALL UPPERCASE"))) == UTF8TEXTVIEW("ALL UPPERCASE"));
		CHECK(WriteToUtf8String<128>(UpperCase(UTF8TEXTVIEW("  spaces   and\ttabs\n"))) == UTF8TEXTVIEW("  SPACES   AND\tTABS\n"));
		CHECK(WriteToUtf8String<128>(UpperCase(UTF8TEXTVIEW(""))) == UTF8TEXTVIEW(""));
		CHECK(WriteToUtf8String<128>(UpperCase(UTF8TEXTVIEW("a"))) == UTF8TEXTVIEW("A"));
		CHECK(WriteToUtf8String<128>(UpperCase(UTF8TEXTVIEW("Z"))) == UTF8TEXTVIEW("Z"));
		CHECK(WriteToUtf8String<128>(UpperCase(UTF8TEXTVIEW("1234!@#$"))) == UTF8TEXTVIEW("1234!@#$"));
	}
}

TEST_CASE_NAMED(FStringLowerCaseTest, "System::Core::String::LowerCase", "[Core][String][SmokeFilter]")
{
	SECTION("LowerCase with ANSICHAR")
	{
		CHECK(WriteToAnsiString<128>(LowerCase(ANSITEXTVIEW("NO LOWERCASE"))) == ANSITEXTVIEW("no lowercase"));
		CHECK(WriteToAnsiString<128>(LowerCase(ANSITEXTVIEW("Mixed CASE"))) == ANSITEXTVIEW("mixed case"));
		CHECK(WriteToAnsiString<128>(LowerCase(ANSITEXTVIEW("already lowercase"))) == ANSITEXTVIEW("already lowercase"));
		CHECK(WriteToAnsiString<128>(LowerCase(ANSITEXTVIEW("  SPACES   AND\tTABS\n"))) == ANSITEXTVIEW("  spaces   and\ttabs\n"));
		CHECK(WriteToAnsiString<128>(LowerCase(ANSITEXTVIEW(""))) == ANSITEXTVIEW(""));
		CHECK(WriteToAnsiString<128>(LowerCase(ANSITEXTVIEW("A"))) == ANSITEXTVIEW("a"));
		CHECK(WriteToAnsiString<128>(LowerCase(ANSITEXTVIEW("z"))) == ANSITEXTVIEW("z"));
		CHECK(WriteToAnsiString<128>(LowerCase(ANSITEXTVIEW("1234!@#$"))) == ANSITEXTVIEW("1234!@#$"));
	}

	SECTION("LowerCase with WIDECHAR")
	{
		CHECK(WriteToWideString<128>(LowerCase(WIDETEXTVIEW("NO LOWERCASE"))) == WIDETEXTVIEW("no lowercase"));
		CHECK(WriteToWideString<128>(LowerCase(WIDETEXTVIEW("Mixed CASE"))) == WIDETEXTVIEW("mixed case"));
		CHECK(WriteToWideString<128>(LowerCase(WIDETEXTVIEW("already lowercase"))) == WIDETEXTVIEW("already lowercase"));
		CHECK(WriteToWideString<128>(LowerCase(WIDETEXTVIEW("  SPACES   AND\tTABS\n"))) == WIDETEXTVIEW("  spaces   and\ttabs\n"));
		CHECK(WriteToWideString<128>(LowerCase(WIDETEXTVIEW(""))) == WIDETEXTVIEW(""));
		CHECK(WriteToWideString<128>(LowerCase(WIDETEXTVIEW("A"))) == WIDETEXTVIEW("a"));
		CHECK(WriteToWideString<128>(LowerCase(WIDETEXTVIEW("z"))) == WIDETEXTVIEW("z"));
		CHECK(WriteToWideString<128>(LowerCase(WIDETEXTVIEW("1234!@#$"))) == WIDETEXTVIEW("1234!@#$"));
	}

	SECTION("LowerCase with UTF8CHAR")
	{
		CHECK(WriteToUtf8String<128>(LowerCase(UTF8TEXTVIEW("NO LOWERCASE"))) == UTF8TEXTVIEW("no lowercase"));
		CHECK(WriteToUtf8String<128>(LowerCase(UTF8TEXTVIEW("Mixed CASE"))) == UTF8TEXTVIEW("mixed case"));
		CHECK(WriteToUtf8String<128>(LowerCase(UTF8TEXTVIEW("already lowercase"))) == UTF8TEXTVIEW("already lowercase"));
		CHECK(WriteToUtf8String<128>(LowerCase(UTF8TEXTVIEW("  SPACES   AND\tTABS\n"))) == UTF8TEXTVIEW("  spaces   and\ttabs\n"));
		CHECK(WriteToUtf8String<128>(LowerCase(UTF8TEXTVIEW(""))) == UTF8TEXTVIEW(""));
		CHECK(WriteToUtf8String<128>(LowerCase(UTF8TEXTVIEW("A"))) == UTF8TEXTVIEW("a"));
		CHECK(WriteToUtf8String<128>(LowerCase(UTF8TEXTVIEW("z"))) == UTF8TEXTVIEW("z"));
		CHECK(WriteToUtf8String<128>(LowerCase(UTF8TEXTVIEW("1234!@#$"))) == UTF8TEXTVIEW("1234!@#$"));
	}
}

TEST_CASE_NAMED(FStringPascalCaseTest, "System::Core::String::PascalCase", "[Core][String][SmokeFilter]")
{
	SECTION("PascalCase with WIDECHAR")
	{
		CHECK(WriteToWideString<128>(PascalCase(WIDETEXTVIEW("NO PASCALCASE"))) == WIDETEXTVIEW("NoPascalcase"));
		CHECK(WriteToWideString<128>(PascalCase(WIDETEXTVIEW("Mixed CASE"))) == WIDETEXTVIEW("MixedCase"));
		CHECK(WriteToWideString<128>(PascalCase(WIDETEXTVIEW("lowercase words"))) == WIDETEXTVIEW("LowercaseWords"));
		CHECK(WriteToWideString<128>(PascalCase(WIDETEXTVIEW("  SPACES   AND\tTABS\n"))) == WIDETEXTVIEW("SpacesAndTabs\n"));
		CHECK(WriteToWideString<128>(PascalCase(WIDETEXTVIEW(""))) == WIDETEXTVIEW(""));
		CHECK(WriteToWideString<128>(PascalCase(WIDETEXTVIEW("A"))) == WIDETEXTVIEW("A"));
		CHECK(WriteToWideString<128>(PascalCase(WIDETEXTVIEW("z"))) == WIDETEXTVIEW("Z"));
		CHECK(WriteToWideString<128>(PascalCase(WIDETEXTVIEW("1234!@#$"))) == WIDETEXTVIEW("1234!@#$"));
		CHECK(WriteToWideString<128>(PascalCase(WIDETEXTVIEW("You're Text"))) == WIDETEXTVIEW("YoureText"));
	}
}

} // UE::String

#endif //WITH_TESTS
