// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "String/Escape.h"

#include "Containers/StringView.h"
#include "Misc/StringBuilder.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::String
{

TEST_CASE_NAMED(FStringEscapeTest, "System::Core::String::Escape", "[Core][String][SmokeFilter]")
{
	SECTION("Escape")
	{
		CHECK(WriteToString<128>(Escape(TEXTVIEW("No escaping necessary."))) == TEXTVIEW("No escaping necessary."));
		CHECK(WriteToString<128>(Escape(TEXTVIEW("Tab is \t & \" is quote."))) == TEXTVIEW("Tab is \\t & \\\" is quote."));
		CHECK(WriteToString<128>(Escape(TEXTVIEW("\a\b\f\n\r\t\v\"\'\\"))) == TEXTVIEW("\\a\\b\\f\\n\\r\\t\\v\\\"\\\'\\\\"));
		CHECK(WriteToString<128>(Escape(TEXTVIEW("\x00\x01\x02\x03\x1c\x1d\x1e\x1f\x7f"))) == TEXTVIEW("\\x00\\x01\\x02\\x03\\x1c\\x1d\\x1e\\x1f\\x7f"));
		CHECK(WriteToString<128>(Escape(TEXTVIEW("\u0080\u0081\u0082\u0083"))) == TEXTVIEW("\\u0080\\u0081\\u0082\\u0083"));
		CHECK(WriteToString<128>(Escape(TEXTVIEW("\u00a1\u00a2\u00a3\u00a4"))) == TEXTVIEW("\\u00a1\\u00a2\\u00a3\\u00a4"));
		CHECK(WriteToString<128>(Escape(TEXTVIEW("\u0101\u0102\u0103\u0104"))) == TEXTVIEW("\\u0101\\u0102\\u0103\\u0104"));
	}

	SECTION("QuoteEscape")
	{
		CHECK(WriteToString<128>(QuoteEscape(TEXTVIEW("No escaping necessary."))) == TEXTVIEW("\"No escaping necessary.\""));
		CHECK(WriteToString<128>(QuoteEscape(TEXTVIEW("Tab is \t & \" is quote."))) == TEXTVIEW("\"Tab is \\t & \\\" is quote.\""));
		CHECK(WriteToString<128>(QuoteEscape(TEXTVIEW("\a\b\f\n\r\t\v\"\'\\"))) == TEXTVIEW("\"\\a\\b\\f\\n\\r\\t\\v\\\"\\\'\\\\\""));
		CHECK(WriteToString<128>(QuoteEscape(TEXTVIEW("\x00\x01\x02\x03\x1c\x1d\x1e\x1f\x7f"))) == TEXTVIEW("\"\\x00\\x01\\x02\\x03\\x1c\\x1d\\x1e\\x1f\\x7f\""));
		CHECK(WriteToString<128>(QuoteEscape(TEXTVIEW("\u0080\u0081\u0082\u0083"))) == TEXTVIEW("\"\\u0080\\u0081\\u0082\\u0083\""));
		CHECK(WriteToString<128>(QuoteEscape(TEXTVIEW("\u00a1\u00a2\u00a3\u00a4"))) == TEXTVIEW("\"\\u00a1\\u00a2\\u00a3\\u00a4\""));
		CHECK(WriteToString<128>(QuoteEscape(TEXTVIEW("\u0101\u0102\u0103\u0104"))) == TEXTVIEW("\"\\u0101\\u0102\\u0103\\u0104\""));
	}
}

} // UE::String

#endif //WITH_TESTS