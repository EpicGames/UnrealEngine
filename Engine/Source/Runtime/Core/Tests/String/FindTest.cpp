// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "String/Find.h"

#include "Containers/StringView.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::String
{

TEST_CASE_NAMED(FStringFindTest, "System::Core::String::Find", "[Core][String][SmokeFilter]")
{
	SECTION("FindFirst")
	{
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("A")) == 0);
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("a"), ESearchCase::IgnoreCase) == 0);
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("b")) == 1);
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("B")) == 4);
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("B"), ESearchCase::IgnoreCase) == 1);
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("a")) == INDEX_NONE);
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("D"), ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindFirst(TEXT("AbCABCAbCABD"), TEXT("D")) == 11);
		CHECK(FindFirst(TEXT("AbCABCAbCABD"), TEXT("d"), ESearchCase::IgnoreCase) == 11);

		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("AbC")) == 0);
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("ABC")) == 3);
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("Bc"), ESearchCase::IgnoreCase) == 1);
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("ab")) == INDEX_NONE);
		CHECK(FindFirst(TEXT("AbCABCAbCABC"), TEXT("CD"), ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindFirst(TEXT("AbCABCAbCABD"), TEXT("BD")) == 10);
		CHECK(FindFirst(TEXT("AbCABCAbCABD"), TEXT("Bd"), ESearchCase::IgnoreCase) == 10);

		CHECK(FindFirst(TEXT(""), TEXT("A")) == INDEX_NONE);
		CHECK(FindFirst(TEXT(""), TEXT("")) == 0);
		CHECK(FindFirst(TEXT("A"), TEXT("")) == 0);
		CHECK(FindFirst(TEXT("A"), TEXT("A")) == 0);
		CHECK(FindFirst(TEXT("A"), TEXT("A"), ESearchCase::IgnoreCase) == 0);
		CHECK(FindFirst(TEXT("ABC"), TEXT("ABC")) == 0);
		CHECK(FindFirst(TEXT("ABC"), TEXT("abc"), ESearchCase::IgnoreCase) == 0);
		CHECK(FindFirst(TEXT("AB"), TEXT("ABC")) == INDEX_NONE);

		CHECK(FindFirst("AbCABCAbCABC", "ABC") == 3);

		CHECK(FindFirst(FStringView(nullptr, 0), TEXT("SearchTerm")) == INDEX_NONE); //-V575
		CHECK(FindFirst(FStringView(), TEXT("SearchTerm")) == INDEX_NONE);
		CHECK(FindFirst(FStringView(), FStringView()) == 0);
		CHECK(FindFirst(TEXT("A"), FStringView()) == 0);
	}

	SECTION("FindLast")
	{
		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("b")) == 7);
		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("B")) == 10);
		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("b"), ESearchCase::IgnoreCase) == 10);
		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("a")) == INDEX_NONE);
		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("D"), ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindLast(TEXT("AbCABCAbCABD"), TEXT("D")) == 11);
		CHECK(FindLast(TEXT("AbCABCAbCABD"), TEXT("d"), ESearchCase::IgnoreCase) == 11);

		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("AbC")) == 6);
		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("ABC")) == 9);
		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("ab")) == INDEX_NONE);
		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("CD"), ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("BC")) == 10);
		CHECK(FindLast(TEXT("AbCABCAbCABC"), TEXT("Bc"), ESearchCase::IgnoreCase) == 10);

		CHECK(FindLast(TEXT(""), TEXT("A")) == INDEX_NONE);
		CHECK(FindLast(TEXT(""), TEXT("")) == 0);
		CHECK(FindLast(TEXT("A"), TEXT("")) == 1);
		CHECK(FindLast(TEXT("A"), TEXT("A")) == 0);
		CHECK(FindLast(TEXT("A"), TEXT("A"), ESearchCase::IgnoreCase) == 0);
		CHECK(FindLast(TEXT("ABC"), TEXT("ABC")) == 0);
		CHECK(FindLast(TEXT("ABC"), TEXT("abc"), ESearchCase::IgnoreCase) == 0);
		CHECK(FindLast(TEXT("AB"), TEXT("ABC")) == INDEX_NONE);

		CHECK(FindLast("AbCABCAbCABC", "ABC") == 9);

		CHECK(FindLast(FStringView(nullptr, 0), TEXT("SearchTerm")) == INDEX_NONE); //-V575
		CHECK(FindLast(FStringView(), TEXT("SearchTerm")) == INDEX_NONE);
		CHECK(FindLast(FStringView(), FStringView()) == 0);
		CHECK(FindLast(TEXT("A"), FStringView()) == 1);
	}

	SECTION("FindFirstOfAny")
	{
		CHECK(FindFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("A"), TEXT("B")}) == 0);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("a"), TEXT("B")}, ESearchCase::IgnoreCase) == 0);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("b")}) == 1);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("c"), TEXT("B")}) == 4);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("c"), TEXT("B")}, ESearchCase::IgnoreCase) == 1);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("D"), TEXT("a")}) == INDEX_NONE);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("E"), TEXT("D")}, ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbcABD"), {TEXT("E"), TEXT("D")}) == 11);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbcABD"), {TEXT("E"), TEXT("d")}, ESearchCase::IgnoreCase) == 11);

		CHECK(FindFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("AbC")}) == 0);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("CABc"), TEXT("ABC")}) == 3);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("ABD"), TEXT("Bc")}, ESearchCase::IgnoreCase) == 1);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("bc"), TEXT("ab")}) == INDEX_NONE);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("DA"), TEXT("CD")}, ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbCABD"), {TEXT("BD"), TEXT("CABB")}) == 10);
		CHECK(FindFirstOfAny(TEXT("AbCABCAbCABD"), {TEXT("Bd"), TEXT("CABB")}, ESearchCase::IgnoreCase) == 10);

		CHECK(FindFirstOfAny(TEXT(""), {TEXT("AB")}) == INDEX_NONE);
		CHECK(FindFirstOfAny(TEXT(""), {TEXT("A"), TEXT("B")}) == INDEX_NONE);
		CHECK(FindFirstOfAny(TEXT("A"), {TEXT("A"), TEXT("B")}) == 0);
		CHECK(FindFirstOfAny(TEXT("A"), {TEXT("A"), TEXT("B")}, ESearchCase::IgnoreCase) == 0);
		CHECK(FindFirstOfAny(TEXT("ABC"), {TEXT("ABC"), TEXT("BC")}) == 0);
		CHECK(FindFirstOfAny(TEXT("ABC"), {TEXT("abc"), TEXT("bc")}, ESearchCase::IgnoreCase) == 0);
		CHECK(FindFirstOfAny(TEXT("AB"), {TEXT("ABC"), TEXT("ABD")}) == INDEX_NONE);

		CHECK(FindFirstOfAny(ANSITEXTVIEW("AbCABCAbCABC"), {"CABc", "ABC"}) == 3);

		CHECK(FindFirstOfAny(FStringView(nullptr, 0), {TEXT("ABC"), TEXT("ABD")}) == INDEX_NONE); //-V575
		CHECK(FindFirstOfAny(FStringView(), {TEXT("ABC"), TEXT("ABD")}) == INDEX_NONE);

		int32 MatchIndex = INDEX_NONE;
		FindFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("ABC")}, ESearchCase::CaseSensitive, &MatchIndex);
		CHECK(MatchIndex == 0);
		FindFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("CAbC"), TEXT("ABC")}, ESearchCase::CaseSensitive, &MatchIndex);
		CHECK(MatchIndex == 1);
		FindFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("DEF"), TEXT("ABCD"), TEXT("Cab")}, ESearchCase::IgnoreCase, &MatchIndex);
		CHECK(MatchIndex == 2);

		CHECK(FindFirstOfAny(TEXT(""), {TEXT("")}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 0);
		CHECK(FindFirstOfAny(TEXT(""), {TEXT("AB"), TEXT("")}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 1);

		CHECK(FindFirstOfAny(FStringView(), {FStringView()}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 0);
		CHECK(FindFirstOfAny(FStringView(), {TEXTVIEW("AB"), FStringView()}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 1);

		CHECK(FindFirstOfAny(TEXT("ABC"), {TEXT("")}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 0);
		CHECK(FindFirstOfAny(TEXT("--abCDABCD--"), {TEXT("XY"), TEXT("")}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 1);
		CHECK(FindFirstOfAny(TEXT("--abCDABCD--"), {TEXT("AB"), TEXT("CD"), TEXT("")}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 2);

		CHECK(FindFirstOfAny(TEXT("ABC"), {FStringView()}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 0);
		CHECK(FindFirstOfAny(TEXTVIEW("--abCDABCD--"), {TEXTVIEW("XY"), FStringView()}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 1);
		CHECK(FindFirstOfAny(TEXTVIEW("--abCDABCD--"), {TEXTVIEW("AB"), TEXTVIEW("CD"), FStringView()}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 2);
	}

	SECTION("FindLastOfAny")
	{
		CHECK(FindLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("a"), TEXT("b")}) == 7);
		CHECK(FindLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("a"), TEXT("b")}, ESearchCase::IgnoreCase) == 10);
		CHECK(FindLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("b")}) == 7);
		CHECK(FindLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("c"), TEXT("B")}) == 10);
		CHECK(FindLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("c"), TEXT("B")}, ESearchCase::IgnoreCase) == 11);
		CHECK(FindLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("D"), TEXT("a")}) == INDEX_NONE);
		CHECK(FindLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("E"), TEXT("D")}, ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindLastOfAny(TEXT("AbCABCAbcABD"), {TEXT("E"), TEXT("D")}) == 11);
		CHECK(FindLastOfAny(TEXT("AbCABCAbcABD"), {TEXT("E"), TEXT("d")}, ESearchCase::IgnoreCase) == 11);

		CHECK(FindLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("AbC")}) == 6);
		CHECK(FindLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("CABc"), TEXT("ABC")}) == 9);
		CHECK(FindLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("ABD"), TEXT("Bc")}, ESearchCase::IgnoreCase) == 10);
		CHECK(FindLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("bc"), TEXT("ab")}) == INDEX_NONE);
		CHECK(FindLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("DA"), TEXT("CD")}, ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindLastOfAny(TEXT("AbCABCAbCABD"), {TEXT("BD"), TEXT("CABB")}) == 10);
		CHECK(FindLastOfAny(TEXT("AbCABCAbCABD"), {TEXT("Bd"), TEXT("CABB")}, ESearchCase::IgnoreCase) == 10);

		CHECK(FindLastOfAny(TEXT(""), {TEXT("AB")}) == INDEX_NONE);
		CHECK(FindLastOfAny(TEXT(""), {TEXT("A"), TEXT("B")}) == INDEX_NONE);
		CHECK(FindLastOfAny(TEXT("A"), {TEXT("A"), TEXT("B")}) == 0);
		CHECK(FindLastOfAny(TEXT("A"), {TEXT("A"), TEXT("B")}, ESearchCase::IgnoreCase) == 0);
		CHECK(FindLastOfAny(TEXT("ABC"), {TEXT("ABC"), TEXT("BC")}) == 1);
		CHECK(FindLastOfAny(TEXT("ABC"), {TEXT("abc"), TEXT("bc")}, ESearchCase::IgnoreCase) == 1);
		CHECK(FindLastOfAny(TEXT("AB"), {TEXT("ABC"), TEXT("ABD")}) == INDEX_NONE);

		CHECK(FindLastOfAny(ANSITEXTVIEW("AbCABCAbCABC"), {"CABc", "ABC"}) == 9);

		CHECK(FindLastOfAny(FStringView(nullptr, 0), {TEXT("ABC"), TEXT("ABD")}) == INDEX_NONE); //-V575
		CHECK(FindLastOfAny(FStringView(), {TEXT("ABC"), TEXT("ABD")}) == INDEX_NONE);

		int32 MatchIndex = INDEX_NONE;
		FindLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("AbC")}, ESearchCase::CaseSensitive, &MatchIndex);
		CHECK(MatchIndex == 0);
		FindLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("CAbC"), TEXT("AbC")}, ESearchCase::CaseSensitive, &MatchIndex);
		CHECK(MatchIndex == 1);
		FindLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("DEF"), TEXT("CAB"), TEXT("AbC")}, ESearchCase::IgnoreCase, &MatchIndex);
		CHECK(MatchIndex == 2);

		CHECK(FindLastOfAny(TEXT(""), {TEXT("")}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 0);
		CHECK(FindLastOfAny(TEXT(""), {TEXT("AB"), TEXT("")}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 1);

		CHECK(FindLastOfAny(FStringView(), {FStringView()}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 0);
		CHECK(FindLastOfAny(FStringView(), {TEXTVIEW("AB"), FStringView()}, ESearchCase::IgnoreCase, &MatchIndex) == 0);
		CHECK(MatchIndex == 1);

		CHECK(FindLastOfAny(TEXT("ABC"), {TEXT("")}, ESearchCase::IgnoreCase, &MatchIndex) == 3);
		CHECK(MatchIndex == 0);
		CHECK(FindLastOfAny(TEXT("--ABCDABcd--"), {TEXT("XY"), TEXT("")}, ESearchCase::IgnoreCase, &MatchIndex) == 12);
		CHECK(MatchIndex == 1);
		CHECK(FindLastOfAny(TEXT("--ABCDABcd--"), {TEXT("AB"), TEXT("CD"), TEXT("")}, ESearchCase::IgnoreCase, &MatchIndex) == 12);
		CHECK(MatchIndex == 2);

		CHECK(FindLastOfAny(TEXT("ABC"), {FStringView()}, ESearchCase::IgnoreCase, &MatchIndex) == 3);
		CHECK(MatchIndex == 0);
		CHECK(FindLastOfAny(TEXT("--ABCDABcd--"), {TEXTVIEW("XY"), FStringView()}, ESearchCase::IgnoreCase, &MatchIndex) == 12);
		CHECK(MatchIndex == 1);
		CHECK(FindLastOfAny(TEXT("--ABCDABcd--"), {TEXTVIEW("AB"), TEXTVIEW("CD"), FStringView()}, ESearchCase::IgnoreCase, &MatchIndex) == 12);
		CHECK(MatchIndex == 2);
	}

	SECTION("FindFirstChar")
	{
		CHECK(FindFirstChar(TEXT("AbCABCAbCABC"), TEXT('b')) == 1);
		CHECK(FindFirstChar(TEXT("AbCABCAbCABC"), TEXT('B')) == 4);
		CHECK(FindFirstChar(TEXT("AbCABCAbCABC"), TEXT('B'), ESearchCase::IgnoreCase) == 1);
		CHECK(FindFirstChar(TEXT("AbCABCAbCABC"), TEXT('a')) == INDEX_NONE);
		CHECK(FindFirstChar(TEXT("AbCABCAbCABC"), TEXT('D'), ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindFirstChar(TEXT("AbCABCAbCABD"), TEXT('D')) == 11);
		CHECK(FindFirstChar(TEXT("AbCABCAbCABD"), TEXT('d'), ESearchCase::IgnoreCase) == 11);

		CHECK(FindFirstChar(TEXT(""), TEXT('A')) == INDEX_NONE);
		CHECK(FindFirstChar(TEXT("A"), TEXT('A')) == 0);
		CHECK(FindFirstChar(TEXT("A"), TEXT('A'), ESearchCase::IgnoreCase) == 0);

		CHECK(FindFirstChar("AbCABCAbCABC", 'B') == 4);

		CHECK(FindFirstChar(FStringView(nullptr, 0), TEXT('A')) == INDEX_NONE); //-V575
		CHECK(FindFirstChar(FStringView(), TEXT('A')) == INDEX_NONE);
	}

	SECTION("FindLastChar")
	{
		CHECK(FindLastChar(TEXT("AbCABCAbCABC"), TEXT('b')) == 7);
		CHECK(FindLastChar(TEXT("AbCABCAbCABC"), TEXT('B')) == 10);
		CHECK(FindLastChar(TEXT("AbCABCAbCABC"), TEXT('b'), ESearchCase::IgnoreCase) == 10);
		CHECK(FindLastChar(TEXT("AbCABCAbCABC"), TEXT('a')) == INDEX_NONE);
		CHECK(FindLastChar(TEXT("AbCABCAbCABC"), TEXT('D'), ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindLastChar(TEXT("AbCABCAbCABD"), TEXT('D')) == 11);
		CHECK(FindLastChar(TEXT("AbCABCAbCABD"), TEXT('d'), ESearchCase::IgnoreCase) == 11);

		CHECK(FindLastChar(TEXT(""), TEXT('A')) == INDEX_NONE);
		CHECK(FindLastChar(TEXT("A"), TEXT('A')) == 0);
		CHECK(FindLastChar(TEXT("A"), TEXT('A'), ESearchCase::IgnoreCase) == 0);

		CHECK(FindLastChar("AbCABCAbCABC", 'B') == 10);

		CHECK(FindLastChar(FStringView(nullptr, 0), TEXT('A')) == INDEX_NONE); //-V575
		CHECK(FindLastChar(FStringView(), TEXT('A')) == INDEX_NONE);
	}

	SECTION("FindFirstOfAnyChar")
	{
		CHECK(FindFirstOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('b')}) == 1);
		CHECK(FindFirstOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('c'), TEXT('B')}) == 4);
		CHECK(FindFirstOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('c'), TEXT('B')}, ESearchCase::IgnoreCase) == 1);
		CHECK(FindFirstOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('D'), TEXT('a')}) == INDEX_NONE);
		CHECK(FindFirstOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('E'), TEXT('D')}, ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindFirstOfAnyChar(TEXT("AbCABCAbcABD"), {TEXT('E'), TEXT('D')}) == 11);
		CHECK(FindFirstOfAnyChar(TEXT("AbCABCAbcABD"), {TEXT('E'), TEXT('d')}, ESearchCase::IgnoreCase) == 11);

		CHECK(FindFirstOfAnyChar(TEXT(""), {TEXT('A'), TEXT('B')}) == INDEX_NONE);
		CHECK(FindFirstOfAnyChar(TEXT("A"), {TEXT('A'), TEXT('B')}) == 0);
		CHECK(FindFirstOfAnyChar(TEXT("A"), {TEXT('A'), TEXT('B')}, ESearchCase::IgnoreCase) == 0);

		CHECK(FindFirstOfAnyChar("AbCABCAbcABC", {'c', 'B'}) == 4);

		CHECK(FindFirstOfAnyChar(FStringView(nullptr, 0), {TEXT('A'), TEXT('B')}) == INDEX_NONE); //-V575
		CHECK(FindFirstOfAnyChar(FStringView(), {TEXT('A'), TEXT('B')}) == INDEX_NONE);

		int32 MatchIndex = INDEX_NONE;
		FindFirstOfAnyChar(TEXT("AbCABCAbCABC"), {TEXT('B')}, ESearchCase::CaseSensitive, &MatchIndex);
		CHECK(MatchIndex == 0);
		FindFirstOfAnyChar(TEXT("AbCABCAbCABC"), {TEXT('B'), TEXT('A')}, ESearchCase::CaseSensitive, &MatchIndex);
		CHECK(MatchIndex == 1);
		FindFirstOfAnyChar(TEXT("AbCABCAbCABC"), {TEXT('D'), TEXT('C'), TEXT('B')}, ESearchCase::IgnoreCase, &MatchIndex);
		CHECK(MatchIndex == 2);
	}

	SECTION("FindLastOfAnyChar")
	{
		CHECK(FindLastOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('b')}) == 7);
		CHECK(FindLastOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('c'), TEXT('B')}) == 10);
		CHECK(FindLastOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('c'), TEXT('B')}, ESearchCase::IgnoreCase) == 11);
		CHECK(FindLastOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('D'), TEXT('a')}) == INDEX_NONE);
		CHECK(FindLastOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('E'), TEXT('D')}, ESearchCase::IgnoreCase) == INDEX_NONE);
		CHECK(FindLastOfAnyChar(TEXT("AbCABCAbcABD"), {TEXT('E'), TEXT('D')}) == 11);
		CHECK(FindLastOfAnyChar(TEXT("AbCABCAbcABD"), {TEXT('E'), TEXT('d')}, ESearchCase::IgnoreCase) == 11);

		CHECK(FindLastOfAnyChar(TEXT(""), {TEXT('A'), TEXT('B')}) == INDEX_NONE);
		CHECK(FindLastOfAnyChar(TEXT("A"), {TEXT('A'), TEXT('B')}) == 0);
		CHECK(FindLastOfAnyChar(TEXT("A"), {TEXT('A'), TEXT('B')}, ESearchCase::IgnoreCase) == 0);

		CHECK(FindLastOfAnyChar("AbCABCAbcABC", {'c', 'B'}) == 10);

		CHECK(FindLastOfAnyChar(FStringView(nullptr, 0), {TEXT('A'), TEXT('B')}) == INDEX_NONE); //-V575
		CHECK(FindLastOfAnyChar(FStringView(), {TEXT('A'), TEXT('B')}) == INDEX_NONE);

		int32 MatchIndex = INDEX_NONE;
		FindLastOfAnyChar(TEXT("AbCABCAbCABC"), {TEXT('A')}, ESearchCase::CaseSensitive, &MatchIndex);
		CHECK(MatchIndex == 0);
		FindLastOfAnyChar(TEXT("AbCABCAbCABC"), {TEXT('b'), TEXT('A')}, ESearchCase::CaseSensitive, &MatchIndex);
		CHECK(MatchIndex == 1);
		FindLastOfAnyChar(TEXT("AbCABCAbCABC"), {TEXT('E'), TEXT('D'), TEXT('a')}, ESearchCase::IgnoreCase, &MatchIndex);
		CHECK(MatchIndex == 2);
	}
}

} // UE::String

#endif //WITH_TESTS