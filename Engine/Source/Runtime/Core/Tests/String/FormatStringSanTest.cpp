// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#if WITH_TESTS

#include "String/FormatStringSan.h"
#include "Containers/AnsiString.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"

#include "Containers/EnumAsByte.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE
{
TEST_CASE_NAMED(FFormatStringValidatorTest, "System::Core::String::FormatStringSan", "[Core][String][FormatStringSan]")
{
	using namespace UE::Core::Private;
	using E = FormatStringSan::EFormatStringSanStatus;

	SECTION("%s")
	{
		// Test passing string pointers of the wrong element size
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsNarrowCharPtrArgButGotWide,          "Test %s",  WIDETEXT("wrong")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsNarrowCharPtrArgButGotWide, UTF8TEXT("Test %s"), WIDETEXT("wrong")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsWideCharPtrArgButGotNarrow, WIDETEXT("Test %s"),          "wrong"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsWideCharPtrArgButGotNarrow, WIDETEXT("Test %s"), UTF8TEXT("wrong")));

		// Test passing non-string pointers
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsNarrowCharPtrArg,          "Test %s",  (int*)nullptr));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsNarrowCharPtrArg, UTF8TEXT("Test %s"), (int*)nullptr));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsWideCharPtrArg,   WIDETEXT("Test %s"), (int*)nullptr));

		// Test passing non-pointers
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsNarrowCharPtrArg,          "Test %s",  5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsNarrowCharPtrArg, UTF8TEXT("Test %s"), 5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsWideCharPtrArg,   WIDETEXT("Test %s"), 5));

		// Test TString passed instead of pointer
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsDereferencedNarrowString,          "Test %s",  FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsDereferencedNarrowString,          "Test %s",  FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsDereferencedNarrowString,          "Test %s",  FWideString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsDereferencedNarrowString, UTF8TEXT("Test %s"), FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsDereferencedNarrowString, UTF8TEXT("Test %s"), FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsDereferencedNarrowString, UTF8TEXT("Test %s"), FWideString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsDereferencedWideString,   WIDETEXT("Test %s"), FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsDereferencedWideString,   WIDETEXT("Test %s"), FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsDereferencedWideString,   WIDETEXT("Test %s"), FWideString()));

		// Test char passed instead of char pointer
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsPtrButGotChar,          "Test %s",           'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsPtrButGotChar,          "Test %s",  UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsPtrButGotChar,          "Test %s",  WIDETEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsPtrButGotChar, UTF8TEXT("Test %s"),          'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsPtrButGotChar, UTF8TEXT("Test %s"), UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsPtrButGotChar, UTF8TEXT("Test %s"), WIDETEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsPtrButGotChar, WIDETEXT("Test %s"),          'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsPtrButGotChar, WIDETEXT("Test %s"), UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::SNeedsPtrButGotChar, WIDETEXT("Test %s"), WIDETEXT('x')));

		// Test matching arguments
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Test %s",           "hello"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Test %s",  UTF8TEXT("hello")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Test %s"),          "hello"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Test %s"), UTF8TEXT("hello")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Test %s"), WIDETEXT("hello")));
	}

	SECTION("%S")
	{
		// Test passing string pointers of the wrong element size
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsWideCharPtrArgButGotNarrow,          "Test %S",           "wrong"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsWideCharPtrArgButGotNarrow, UTF8TEXT("Test %S"),          "wrong"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsWideCharPtrArgButGotNarrow,          "Test %S",  UTF8TEXT("wrong")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsWideCharPtrArgButGotNarrow, UTF8TEXT("Test %S"), UTF8TEXT("wrong")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsNarrowCharPtrArgButGotWide, WIDETEXT("Test %S"), WIDETEXT("wrong")));

		// Test passing non-string pointers
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsWideCharPtrArg,            "Test %S",  (int*)nullptr));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsWideCharPtrArg,   UTF8TEXT("Test %S"), (int*)nullptr));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsNarrowCharPtrArg, WIDETEXT("Test %S"), (int*)nullptr));

		// Test passing non-pointers
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsWideCharPtrArg,            "Test %S",  5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsWideCharPtrArg,   UTF8TEXT("Test %S"), 5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsNarrowCharPtrArg, WIDETEXT("Test %S"), 5));

		// Test TString passed instead of pointer
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsDereferencedWideString,            "Test %S",  FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsDereferencedWideString,            "Test %S",  FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsDereferencedWideString,            "Test %S",  FWideString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsDereferencedWideString,   UTF8TEXT("Test %S"), FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsDereferencedWideString,   UTF8TEXT("Test %S"), FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsDereferencedWideString,   UTF8TEXT("Test %S"), FWideString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsDereferencedNarrowString, WIDETEXT("Test %S"), FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsDereferencedNarrowString, WIDETEXT("Test %S"), FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsDereferencedNarrowString, WIDETEXT("Test %S"), FWideString()));

		// Test char passed instead of char pointer
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsPtrButGotChar,          "Test %S",           'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsPtrButGotChar,          "Test %S",  UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsPtrButGotChar,          "Test %S",  WIDETEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsPtrButGotChar, UTF8TEXT("Test %S"),          'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsPtrButGotChar, UTF8TEXT("Test %S"), UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsPtrButGotChar, UTF8TEXT("Test %S"), WIDETEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsPtrButGotChar, WIDETEXT("Test %S"),          'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsPtrButGotChar, WIDETEXT("Test %S"), UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CapitalSNeedsPtrButGotChar, WIDETEXT("Test %S"), WIDETEXT('x')));

		// Test matching arguments
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Test %S",  WIDETEXT("hello")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Test %S"), WIDETEXT("hello")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Test %S"),          "hello"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Test %S"), UTF8TEXT("hello")));
	}

	SECTION("%hs")
	{
		// Test passing string pointers of the wrong element size
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsNarrowCharPtrArgButGotWideOnNarrowString,          "Test %hs",  WIDETEXT("wrong")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsNarrowCharPtrArgButGotWideOnNarrowString, UTF8TEXT("Test %hs"), WIDETEXT("wrong")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsNarrowCharPtrArgButGotWideOnWideString,   WIDETEXT("Test %hs"), WIDETEXT("wrong")));

		// Test passing non-string pointers
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsNarrowCharPtrArg,          "Test %hs",  (int*)nullptr));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsNarrowCharPtrArg, UTF8TEXT("Test %hs"), (int*)nullptr));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsNarrowCharPtrArg, WIDETEXT("Test %hs"), (int*)nullptr));

		// Test passing non-pointers
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsNarrowCharPtrArg,          "Test %hs",  5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsNarrowCharPtrArg, UTF8TEXT("Test %hs"), 5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsNarrowCharPtrArg, WIDETEXT("Test %hs"), 5));

		// Test TString passed instead of pointer
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsDereferencedNarrowString,          "Test %hs",  FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsDereferencedNarrowString,          "Test %hs",  FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsDereferencedNarrowString,          "Test %hs",  FWideString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsDereferencedNarrowString, UTF8TEXT("Test %hs"), FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsDereferencedNarrowString, UTF8TEXT("Test %hs"), FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsDereferencedNarrowString, UTF8TEXT("Test %hs"), FWideString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsDereferencedNarrowString, WIDETEXT("Test %hs"), FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsDereferencedNarrowString, WIDETEXT("Test %hs"), FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsDereferencedNarrowString, WIDETEXT("Test %hs"), FWideString()));

		// Test char passed instead of char pointer
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsPtrButGotChar,          "Test %hs",           'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsPtrButGotChar,          "Test %hs",  UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsPtrButGotChar,          "Test %hs",  WIDETEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsPtrButGotChar, UTF8TEXT("Test %hs"),          'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsPtrButGotChar, UTF8TEXT("Test %hs"), UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsPtrButGotChar, UTF8TEXT("Test %hs"), WIDETEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsPtrButGotChar, WIDETEXT("Test %hs"),          'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsPtrButGotChar, WIDETEXT("Test %hs"), UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HSNeedsPtrButGotChar, WIDETEXT("Test %hs"), WIDETEXT('x')));

		// Test matching arguments
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Test %hs",           "hello"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Test %hs",  UTF8TEXT("hello")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Test %hs"),          "hello"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Test %hs"), UTF8TEXT("hello")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Test %hs"),          "hello"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Test %hs"), UTF8TEXT("hello")));
	}

	SECTION("%ls")
	{
		// Test passing string pointers of the wrong element size
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArgButGotNarrowOnNarrowString,          "Test %ls",           "wrong"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArgButGotNarrowOnNarrowString,          "Test %ls",  UTF8TEXT("wrong")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArgButGotNarrowOnNarrowString, UTF8TEXT("Test %ls"),          "wrong"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArgButGotNarrowOnNarrowString, UTF8TEXT("Test %ls"), UTF8TEXT("wrong")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArgButGotNarrowOnWideString,   WIDETEXT("Test %ls"),          "wrong"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArgButGotNarrowOnWideString,   WIDETEXT("Test %ls"), UTF8TEXT("wrong")));

		// Test passing non-string pointers
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArg,          "Test %ls",  (int*)nullptr));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArg, UTF8TEXT("Test %ls"), (int*)nullptr));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArg, WIDETEXT("Test %ls"), (int*)nullptr));

		// Test passing non-pointers
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArg,          "Test %ls",  5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArg, UTF8TEXT("Test %ls"), 5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsWideCharPtrArg, WIDETEXT("Test %ls"), 5));

		// Test TString passed instead of pointer
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsDereferencedWideString,          "Test %ls",  FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsDereferencedWideString,          "Test %ls",  FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsDereferencedWideString,          "Test %ls",  FWideString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsDereferencedWideString, UTF8TEXT("Test %ls"), FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsDereferencedWideString, UTF8TEXT("Test %ls"), FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsDereferencedWideString, UTF8TEXT("Test %ls"), FWideString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsDereferencedWideString, WIDETEXT("Test %ls"), FAnsiString()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsDereferencedWideString, WIDETEXT("Test %ls"), FUtf8String()));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsDereferencedWideString, WIDETEXT("Test %ls"), FWideString()));

		// Test char passed instead of char pointer
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsPtrButGotChar,          "Test %ls",           'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsPtrButGotChar,          "Test %ls",  UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsPtrButGotChar,          "Test %ls",  WIDETEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsPtrButGotChar, UTF8TEXT("Test %ls"),          'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsPtrButGotChar, UTF8TEXT("Test %ls"), UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsPtrButGotChar, UTF8TEXT("Test %ls"), WIDETEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsPtrButGotChar, WIDETEXT("Test %ls"),          'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsPtrButGotChar, WIDETEXT("Test %ls"), UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LSNeedsPtrButGotChar, WIDETEXT("Test %ls"), WIDETEXT('x')));

		// Test matching arguments
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Test %ls",  WIDETEXT("hello")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Test %ls"), WIDETEXT("hello")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Test %ls"), WIDETEXT("hello")));
	}

	SECTION("%c")
	{
		enum class EFakeChar : uint8 {};

		// Test passing non-integer numbers
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnNarrowString,          "Hello %c",  42.0));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnNarrowString, UTF8TEXT("Hello %c"), 42.0));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnWideString,   WIDETEXT("Hello %c"), 42.0));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnNarrowString,          "Hello %c",  42.0f));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnNarrowString, UTF8TEXT("Hello %c"), 42.0f));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnWideString,   WIDETEXT("Hello %c"), 42.0f));

		// Test passing an enum that isn't UTF8CHAR
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnNarrowString,          "Hello %c",  EFakeChar{}));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnNarrowString, UTF8TEXT("Hello %c"), EFakeChar{}));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnWideString,   WIDETEXT("Hello %c"), EFakeChar{}));

		// Test passing a 64-bit integer
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnNarrowString,          "Hello %c",  2147483648));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnNarrowString, UTF8TEXT("Hello %c"), 2147483648));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::CNeedsCharArgOnWideString,   WIDETEXT("Hello %c"), 2147483648));

		// Test passing a 32-bit integer
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Hello %c",  42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Hello %c"), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Hello %c"), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Hello %c",  2147483647));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Hello %c"), 2147483647));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Hello %c"), 2147483647));

		// Test passing a char of any type
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Hello %c",           'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Hello %c"),          'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Hello %c"),          'x'));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Hello %c",  UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Hello %c"), UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Hello %c"), UTF8TEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,          "Hello %c",  WIDETEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, UTF8TEXT("Hello %c"), WIDETEXT('x')));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, WIDETEXT("Hello %c"), WIDETEXT('x')));
	}

	SECTION("Error Handling")
	{
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::IncompleteFormatSpecifierOrUnescapedPercent, TEXT("Hello %")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::PNeedsPointerArg, TEXT("Hello %p"), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::FNeedsFloatOrDoubleArg, TEXT("Hello %f"), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::InvalidFormatSpec, TEXT("Hello %k"), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::DNeedsIntegerArg, TEXT("Hello %d"), 42.0));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::ZNeedsIntegerSpec, TEXT("Hello %z test"), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::ZNeedsIntegerArg, TEXT("Hello %zu"), "hi"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::DynamicLengthSpecNeedsIntegerArg, TEXT("Hey %*.*d"), "hi", "hi"));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LNeedsIntegerArg, TEXT("Hello %ld"), 43.0f));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HNeedsIntegerArg, TEXT("Hello %hd"), 43.0f));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HHNeedsIntegerSpec, TEXT("Hello %hh "), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::HHNeedsIntegerArg, TEXT("Hello %hhd"), 43.0f));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LLNeedsIntegerSpec, TEXT("Hello %ll "), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::LLNeedsIntegerArg, TEXT("Hello %lld"), 43.0f));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::I64BadSpec, TEXT("Hello %I32d"), 44));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::I64BadSpec, TEXT("Hello %I64p"), 44));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::I64NeedsIntegerArg, TEXT("Hello %I64u"), 44.0f));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::InvalidFormatSpec, TEXT("%l^"), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::InvalidFormatSpec, TEXT("%h^"), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::IncompleteFormatSpecifierOrUnescapedPercent, TEXT("%-*"), 42));		
	}

	SECTION("Accepted Formatting")
	{
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("Test %d %% %% %d"), 32, 44));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("Test")));

		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%% Test %d %f %s "), 32, 44.4, TEXT("hey")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("Test %.3f %d"), 4.4, 2));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("Test %2.3f"), 4.4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("Test %2.f"), 4.4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("Test %2f"), 4.4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("Test %ld"), long(32)));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("Test %s"), TEXT("hello")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("Test percent %% more")));

		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%-8d %f"), 42, 4.4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%8d %f"), 42, 4.4));

		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%-8.8d %f"), 42, 4.4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%hhd %d"), int(42), 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%lld %d"), 42LL, 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%I64d %d "), 42LL, 42));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%zu"), size_t(44)));

		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%f"), 42.f));

		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%-*.*d %f"), 4, 8, 42, 4.4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%*.*d %f"), 4, 8, 42, 4.4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%-*.9d %d"), 4, 42, 44));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%*.9d %d"), 4, 42, 44));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%-*d %s"), 4, 42, TEXT("hi")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%*d %s"), 4, 42, TEXT("hi")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%.*f %s"), 4, 42.4, TEXT("hi")));

		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("a")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("")));

		{
			const TCHAR* Foo;
			STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("hello %s"), Foo));
		}
		{
			enum MyIntegralEnum { MyIntegralEnumA };
			STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("hello %d is an enum actually"), MyIntegralEnumA));
		}
		{
			enum class MyEnum { Value };
			STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("enum class %d value"), MyEnum::Value));
		}

		{
			enum ETestEnumAsByte { ETestEnumAsByte_Zero = 0 };
			TEnumAsByte<ETestEnumAsByte> EnumAsByteParam = ETestEnumAsByte_Zero;
			STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok, TEXT("%d"), EnumAsByteParam));
		}
	}

	SECTION("NumberOfArguments")
	{
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,                  TEXT("")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT(""), 1));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT(""), 1, 2));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT(""), 1, 2, 3));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,                  TEXT("abcxyz")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abcxyz"), 1));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abcxyz"), 1, 2));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abcxyz"), 1, 2, 3));

		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("%d")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,                  TEXT("%d"), 1));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("%d"), 1, 2));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("%d"), 1, 2, 3));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("%d"), 1, 2, 3, 4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("abc%dxyz")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,                  TEXT("abc%dxyz"), 1));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abc%dxyz"), 1, 2));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abc%dxyz"), 1, 2, 3));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abc%dxyz"), 1, 2, 3, 4));

		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("%d %d")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("%d %d"), 1));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,                  TEXT("%d %d"), 1, 2));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("%d %d"), 1, 2, 3));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("%d %d"), 1, 2, 3, 4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("%d %d"), 1, 2, 3, 4, 5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("abc%d %dxyz")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("abc%d %dxyz"), 1));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,                  TEXT("abc%d %dxyz"), 1, 2));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abc%d %dxyz"), 1, 2, 3));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abc%d %dxyz"), 1, 2, 3, 4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abc%d %dxyz"), 1, 2, 3, 4, 5));

		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("%d %d %d")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("%d %d %d"), 1));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("%d %d %d"), 1, 2));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,                  TEXT("%d %d %d"), 1, 2, 3));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("%d %d %d"), 1, 2, 3, 4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("%d %d %d"), 1, 2, 3, 4, 5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("%d %d %d"), 1, 2, 3, 4, 5, 6));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("abc%d %d %dxyz")));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("abc%d %d %dxyz"), 1));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughArguments,  TEXT("abc%d %d %dxyz"), 1, 2));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::Ok,                  TEXT("abc%d %d %dxyz"), 1, 2, 3));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abc%d %d %dxyz"), 1, 2, 3, 4));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abc%d %d %dxyz"), 1, 2, 3, 4, 5));
		STATIC_CHECK(UE_CHECK_FORMAT_STRING_ERR(E::NotEnoughSpecifiers, TEXT("abc%d %d %dxyz"), 1, 2, 3, 4, 5, 6));
	}
}

TEST_CASE_NAMED(FFormatStringConstStringValidationTest, "System::Core::String::FormatStringSan::ConstString", "[Core][String][FormatStringSan]")
{
	using namespace UE::Core::Private::FormatStringSan;

	SECTION("Valid Const String Conditions")
	{
		STATIC_CHECK(bIsAConstString<decltype("Raw CString")>);
		STATIC_CHECK(bIsAConstString<decltype(TEXT("Raw WString"))>);

		{
			const char Array[] = "CString";
			STATIC_CHECK(bIsAConstString<decltype(Array)>);
		}

		{
			const TCHAR Array[] = TEXT("WString");
			STATIC_CHECK(bIsAConstString<decltype(Array)>);
		}

		{
			const char* ConstPtr = "CString";
			STATIC_CHECK(bIsAConstString<decltype(ConstPtr)>);
		}

		{
			const TCHAR* ConstPtr = TEXT("WString");
			STATIC_CHECK(bIsAConstString<decltype(ConstPtr)>);
		}

		{
			const char* const ConstPtrConst = "CString";
			STATIC_CHECK(bIsAConstString<decltype(ConstPtrConst)>);
		}

		{
			const TCHAR* const ConstPtrConst = TEXT("WString");
			STATIC_CHECK(bIsAConstString<decltype(ConstPtrConst)>);
		}
		
		{
			struct FImplicitConvertToChar
			{
				operator const char*() const { return (const char*)this; }
			};
			FImplicitConvertToChar ToChar;
			STATIC_CHECK(bIsAConstString<decltype(ToChar)>);
		}

		{
			struct FImplicitConvertToTChar
			{
				operator const TCHAR* () const { return (const TCHAR*)this; }
			};
			FImplicitConvertToTChar ToTChar;
			STATIC_CHECK(bIsAConstString<decltype(ToTChar)>);
		}
	}

	SECTION("Invalid Const String Conditions")
	{
		bool bBool = true;
		STATIC_CHECK_FALSE(bIsAConstString<decltype(bBool)>);

		{
			char* Ptr = nullptr;
			STATIC_CHECK_FALSE(bIsAConstString<decltype(Ptr)>);
		}

		{
			TCHAR* Ptr = nullptr;
			STATIC_CHECK_FALSE(bIsAConstString<decltype(Ptr)>);
		}

		{
			char* const Ptr = nullptr;
			STATIC_CHECK_FALSE(bIsAConstString<decltype(Ptr)>);
		}

		{
			TCHAR* const Ptr = nullptr;
			STATIC_CHECK_FALSE(bIsAConstString<decltype(Ptr)>);
		}

		{
			struct FExplicitConvertToChar
			{
				explicit operator const char* () const { return (const char*)this; }
			};
			FExplicitConvertToChar ToChar;
			STATIC_CHECK_FALSE(bIsAConstString<decltype(ToChar)>);
		}

		{
			struct FExplicitConvertToTChar
			{
				explicit operator const TCHAR* () const { return (const TCHAR*)this; }
			};
			FExplicitConvertToTChar ToTChar;
			STATIC_CHECK_FALSE(bIsAConstString<decltype(ToTChar)>);
		}
	}
}

} // namespace UE

#endif // WITH_TESTS
