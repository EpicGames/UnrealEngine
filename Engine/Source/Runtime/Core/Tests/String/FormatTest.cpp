// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FStringFormattingTestStaticRequirements, "System::Core::String Formatting::Static Requirements", "[Core][String][EditorContext][SmokeFilter]")
{
	STATIC_REQUIRE(!std::is_constructible_v<FStringFormatArg>);
	STATIC_REQUIRE(std::is_copy_constructible_v<FStringFormatArg>);
	STATIC_REQUIRE(std::is_move_constructible_v<FStringFormatArg>);
	STATIC_REQUIRE(std::is_copy_assignable_v<FStringFormatArg>);
	STATIC_REQUIRE(std::is_move_assignable_v<FStringFormatArg>);
}

TEST_CASE_NAMED(FStringFormattingTestNamedSimple, "System::Core::String Formatting::Simple (Named)", "[Core][String][EditorContext][SmokeFilter]")
{
	TMap<FString, FStringFormatArg> Args;
	Args.Add(TEXT("Argument1"), TEXT("Replacement 1"));
	Args.Add(TEXT("Argument2"), TEXT("Replacement 2"));

	const TCHAR* Pattern        = TEXT("This is some text containing two arguments, { Argument1 } and {Argument2}.");
	const TCHAR* ExpectedResult = TEXT("This is some text containing two arguments, Replacement 1 and Replacement 2.");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestNamedMultiple, "System::Core::String Formatting::Multiple (Named)", "[Core][String][EditorContext][SmokeFilter]")
{
	TMap<FString, FStringFormatArg> Args;
	Args.Add(TEXT("Argument1"), TEXT("Replacement 1"));

	const TCHAR* Pattern        = TEXT("This is some text containing the same argument, { Argument1 } and {Argument1}.");
	const TCHAR* ExpectedResult = TEXT("This is some text containing the same argument, Replacement 1 and Replacement 1.");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestNamedEscaped, "System::Core::String Formatting::Escaped (Named)", "[Core][String][EditorContext][SmokeFilter]")
{
	TMap<FString, FStringFormatArg> Args;
	Args.Add(TEXT("Argument1"), TEXT("Replacement 1"));

	const TCHAR* Pattern        = TEXT("This is some text containing brace and an arg, `{ Argument1 } and {Argument1}.");
	const TCHAR* ExpectedResult = TEXT("This is some text containing brace and an arg, { Argument1 } and Replacement 1.");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestNamedUnbounded, "System::Core::String Formatting::Unbounded (Named)", "[Core][String][EditorContext][SmokeFilter]")
{
	TMap<FString, FStringFormatArg> Args;
	Args.Add(TEXT("Argument1"), TEXT("Replacement 1"));

	const TCHAR* Pattern        = TEXT("This is some text containing an unbounded arg, { Argument1");
	const TCHAR* ExpectedResult = TEXT("This is some text containing an unbounded arg, { Argument1");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestNamedNonExistent, "System::Core::String Formatting::Non Existent (Named)", "[Core][String][EditorContext][SmokeFilter]")
{
	TMap<FString, FStringFormatArg> Args;
	Args.Add(TEXT("Argument1"), TEXT("Replacement 1"));

	const TCHAR* Pattern        = TEXT("This is some text containing a non-existent arg { Argument2 }");
	const TCHAR* ExpectedResult = TEXT("This is some text containing a non-existent arg { Argument2 }");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestNamedInvalid, "System::Core::String Formatting::Invalid (Named)", "[Core][String][EditorContext][SmokeFilter]")
{
	TMap<FString, FStringFormatArg> Args;
	Args.Add(TEXT("Argument1"), TEXT("Replacement 1"));

	const TCHAR* Pattern        = TEXT("This is some text containing an invalid arg {Argument1 1}  { a8f7690-23\\ {} }");
	const TCHAR* ExpectedResult = TEXT("This is some text containing an invalid arg {Argument1 1}  { a8f7690-23\\ {} }");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestNamedEmpty, "System::Core::String Formatting::Empty (Named)", "[Core][String][EditorContext][SmokeFilter]")
{
	TMap<FString, FStringFormatArg> Args;
	Args.Add(TEXT("Argument1"), TEXT("Replacement 1"));
	Args.Add(TEXT("Argument2"), TEXT("Replacement 2"));

	const TCHAR* Pattern        = TEXT("");
	const TCHAR* ExpectedResult = TEXT("");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestOrderedSimple, "System::Core::String Formatting::Simple (Ordered)", "[Core][String][EditorContext][SmokeFilter]")
{
	TArray<FStringFormatArg> Args;
	Args.Add(TEXT("Replacement 1"));
	Args.Add(TEXT("Replacement 2"));

	const TCHAR* Pattern        = TEXT("This is some text containing two arguments, { 0 } and {1}.");
	const TCHAR* ExpectedResult = TEXT("This is some text containing two arguments, Replacement 1 and Replacement 2.");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestOrderedMultiple, "System::Core::String Formatting::Multiple (Ordered)", "[Core][String][EditorContext][SmokeFilter]")
{
	TArray<FStringFormatArg> Args;
	Args.Add(TEXT("Replacement 1"));

	const TCHAR* Pattern        = TEXT("This is some text containing the same argument, { 0 } and {0}.");
	const TCHAR* ExpectedResult = TEXT("This is some text containing the same argument, Replacement 1 and Replacement 1.");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestOrderedEscaped, "System::Core::String Formatting::Escaped (Ordered)", "[Core][String][EditorContext][SmokeFilter]")
{
	TArray<FStringFormatArg> Args;
	Args.Add(TEXT("Replacement 1"));

	const TCHAR* Pattern        = TEXT("This is some text containing brace and an arg, `{ 0 } and {0}.");
	const TCHAR* ExpectedResult = TEXT("This is some text containing brace and an arg, { 0 } and Replacement 1.");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestOrderedUnbounded, "System::Core::String Formatting::Unbounded (Ordered)", "[Core][String][EditorContext][SmokeFilter]")
{
	TArray<FStringFormatArg> Args;
	Args.Add(TEXT("Replacement 1"));

	const TCHAR* Pattern        = TEXT("This is some text containing an unbounded arg, { 0");
	const TCHAR* ExpectedResult = TEXT("This is some text containing an unbounded arg, { 0");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestOrderedNonExistent, "System::Core::String Formatting::Non Existent (Ordered)", "[Core][String][EditorContext][SmokeFilter]")
{
	TArray<FStringFormatArg> Args;
	Args.Add(TEXT("Replacement 1"));

	const TCHAR* Pattern        = TEXT("This is some text containing a non-existent arg { 5 }");
	const TCHAR* ExpectedResult = TEXT("This is some text containing a non-existent arg { 5 }");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

TEST_CASE_NAMED(FStringFormattingTestOrderedAllTypes, "System::Core::String Formatting::All Types (Ordered)", "[Core][String][EditorContext][SmokeFilter]")
{
	const int32 Value1 = 100;
	const uint32 Value2 = 200;
	const int64 Value3 = 300;
	const uint64 Value4 = 400;
	const float Value5 = 500.0;
	const double Value6 = 600.0;
	const FString Value7 = "Text";
	const TCHAR* Value8 = TEXT("Text");

	TArray<FStringFormatArg> Args;
	Args.Add(Value1);
	Args.Add(Value2);
	Args.Add(Value3);
	Args.Add(Value4);
	Args.Add(Value5);
	Args.Add(Value6);
	Args.Add(Value7);
	Args.Add(Value8);

	const TCHAR* Pattern        = TEXT("{0}, {1}, {2}, {3}, {4}, {5}, {6}, {7}");
	const TCHAR* ExpectedResult = TEXT("100, 200, 300, 400, 500.000000, 600.000000, Text, Text");

	const FString ActualResult = FString::Format(Pattern, Args);

	CHECK_MESSAGE(*FString::Printf(TEXT("FString::Format failed. Expected: \"%s\", Actual: \"%s\""), ExpectedResult, *ActualResult), FCString::Strcmp(ExpectedResult, *ActualResult) == 0);
}

#endif //WITH_TESTS