// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "String/Join.h"

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::String
{

TEST_CASE_NAMED(FStringJoinTest, "System::Core::String::Join", "[Core][String][SmokeFilter]")
{
	// Helper lambda for calling ToLower if the parameter type is FString.
	auto ConditionalToLower = []<typename T>(T&& Elem)
	{
		if constexpr (std::is_same_v<T, FString>)
		{
			return Elem.ToLower();
		}
		return Forward<T>(Elem);
	};

	SECTION("Join")
	{
		// TArrayView<T>&&
		CHECK(WriteToString<128>(Join(MakeArrayView<FStringView>({TEXTVIEW("ABC"), TEXTVIEW("DEF")}), TEXT(", "))) == TEXTVIEW("ABC, DEF"));
		CHECK(WriteToString<128>(Join(MakeArrayView<FString>({FString("ABC"), FString("DEF")}), TEXT(", "))) == TEXTVIEW("ABC, DEF"));

		// const T(&)[N]
		const FStringView ArrayStringView[] = {TEXTVIEW("ABC"), TEXTVIEW("DEF")};
		CHECK(WriteToString<128>(Join(ArrayStringView, TEXT(", "))) == TEXTVIEW("ABC, DEF"));
		const FString ArrayString[] = {FString("ABC"), FString("DEF")};
		CHECK(WriteToString<128>(Join(ArrayString, TEXT(", "))) == TEXTVIEW("ABC, DEF"));
	}

	SECTION("JoinBy")
	{
		// TArrayView<FString>&&
		CHECK(WriteToString<128>(JoinBy(MakeArrayView<FString>({FString("ABC"), FString("DEF")}), UE_PROJECTION_MEMBER(FString, ToLower), TEXT(", "))) == TEXTVIEW("abc, def"));

		// const FString(&)[N]
		const FString ArrayString[] = { FString("ABC"), FString("DEF") };
		CHECK(WriteToString<128>(JoinBy(ArrayString, UE_PROJECTION_MEMBER(FString, ToLower), TEXT(", "))) == TEXTVIEW("abc, def"));
	}

	SECTION("JoinQuoted")
	{
		// TArrayView<T>&&
		CHECK(WriteToString<128>(JoinQuoted(MakeArrayView<FStringView>({TEXTVIEW("ABC"), TEXTVIEW("DEF")}), TEXT(", "), TEXT("|"))) == TEXTVIEW("|ABC|, |DEF|"));
		CHECK(WriteToString<128>(JoinQuoted(MakeArrayView<FString>({FString("ABC"), FString("DEF")}), TEXT(", "), TEXT("|"))) == TEXTVIEW("|ABC|, |DEF|"));

		// const T(&)[N]
		const FStringView ArrayStringView[] = { TEXTVIEW("ABC"), TEXTVIEW("DEF") };
		CHECK(WriteToString<128>(JoinQuoted(ArrayStringView, TEXT(", "), TEXT("|"))) == TEXTVIEW("|ABC|, |DEF|"));
		const FString ArrayString[] = { FString("ABC"), FString("DEF") };
		CHECK(WriteToString<128>(JoinQuoted(ArrayString, TEXT(", "), TEXT("|"))) == TEXTVIEW("|ABC|, |DEF|"));
	}

	SECTION("JoinQuotedBy")
	{
		// TArrayView<FString>&&
		CHECK(WriteToString<128>(JoinQuotedBy(MakeArrayView<FString>({FString("ABC"), FString("DEF")}), UE_PROJECTION_MEMBER(FString, ToLower), TEXT(", "), TEXT("|"))) == TEXTVIEW("|abc|, |def|"));

		// const FString(&)[N]
		const FString ArrayString[] = { FString("ABC"), FString("DEF") };
		CHECK(WriteToString<128>(JoinQuotedBy(ArrayString, UE_PROJECTION_MEMBER(FString, ToLower), TEXT(", "), TEXT("|"))) == TEXTVIEW("|abc|, |def|"));
	}

	SECTION("JoinTuple")
	{
		// TTuple<Ts...>&&
		CHECK(WriteToString<128>(JoinTuple(MakeTuple(TEXTVIEW("ABC"), 123), TEXT(", "))) == TEXTVIEW("ABC, 123"));
		CHECK(WriteToString<128>(JoinTuple(MakeTuple(FString("ABC"), 123), TEXT(", "))) == TEXTVIEW("ABC, 123"));

		// const TTuple<Ts...>&
		const TTuple<FStringView, int32> TupleStringInt = MakeTuple(TEXTVIEW("ABC"), 123);
		CHECK(WriteToString<128>(JoinTuple(TupleStringInt, TEXT(", "))) == TEXTVIEW("ABC, 123"));
		const TTuple<FString, int32> TupleStringViewInt = MakeTuple(FString("ABC"), 123);
		CHECK(WriteToString<128>(JoinTuple(TupleStringViewInt, TEXT(", "))) == TEXTVIEW("ABC, 123"));
	}

	SECTION("JoinTupleBy")
	{
		// TTuple<FString, int32>&&
		CHECK(WriteToString<128>(JoinTupleBy(MakeTuple(FString("ABC"), 123), ConditionalToLower, TEXT(", "))) == TEXTVIEW("abc, 123"));

		// const TTuple<FString, int32>&
		const TTuple<FString, int32> TupleStringInt = MakeTuple(FString("ABC"), 123);
		CHECK(WriteToString<128>(JoinTupleBy(TupleStringInt, ConditionalToLower, TEXT(", "))) == TEXTVIEW("abc, 123"));
	}

	SECTION("JoinTupleQuoted")
	{
		// TTuple<Ts...>&&
		CHECK(WriteToString<128>(JoinTupleQuoted(MakeTuple(TEXTVIEW("ABC"), 123), TEXT(", "), TEXT("|"))) == TEXTVIEW("|ABC|, |123|"));
		CHECK(WriteToString<128>(JoinTupleQuoted(MakeTuple(FString("ABC"), 123), TEXT(", "), TEXT("|"))) == TEXTVIEW("|ABC|, |123|"));

		// const TTuple<Ts...>&
		const TTuple<FStringView, int32> TupleStringInt = MakeTuple(TEXTVIEW("ABC"), 123);
		CHECK(WriteToString<128>(JoinTupleQuoted(TupleStringInt, TEXT(", "), TEXT("|"))) == TEXTVIEW("|ABC|, |123|"));
		const TTuple<FString, int32> TupleStringViewInt = MakeTuple(FString("ABC"), 123);
		CHECK(WriteToString<128>(JoinTupleQuoted(TupleStringViewInt, TEXT(", "), TEXT("|"))) == TEXTVIEW("|ABC|, |123|"));
	}

	SECTION("JoinTupleQuotedBy")
	{
		// TTuple<FString, int32>&&
		CHECK(WriteToString<128>(JoinTupleQuotedBy(MakeTuple(FString("ABC"), 123), ConditionalToLower, TEXT(", "), TEXT("|"))) == TEXTVIEW("|abc|, |123|"));

		// const TTuple<FString, int32>&
		const TTuple<FString, int32> TupleStringInt = MakeTuple(FString("ABC"), 123);
		CHECK(WriteToString<128>(JoinTupleQuotedBy(TupleStringInt, ConditionalToLower, TEXT(", "), TEXT("|"))) == TEXTVIEW("|abc|, |123|"));
	}
}

} // UE::String

#endif //WITH_TESTS