// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/AnsiString.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::String
{

TEST_CASE_NAMED(FStringMatchesWildcardTest, "System::Core::String::MatchesWildcard", "[Core][String][SmokeFilter]")
{
	struct FTestCase
	{
		const ANSICHAR* Target;
		const ANSICHAR* Pattern;
		bool bResultCaseSensitive;
		bool bResultIgnoreCase;
	};

	static const FTestCase TestCases[] =
	{
		{ "", "", true, true }, // Trivial: empty matches empty
		{ "", "*", true, true }, // Trivial: * matches empty
		{ "a", "*a*", true, true }, // Trivial: * matches empty on both sides
		{ "abcd", "", false, false }, // Empty pattern doesn't match non-empty string
		{ "abcd", "abcd", true, true }, // Literal match
		{ "Abcd", "abcd", false, true }, // Literal match w/ case diff
		{ "abcd", "aBcd", false, true }, // Literal match w/ case diff
		{ "Abcd", "aBcd", false, true }, // Literal match w/ case diff
		{ "abcd", "ab?d", true, true }, // Trivial use of ?
		{ "abd", "ab?d", true, true }, // our ? CAN match nothing (for some reason)
		{ "abcde", "ab?d", false, false }, // ? should _not_ be able to match 2 chars, though
		{ "abd", "ab???d", true, true }, // multiple ?: with UE semantics, this should match 0, 1, 2, or 3 chars between ab and d
		{ "abcd", "ab???d", true, true }, // multiple ? (1 char between) - should match
		{ "abccd", "ab???d", true, true }, // multiple ? (2 chars between) - should match
		{ "abcccd", "ab???d", true, true }, // multiple ? (3 chars between) - should match
		{ "abccccd", "ab???d", false, false }, // multiple ? (4 chars between) - should not match
		{ "abcd", "*", true, true }, // Trivial use of *
		{ "abc", "a*", true, true }, // Trivial use of *
		{ "abbbbccccd", "a*d", true, true }, // one * in the middle, exact prefix/suffix (hits fast paths)
		{ "abcd", "*.*", false, false }, // DOS-ism: for us, "*.*" isn't the same as "*", demand an actual . if that is given
		{ "abc.def", "*.*", true, true }, // actual match for *.*
		{ "abc...def.g", "*.???.g", true, true }, // first . match after * will fail, need to retry to find real match
		{ "abc/d", "*/D", false, true }, // suffix match
		{ "axbxcxdxexxf", "a*b*c*d*e*f", true, true }, // many * wildcards but deterministic match
		{ "ababababbababaczzx", "a*b*a*b?c??x", true, true }, // match after many wildcards
		{ "ababababbababaczzy", "a*b*a*b?c??x", false, false }, // no match after many wildcards
		{ ":::::::::a a", "*:*:*:*:*:*:*:*:*:*:*", false, false }, // Exponential-time (each candidate fails to find the last missing ':', then backtracks). This particular test is not bad, but adding more : to the pattern and string shows the behavior.
		{ "a", "a*a", false, false }, // Regression test: MatchesWildcard used to get this wrong
		{ "a", "*a*a", false, false }, // Regression test: MatchesWildcard used to get this wrong
	};

	auto RunTests = [](auto EmptyStringWithCorrectType)
	{
		auto Target = EmptyStringWithCorrectType;
		auto Pattern = EmptyStringWithCorrectType;

		for (const FTestCase& Test : TestCases)
		{
			Target = Test.Target;
			Pattern = Test.Pattern;
			bool bResultCaseSensitive = Target.MatchesWildcard(Pattern, ESearchCase::CaseSensitive);
			bool bResultIgnoreCase = Target.MatchesWildcard(Pattern, ESearchCase::IgnoreCase);

			if (bResultCaseSensitive != Test.bResultCaseSensitive ||
				bResultIgnoreCase != Test.bResultIgnoreCase)
			{
				CAPTURE(Test.Target, Test.Pattern);
				CAPTURE(bResultCaseSensitive, Test.bResultCaseSensitive);
				CAPTURE(bResultIgnoreCase, Test.bResultIgnoreCase);
			}

			bool bResultsMatch = (bResultCaseSensitive == Test.bResultCaseSensitive) && (bResultIgnoreCase == Test.bResultIgnoreCase);
			CHECK(bResultsMatch);
		}
	};

	SECTION("MatchesWildcard with ANSICHAR")
	{
		RunTests(FAnsiString{});
	}

	SECTION("MatchesWildcard with WIDECHAR")
	{
		RunTests(FString{});
	}

	SECTION("MatchesWildcard with UTF8CHAR")
	{
		RunTests(FUtf8String{});
	}
}

} // UE::String

#endif //WITH_TESTS
