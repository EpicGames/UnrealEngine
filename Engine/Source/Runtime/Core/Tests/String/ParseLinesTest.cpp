// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "String/ParseLines.h"

#include "Algo/Compare.h"
#include "Containers/StringView.h"
#include "Misc/StringBuilder.h"
#include "String/Escape.h"
#include "String/Join.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

namespace UE::String
{

TEST_CASE("Core::String::ParseLines", "[Core][String][Smoke]")
{
	using FStringViewArray = TArray<FStringView, TInlineAllocator<8>>;

	constexpr EParseLinesOptions KeepEmpty = EParseLinesOptions::None;
	constexpr EParseLinesOptions SkipEmpty = EParseLinesOptions::SkipEmpty;
	constexpr EParseLinesOptions Trim = EParseLinesOptions::Trim;

	const auto [View, ExpectedLines, Options] = GENERATE_COPY(table<FStringView, FStringViewArray, EParseLinesOptions>(
	{
		{TEXTVIEW(""), {}, SkipEmpty},
		{TEXTVIEW(""), {TEXTVIEW("")}, KeepEmpty},
		{TEXTVIEW("\n"), {}, SkipEmpty},
		{TEXTVIEW("\n"), {TEXTVIEW("")}, KeepEmpty},
		{TEXTVIEW("\r"), {}, SkipEmpty},
		{TEXTVIEW("\r"), {TEXTVIEW("")}, KeepEmpty},
		{TEXTVIEW("\r\n"), {}, SkipEmpty},
		{TEXTVIEW("\r\n"), {TEXTVIEW("")}, KeepEmpty},
		{TEXTVIEW("\n\n"), {}, SkipEmpty},
		{TEXTVIEW("\n\n"), {TEXTVIEW(""), TEXTVIEW("")}, KeepEmpty},
		{TEXTVIEW("\r\r"), {}, SkipEmpty},
		{TEXTVIEW("\r\r"), {TEXTVIEW(""), TEXTVIEW("")}, KeepEmpty},
		{TEXTVIEW("\r\n\r\n"), {}, SkipEmpty},
		{TEXTVIEW("\r\n\r\n"), {TEXTVIEW(""), TEXTVIEW("")}, KeepEmpty},
		{TEXTVIEW("\r\nABC").Left(2), {}, SkipEmpty},
		{TEXTVIEW("\r\nABC").Left(2), {TEXTVIEW("")}, KeepEmpty},
		{TEXTVIEW("\r\nABC\r\nDEF").Left(5), {TEXTVIEW("ABC")}, SkipEmpty},
		{TEXTVIEW("\r\nABC\r\nDEF").Left(5), {TEXTVIEW(""), TEXTVIEW("ABC")}, KeepEmpty},
		{TEXTVIEW("ABC DEF"), {TEXTVIEW("ABC DEF")}, SkipEmpty},
		{TEXTVIEW("\nABC DEF\n"), {TEXTVIEW("ABC DEF")}, SkipEmpty},
		{TEXTVIEW("\nABC DEF\n"), {TEXTVIEW(""), TEXTVIEW("ABC DEF")}, KeepEmpty},
		{TEXTVIEW("\rABC DEF\r"), {TEXTVIEW("ABC DEF")}, SkipEmpty},
		{TEXTVIEW("\rABC DEF\r"), {TEXTVIEW(""), TEXTVIEW("ABC DEF")}, KeepEmpty},
		{TEXTVIEW("\r\nABC DEF\r\n"), {TEXTVIEW("ABC DEF")}, SkipEmpty},
		{TEXTVIEW("\r\nABC DEF\r\n"), {TEXTVIEW(""), TEXTVIEW("ABC DEF")}, KeepEmpty},
		{TEXTVIEW("\r\n\r\nABC DEF\r\n\r\n"), {TEXTVIEW("ABC DEF")}, SkipEmpty},
		{TEXTVIEW("\r\n\r\nABC DEF\r\n\r\n"), {TEXTVIEW(""), TEXTVIEW(""), TEXTVIEW("ABC DEF"), TEXTVIEW("")}, KeepEmpty},
		{TEXTVIEW("ABC\nDEF"), {TEXTVIEW("ABC"), TEXTVIEW("DEF")}, SkipEmpty},
		{TEXTVIEW("ABC\rDEF"), {TEXTVIEW("ABC"), TEXTVIEW("DEF")}, SkipEmpty},
		{TEXTVIEW("\r\nABC\r\nDEF\r\n"), {TEXTVIEW("ABC"), TEXTVIEW("DEF")}, SkipEmpty},
		{TEXTVIEW("\r\nABC\r\nDEF\r\n"), {TEXTVIEW(""), TEXTVIEW("ABC"), TEXTVIEW("DEF")}, KeepEmpty},
		{TEXTVIEW("\r\nABC\r\n\r\nDEF\r\n"), {TEXTVIEW("ABC"), TEXTVIEW("DEF")}, SkipEmpty},
		{TEXTVIEW("\r\nABC\r\n\r\nDEF\r\n"), {TEXTVIEW(""), TEXTVIEW("ABC"), TEXTVIEW(""), TEXTVIEW("DEF")}, KeepEmpty},
		{TEXTVIEW(" \t\r\n\t ABC \t\r\n\t \t\r\n\t DEF \t\r\n"), {TEXTVIEW(" \t"), TEXTVIEW("\t ABC \t"), TEXTVIEW("\t \t"), TEXTVIEW("\t DEF \t")}, SkipEmpty},
		{TEXTVIEW(" \t\r\n\t ABC \t\r\n\t \t\r\n\t DEF \t\r\n"), {TEXTVIEW("ABC"), TEXTVIEW("DEF")}, SkipEmpty | Trim},
		{TEXTVIEW(" \t\r\n\t ABC \t\r\n\t \t\r\n\t DEF \t\r\n"), {TEXTVIEW(" \t"), TEXTVIEW("\t ABC \t"), TEXTVIEW("\t \t"), TEXTVIEW("\t DEF \t")}, KeepEmpty},
		{TEXTVIEW(" \t\r\n\t ABC \t\r\n\t \t\r\n\t DEF \t\r\n"), {TEXTVIEW(""), TEXTVIEW("ABC"), TEXTVIEW(""), TEXTVIEW("DEF")}, KeepEmpty | Trim},
	}));

	FStringViewArray ActualLines;
	ParseLines(View, ActualLines, Options);
	const bool bEqual = Algo::Compare(ActualLines, ExpectedLines);
	if (!bEqual)
	{
		TStringBuilder<256> Input, Expected, Actual;
		Input << QuoteEscape(View);
		Expected << TEXTVIEW("[") << JoinBy(ExpectedLines, QuoteEscape, TEXTVIEW(", ")) << TEXTVIEW("]");
		Actual << TEXTVIEW("[") << JoinBy(ActualLines, QuoteEscape, TEXTVIEW(", ")) << TEXTVIEW("]");
		CAPTURE(Input, Expected, Actual);
		CHECK(bEqual);
	}
	else
	{
		CHECK(bEqual);
	}
}

} // UE::String

#endif