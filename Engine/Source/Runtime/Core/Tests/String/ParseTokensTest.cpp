// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "String/ParseTokens.h"

#include "Algo/Compare.h"
#include "Misc/StringBuilder.h"
#include "String/Escape.h"
#include "String/Join.h"
#include "TestHarness.h"

#include <catch2/generators/catch_generators.hpp>

namespace UE::String
{

TEST_CASE("Core::String::ParseTokens", "[Core][String][Smoke]")
{
	using FCharArray = TArray<TCHAR, TInlineAllocator<8>>;
	using FStringViewArray = TArray<FStringView, TInlineAllocator<8>>;

	constexpr EParseTokensOptions KeepEmpty = EParseTokensOptions::None;
	constexpr EParseTokensOptions SkipEmpty = EParseTokensOptions::SkipEmpty;
	constexpr EParseTokensOptions IgnoreCase = EParseTokensOptions::IgnoreCase;
	constexpr EParseTokensOptions Trim = EParseTokensOptions::Trim;

	SECTION("String")
	{
		const auto [View, DelimiterArray, ExpectedTokens, Options] = GENERATE_COPY(table<FStringView, FStringViewArray, FStringViewArray, EParseTokensOptions>(
		{
			{TEXTVIEW(""),                {},                   {},                                                     SkipEmpty},
			{TEXTVIEW(""),                {},                   {TEXTVIEW("")},                                         KeepEmpty},
			{TEXTVIEW("ABC"),             {},                   {TEXTVIEW("ABC")},                                      KeepEmpty},

			{TEXTVIEW(""),                {TEXTVIEW(",")},      {},                                                     SkipEmpty},
			{TEXTVIEW(""),                {TEXTVIEW(",")},      {TEXTVIEW("")},                                         KeepEmpty},
			{TEXTVIEW(","),               {TEXTVIEW(",")},      {},                                                     SkipEmpty},
			{TEXTVIEW(","),               {TEXTVIEW(",")},      {TEXTVIEW(""), TEXTVIEW("")},                           KeepEmpty},
			{TEXTVIEW(",,"),              {TEXTVIEW(",")},      {},                                                     SkipEmpty},
			{TEXTVIEW(",,"),              {TEXTVIEW(",")},      {TEXTVIEW(""), TEXTVIEW(""), TEXTVIEW("")},             KeepEmpty},
			{TEXTVIEW(", ,"),             {TEXTVIEW(",")},      {TEXTVIEW(" ")},                                        SkipEmpty},
			{TEXTVIEW(", ,"),             {TEXTVIEW(",")},      {},                                                     SkipEmpty | Trim},
			{TEXTVIEW(", ,"),             {TEXTVIEW(",")},      {TEXTVIEW(""), TEXTVIEW(" "), TEXTVIEW("")},            KeepEmpty},
			{TEXTVIEW(", ,"),             {TEXTVIEW(",")},      {TEXTVIEW(""), TEXTVIEW(""), TEXTVIEW("")},             KeepEmpty | Trim},
			{TEXTVIEW("ABC"),             {TEXTVIEW(",")},      {TEXTVIEW("ABC")},                                      KeepEmpty},
			{TEXTVIEW("A,,C"),            {TEXTVIEW(",")},      {TEXTVIEW("A"), TEXTVIEW("C")},                         SkipEmpty},
			{TEXTVIEW("A,,C"),            {TEXTVIEW(",")},      {TEXTVIEW("A"), TEXTVIEW(""), TEXTVIEW("C")},           KeepEmpty},
			{TEXTVIEW("A,\tB\t,C"),       {TEXTVIEW(",")},      {TEXTVIEW("A"), TEXTVIEW("\tB\t"), TEXTVIEW("C")},      KeepEmpty},
			{TEXTVIEW(",A, B ,C,"),       {TEXTVIEW(",")},      {TEXTVIEW("A"), TEXTVIEW(" B "), TEXTVIEW("C")},        SkipEmpty},
			{TEXTVIEW(",A, B ,C,"),       {TEXTVIEW(",")},      {TEXTVIEW(""), TEXTVIEW("A"), TEXTVIEW(" B "), TEXTVIEW("C"), TEXTVIEW("")}, KeepEmpty},
			{TEXTVIEW("A\u2022B\u2022C"), {TEXTVIEW("\u2022")}, {TEXTVIEW("A"), TEXTVIEW("B"), TEXTVIEW("C")},          KeepEmpty},

			{TEXTVIEW("ABCDABCD"), {TEXTVIEW("AB")},   {TEXTVIEW("CD"), TEXTVIEW("CD")},               SkipEmpty},
			{TEXTVIEW("ABCDABCD"), {TEXTVIEW("AB")},   {TEXTVIEW(""), TEXTVIEW("CD"), TEXTVIEW("CD")}, KeepEmpty},
			{TEXTVIEW("ABCDABCD"), {TEXTVIEW("ABCD")}, {},                                             SkipEmpty},
			{TEXTVIEW("ABCDABCD"), {TEXTVIEW("ABCD")}, {TEXTVIEW(""), TEXTVIEW(""), TEXTVIEW("")},     KeepEmpty},
			{TEXTVIEW("ABCDABCD"), {TEXTVIEW("DA")},   {TEXTVIEW("ABC"), TEXTVIEW("BCD")},             KeepEmpty},

			{TEXTVIEW("ABCDABCD"), {TEXTVIEW("B"),  TEXTVIEW("D")},  {TEXTVIEW("A"), TEXTVIEW("C"), TEXTVIEW("A"), TEXTVIEW("C")},               SkipEmpty},
			{TEXTVIEW("ABCDABCD"), {TEXTVIEW("B"),  TEXTVIEW("D")},  {TEXTVIEW("A"), TEXTVIEW("C"), TEXTVIEW("A"), TEXTVIEW("C"), TEXTVIEW("")}, KeepEmpty},
			{TEXTVIEW("ABCDABCD"), {TEXTVIEW("BC"), TEXTVIEW("DA")}, {TEXTVIEW("A"), TEXTVIEW("D")},                                             SkipEmpty},
			{TEXTVIEW("ABCDABCD"), {TEXTVIEW("BC"), TEXTVIEW("DA")}, {TEXTVIEW("A"), TEXTVIEW(""), TEXTVIEW(""), TEXTVIEW("D")},                 KeepEmpty},

			{TEXTVIEW("AbCdaBcDAbCd"), {TEXTVIEW("Bc"), TEXTVIEW("da")}, {TEXTVIEW("AbC"), TEXTVIEW("DAbCd")}, SkipEmpty},
			{TEXTVIEW("AbCdaBcDAbCd"), {TEXTVIEW("Bc"), TEXTVIEW("da")}, {TEXTVIEW("A"), TEXTVIEW("d")},       SkipEmpty | IgnoreCase},

			{TEXTVIEW("A\u2022\u2022B,,C"), {TEXTVIEW(",,"), TEXTVIEW("\u2022\u2022")},                     {TEXTVIEW("A"), TEXTVIEW("B"), TEXTVIEW("C")}, KeepEmpty},
			{TEXTVIEW("A\u2022\u2022B\u0085\u0085C"), {TEXTVIEW("\u0085\u0085"), TEXTVIEW("\u2022\u2022")}, {TEXTVIEW("A"), TEXTVIEW("B"), TEXTVIEW("C")}, KeepEmpty},
		}));

		FStringViewArray ActualTokens;
		if (GetNum(DelimiterArray) == 1)
		{
			ParseTokens(View, GetData(DelimiterArray)[0], ActualTokens, Options);
		}
		else
		{
			ParseTokensMultiple(View, DelimiterArray, ActualTokens, Options);
		}

		const bool bEqual = Algo::Compare(ActualTokens, ExpectedTokens);
		if (!bEqual)
		{
			TStringBuilder<256> Input, Delimiters, Expected, Actual;
			Input << QuoteEscape(View);
			Delimiters << TEXTVIEW("[") << JoinBy(DelimiterArray, QuoteEscape, TEXTVIEW(", ")) << TEXTVIEW("]");
			Expected << TEXTVIEW("[") << JoinBy(ExpectedTokens, QuoteEscape, TEXTVIEW(", ")) << TEXTVIEW("]");
			Actual << TEXTVIEW("[") << JoinBy(ActualTokens, QuoteEscape, TEXTVIEW(", ")) << TEXTVIEW("]");
			CAPTURE(Input, Delimiters, Expected, Actual);
			CHECK(bEqual);
		}
		else
		{
			CHECK(bEqual);
		}
	}

	SECTION("Char")
	{
		const auto [View, DelimiterArray, ExpectedTokens, Options] = GENERATE_COPY(table<FStringView, FCharArray, FStringViewArray, EParseTokensOptions>(
		{
			{TEXTVIEW(""),                {},               {},                                    SkipEmpty},
			{TEXTVIEW(""),                {},               {TEXT("")},                            KeepEmpty},
			{TEXTVIEW("ABC"),             {},               {TEXT("ABC")},                         KeepEmpty},

			{TEXTVIEW(""),                {TEXT(',')},      {},                                    SkipEmpty},
			{TEXTVIEW(""),                {TEXT(',')},      {TEXT("")},                            KeepEmpty},
			{TEXTVIEW(","),               {TEXT(',')},      {},                                    SkipEmpty},
			{TEXTVIEW(","),               {TEXT(',')},      {TEXT(""), TEXT("")},                  KeepEmpty},
			{TEXTVIEW(",,"),              {TEXT(',')},      {},                                    SkipEmpty},
			{TEXTVIEW(",,"),              {TEXT(',')},      {TEXT(""), TEXT(""), TEXT("")},        KeepEmpty},
			{TEXTVIEW(", ,"),             {TEXT(',')},      {TEXT(" ")},                           SkipEmpty},
			{TEXTVIEW(", ,"),             {TEXT(',')},      {},                                    SkipEmpty | Trim},
			{TEXTVIEW(", ,"),             {TEXT(',')},      {TEXT(""), TEXT(" "), TEXT("")},       KeepEmpty},
			{TEXTVIEW(", ,"),             {TEXT(',')},      {TEXT(""), TEXT(""), TEXT("")},        KeepEmpty | Trim},
			{TEXTVIEW("ABC"),             {TEXT(',')},      {TEXT("ABC")},                         KeepEmpty},
			{TEXTVIEW("A,,C"),            {TEXT(',')},      {TEXT("A"), TEXT("C")},                SkipEmpty},
			{TEXTVIEW("A,,C"),            {TEXT(',')},      {TEXT("A"), TEXT(""), TEXT("C")},      KeepEmpty},
			{TEXTVIEW("A,\tB\t,C"),       {TEXT(',')},      {TEXT("A"), TEXT("\tB\t"), TEXT("C")}, KeepEmpty},
			{TEXTVIEW(",A, B ,C,"),       {TEXT(',')},      {TEXT("A"), TEXT(" B "), TEXT("C")},                     SkipEmpty},
			{TEXTVIEW(",A, B ,C,"),       {TEXT(',')},      {TEXT(""), TEXT("A"), TEXT(" B "), TEXT("C"), TEXT("")}, KeepEmpty},
			{TEXTVIEW("A\u2022B\u2022C"), {TEXT('\u2022')}, {TEXT("A"), TEXT("B"), TEXT("C")},     KeepEmpty},

			{TEXTVIEW("ABCDABCD"),        {TEXT('B'), TEXT('D')},           {TEXT("A"), TEXT("C"), TEXT("A"), TEXT("C")},           SkipEmpty},
			{TEXTVIEW("ABCDABCD"),        {TEXT('B'), TEXT('D')},           {TEXT("A"), TEXT("C"), TEXT("A"), TEXT("C"), TEXT("")}, KeepEmpty},
			{TEXTVIEW("A\u2022B,C"),      {TEXT(','), TEXT('\u2022')},      {TEXT("A"), TEXT("B"), TEXT("C")},                      KeepEmpty},
			{TEXTVIEW("A\u2022B\u0085C"), {TEXT('\u0085'), TEXT('\u2022')}, {TEXT("A"), TEXT("B"), TEXT("C")},                      KeepEmpty},

			{TEXTVIEW("ABC"), {TEXT('b')}, {TEXT("ABC")},          SkipEmpty},
			{TEXTVIEW("ABC"), {TEXT('B')}, {TEXT("A"), TEXT("C")}, SkipEmpty},
			{TEXTVIEW("ABC"), {TEXT('b')}, {TEXT("A"), TEXT("C")}, SkipEmpty | IgnoreCase},
			{TEXTVIEW("abc"), {TEXT('B')}, {TEXT("abc")},          SkipEmpty},
			{TEXTVIEW("abc"), {TEXT('b')}, {TEXT("a"), TEXT("c")}, SkipEmpty},
			{TEXTVIEW("abc"), {TEXT('B')}, {TEXT("a"), TEXT("c")}, SkipEmpty | IgnoreCase},

			{TEXTVIEW("AbCdaBcD"), {TEXT('B'), TEXT('d')}, {TEXT("AbC"), TEXT("A"), TEXT("cD")},         SkipEmpty},
			{TEXTVIEW("AbCdaBcD"), {TEXT('B'), TEXT('d')}, {TEXT("A"), TEXT("C"), TEXT("a"), TEXT("c")}, SkipEmpty | IgnoreCase},

			{TEXTVIEW("A\u2022B\u2022C"), {TEXT('\u2022'), TEXT('b')}, {TEXT("A"), TEXT("B"), TEXT("C")}, SkipEmpty},
			{TEXTVIEW("A\u2022B\u2022C"), {TEXT('\u2022'), TEXT('b')}, {TEXT("A"), TEXT("C")},            SkipEmpty | IgnoreCase},
		}));

		FStringViewArray ActualTokens;
		if (GetNum(DelimiterArray) == 1)
		{
			ParseTokens(View, GetData(DelimiterArray)[0], ActualTokens, Options);
		}
		else
		{
			ParseTokensMultiple(View, DelimiterArray, ActualTokens, Options);
		}

		const bool bEqual = Algo::Compare(ActualTokens, ExpectedTokens);
		if (!bEqual)
		{
			TStringBuilder<256> Input, Delimiters, Expected, Actual;
			Input << QuoteEscape(View);
			Delimiters << TEXTVIEW("[") << JoinQuoted(DelimiterArray, TEXTVIEW(", "), TEXT('\'')) << TEXTVIEW("]");
			Expected << TEXTVIEW("[") << JoinBy(ExpectedTokens, QuoteEscape, TEXTVIEW(", ")) << TEXTVIEW("]");
			Actual << TEXTVIEW("[") << JoinBy(ActualTokens, QuoteEscape, TEXTVIEW(", ")) << TEXTVIEW("]");
			CAPTURE(Input, Delimiters, Expected, Actual);
			CHECK(bEqual);
		}
		else
		{
			CHECK(bEqual);
		}
	}
}

} // UE::String

#endif
