// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "String/RemoveFrom.h"

#include "Containers/StringView.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FStringRemoveFromTest, "System::Core::String::RemoveFrom", "[Core][String][SmokeFilter]")
{
	SECTION("RemoveFromStart")
	{
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("AbCABCAbCABC"), TEXTVIEW("Ab")) == TEXTVIEW("CABCAbCABC"));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("AbCABCAbCABC"), TEXTVIEW("Abc")) == TEXTVIEW("AbCABCAbCABC"));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("AbCABCAbCABC"), TEXTVIEW("Abc"), ESearchCase::IgnoreCase) == TEXTVIEW("ABCAbCABC"));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("AbCABCAbCABC"), TEXTVIEW("AbD"), ESearchCase::IgnoreCase) == TEXTVIEW("AbCABCAbCABC"));
		
		// Empty
		CHECK(UE::String::RemoveFromStart(TEXTVIEW(""), TEXTVIEW("A")) == TEXTVIEW(""));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW(""), TEXTVIEW("ABC")) == TEXTVIEW(""));

		// Exact
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("A"), TEXTVIEW("A")) == TEXTVIEW(""));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("A"), TEXTVIEW("A"), ESearchCase::IgnoreCase) == TEXTVIEW(""));

		// Duplicate
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("AA"), TEXTVIEW("A")) == TEXTVIEW("A"));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("AA"), TEXTVIEW("A"), ESearchCase::IgnoreCase) == TEXTVIEW("A"));

		// Different Case
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("A"), TEXTVIEW("a")) == TEXTVIEW("A"));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("A"), TEXTVIEW("a"), ESearchCase::IgnoreCase) == TEXTVIEW(""));

		// Substring
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("ABC"), TEXTVIEW("A")) == TEXTVIEW("BC"));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("ABC"), TEXTVIEW("a")) == TEXTVIEW("ABC"));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("ABC"), TEXTVIEW("a"), ESearchCase::IgnoreCase) == TEXTVIEW("BC"));

		CHECK(UE::String::RemoveFromStart(TEXTVIEW("ABC"), TEXTVIEW("B")) == TEXTVIEW("ABC"));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("ABC"), TEXTVIEW("C")) == TEXTVIEW("ABC"));
		CHECK(UE::String::RemoveFromStart(TEXTVIEW("ABC"), TEXTVIEW("BC")) == TEXTVIEW("ABC"));
	}

	SECTION("RemoveFromEnd")
	{
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("AbCABCAbCABC"), TEXTVIEW("ABC")) == TEXTVIEW("AbCABCAbC"));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("AbCABCAbCABC"), TEXTVIEW("abc")) == TEXTVIEW("AbCABCAbCABC"));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("AbCABCAbCABC"), TEXTVIEW("abc"), ESearchCase::IgnoreCase) == TEXTVIEW("AbCABCAbC"));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("AbCABCAbCABC"), TEXTVIEW("abD"), ESearchCase::IgnoreCase) == TEXTVIEW("AbCABCAbCABC"));

		// Empty
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW(""), TEXTVIEW("A")) == TEXTVIEW(""));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW(""), TEXTVIEW("ABC")) == TEXTVIEW(""));

		// Exact
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("A"), TEXTVIEW("A")) == TEXTVIEW(""));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("A"), TEXTVIEW("A"), ESearchCase::IgnoreCase) == TEXTVIEW(""));

		// Duplicate
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("AA"), TEXTVIEW("A")) == TEXTVIEW("A"));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("AA"), TEXTVIEW("A"), ESearchCase::IgnoreCase) == TEXTVIEW("A"));

		// Different Case
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("A"), TEXTVIEW("a")) == TEXTVIEW("A"));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("A"), TEXTVIEW("a"), ESearchCase::IgnoreCase) == TEXTVIEW(""));

		// Substring
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("ABC"), TEXTVIEW("C")) == TEXTVIEW("AB"));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("ABC"), TEXTVIEW("c")) == TEXTVIEW("ABC"));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("ABC"), TEXTVIEW("c"), ESearchCase::IgnoreCase) == TEXTVIEW("AB"));

		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("ABC"), TEXTVIEW("A")) == TEXTVIEW("ABC"));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("ABC"), TEXTVIEW("B")) == TEXTVIEW("ABC"));
		CHECK(UE::String::RemoveFromEnd(TEXTVIEW("ABC"), TEXTVIEW("AB")) == TEXTVIEW("ABC"));
	}
}

#endif //WITH_TESTS
