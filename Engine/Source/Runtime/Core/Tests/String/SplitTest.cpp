// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "String/Split.h"

#include "Containers/StringView.h"
#include "Tests/TestHarnessAdapter.h"

namespace UE::String
{

TEST_CASE_NAMED(FStringSplitTest, "System::Core::String::Split", "[Core][String][SmokeFilter]")
{
	FStringView Left;
	FStringView Right;

	SECTION("SplitFirst")
	{
		const auto TestSplitFirst = []<typename T>(const T* View, const T* Search, const T* ExpectedLeft, const T* ExpectedRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
		{
			TStringView<T> ActualLeft, ActualRight;
			//CAPTURE(View, Search, SearchCase);
			CHECK(SplitFirst(View, Search, ActualLeft, ActualRight, SearchCase));
			//CAPTURE(ActualLeft, ExpectedLeft, ActualRight, ExpectedRight);
			CHECK(ActualLeft.Equals(ExpectedLeft));
			CHECK(ActualRight.Equals(ExpectedRight));
		};

		TestSplitFirst(TEXT("AbCABCAbCABC"), TEXT("A"), TEXT(""), TEXT("bCABCAbCABC"));
		TestSplitFirst(TEXT("AbCABCAbCABC"), TEXT("a"), TEXT(""), TEXT("bCABCAbCABC"), ESearchCase::IgnoreCase);
		TestSplitFirst(TEXT("AbCABCAbCABC"), TEXT("b"), TEXT("A"), TEXT("CABCAbCABC"));
		TestSplitFirst(TEXT("AbCABCAbCABC"), TEXT("B"), TEXT("AbCA"), TEXT("CAbCABC"));
		TestSplitFirst(TEXT("AbCABCAbCABC"), TEXT("B"), TEXT("A"), TEXT("CABCAbCABC"), ESearchCase::IgnoreCase);
		TestSplitFirst(TEXT("AbCABCAbCABD"), TEXT("D"), TEXT("AbCABCAbCAB"), TEXT(""));
		TestSplitFirst(TEXT("AbCABCAbCABD"), TEXT("d"), TEXT("AbCABCAbCAB"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitFirst(TEXT("AbCABCAbCABC"), TEXT("a"), Left, Right));
		CHECK_FALSE(SplitFirst(TEXT("AbCABCAbCABC"), TEXT("D"), Left, Right, ESearchCase::IgnoreCase));

		TestSplitFirst(TEXT("AbCABCAbCABC"), TEXT("AbC"), TEXT(""), TEXT("ABCAbCABC"));
		TestSplitFirst(TEXT("AbCABCAbCABC"), TEXT("ABC"), TEXT("AbC"), TEXT("AbCABC"));
		TestSplitFirst(TEXT("AbCABCAbCABC"), TEXT("Bc"), TEXT("A"), TEXT("ABCAbCABC"), ESearchCase::IgnoreCase);
		TestSplitFirst(TEXT("AbCABCAbCABD"), TEXT("BD"), TEXT("AbCABCAbCA"), TEXT(""));
		TestSplitFirst(TEXT("AbCABCAbCABD"), TEXT("Bd"), TEXT("AbCABCAbCA"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitFirst(TEXT("AbCABCAbCABC"), TEXT("ab"), Left, Right));
		CHECK_FALSE(SplitFirst(TEXT("AbCABCAbCABC"), TEXT("CD"), Left, Right, ESearchCase::IgnoreCase));

		TestSplitFirst(TEXT("A"), TEXT("A"), TEXT(""), TEXT(""));
		TestSplitFirst(TEXT("A"), TEXT("a"), TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		TestSplitFirst(TEXT("ABC"), TEXT("ABC"), TEXT(""), TEXT(""));
		TestSplitFirst(TEXT("ABC"), TEXT("abc"), TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitFirst(TEXT(""), TEXT("A"), Left, Right));
		CHECK_FALSE(SplitFirst(TEXT("AB"), TEXT("ABC"), Left, Right));

		TestSplitFirst("AbCABCAbCABC", "ABC", "AbC", "AbCABC");

		CHECK_FALSE(SplitFirst(FStringView(nullptr, 0), TEXT("SearchTerm"), Left, Right)); //-V575
		CHECK_FALSE(SplitFirst(FStringView(), TEXT("SearchTerm"), Left, Right));
	}

	SECTION("SplitLast")
	{
		const auto TestSplitLast = []<typename T>(const T* View, const T* Search, const T* ExpectedLeft, const T* ExpectedRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
		{
			TStringView<T> ActualLeft, ActualRight;
			//CAPTURE(View, Search, SearchCase);
			CHECK(SplitLast(View, Search, ActualLeft, ActualRight, SearchCase));
			//CAPTURE(ActualLeft, ExpectedLeft, ActualRight, ExpectedRight);
			CHECK(ActualLeft.Equals(ExpectedLeft));
			CHECK(ActualRight.Equals(ExpectedRight));
		};

		TestSplitLast(TEXT("AbCABCAbCABC"), TEXT("b"), TEXT("AbCABCA"), TEXT("CABC"));
		TestSplitLast(TEXT("AbCABCAbCABC"), TEXT("B"), TEXT("AbCABCAbCA"), TEXT("C"));
		TestSplitLast(TEXT("AbCABCAbCABC"), TEXT("b"), TEXT("AbCABCAbCA"), TEXT("C"), ESearchCase::IgnoreCase);
		TestSplitLast(TEXT("AbCABCAbCABD"), TEXT("D"), TEXT("AbCABCAbCAB"), TEXT(""));
		TestSplitLast(TEXT("AbCABCAbCABD"), TEXT("d"), TEXT("AbCABCAbCAB"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitLast(TEXT("AbCABCAbCABC"), TEXT("a"), Left, Right));
		CHECK_FALSE(SplitLast(TEXT("AbCABCAbCABC"), TEXT("D"), Left, Right, ESearchCase::IgnoreCase));

		TestSplitLast(TEXT("AbCABCAbCABC"), TEXT("AbC"), TEXT("AbCABC"), TEXT("ABC"));
		TestSplitLast(TEXT("AbCABCAbCABC"), TEXT("ABC"), TEXT("AbCABCAbC"), TEXT(""));
		TestSplitLast(TEXT("AbCABCAbCABC"), TEXT("BC"), TEXT("AbCABCAbCA"), TEXT(""));
		TestSplitLast(TEXT("AbCABCAbCABC"), TEXT("Bc"), TEXT("AbCABCAbCA"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitLast(TEXT("AbCABCAbCABC"), TEXT("ab"), Left, Right));
		CHECK_FALSE(SplitLast(TEXT("AbCABCAbCABC"), TEXT("CD"), Left, Right, ESearchCase::IgnoreCase));

		TestSplitLast(TEXT("A"), TEXT("A"), TEXT(""), TEXT(""));
		TestSplitLast(TEXT("A"), TEXT("A"), TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		TestSplitLast(TEXT("ABC"), TEXT("ABC"), TEXT(""), TEXT(""));
		TestSplitLast(TEXT("ABC"), TEXT("abc"), TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitLast(TEXT(""), TEXT("A"), Left, Right));
		CHECK_FALSE(SplitLast(TEXT("AB"), TEXT("ABC"), Left, Right));

		TestSplitLast("AbCABCAbCABC", "ABC", "AbCABCAbC", "");

		CHECK_FALSE(SplitLast(FStringView(nullptr, 0), TEXT("SearchTerm"), Left, Right)); //-V575
		CHECK_FALSE(SplitLast(FStringView(), TEXT("SearchTerm"), Left, Right));
	}

	SECTION("SplitFirstOfAny")
	{
		const auto TestSplitFirstOfAny = []<typename T>(const T* View, TConstArrayView<TStringView<T>> Search, const T* ExpectedLeft, const T* ExpectedRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
		{
			TStringView<T> ActualLeft, ActualRight;
			//CAPTURE(View, Search, SearchCase);
			CHECK(SplitFirstOfAny(View, Search, ActualLeft, ActualRight, SearchCase));
			//CAPTURE(ActualLeft, ExpectedLeft, ActualRight, ExpectedRight);
			CHECK(ActualLeft.Equals(ExpectedLeft));
			CHECK(ActualRight.Equals(ExpectedRight));
		};

		TestSplitFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("A"), TEXT("B")}, TEXT(""), TEXT("bCABCAbcABC"));
		TestSplitFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("a"), TEXT("B")}, TEXT(""), TEXT("bCABCAbcABC"), ESearchCase::IgnoreCase);
		TestSplitFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("b")}, TEXT("A"), TEXT("CABCAbcABC"));
		TestSplitFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("c"), TEXT("B")}, TEXT("AbCA"), TEXT("CAbcABC"));
		TestSplitFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("c"), TEXT("B")}, TEXT("A"), TEXT("CABCAbcABC"), ESearchCase::IgnoreCase);
		TestSplitFirstOfAny(TEXT("AbCABCAbcABD"), {TEXT("E"), TEXT("D")}, TEXT("AbCABCAbcAB"), TEXT(""));
		TestSplitFirstOfAny(TEXT("AbCABCAbcABD"), {TEXT("E"), TEXT("d")}, TEXT("AbCABCAbcAB"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("D"), TEXT("a")}, Left, Right));
		CHECK_FALSE(SplitFirstOfAny(TEXT("AbCABCAbcABC"), {TEXT("E"), TEXT("D")}, Left, Right, ESearchCase::IgnoreCase));

		TestSplitFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("AbC")}, TEXT(""), TEXT("ABCAbCABC"));
		TestSplitFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("CABc"), TEXT("ABC")}, TEXT("AbC"), TEXT("AbCABC"));
		TestSplitFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("ABD"), TEXT("Bc")}, TEXT("A"), TEXT("ABCAbCABC"), ESearchCase::IgnoreCase);
		TestSplitFirstOfAny(TEXT("AbCABCAbCABD"), {TEXT("BD"), TEXT("CABB")}, TEXT("AbCABCAbCA"), TEXT(""));
		TestSplitFirstOfAny(TEXT("AbCABCAbCABD"), {TEXT("Bd"), TEXT("CABB")}, TEXT("AbCABCAbCA"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("bc"), TEXT("ab")}, Left, Right));
		CHECK_FALSE(SplitFirstOfAny(TEXT("AbCABCAbCABC"), {TEXT("DA"), TEXT("CD")}, Left, Right, ESearchCase::IgnoreCase));

		TestSplitFirstOfAny(TEXT("A"), {TEXT("A"), TEXT("B")}, TEXT(""), TEXT(""));
		TestSplitFirstOfAny(TEXT("A"), {TEXT("A"), TEXT("B")}, TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		TestSplitFirstOfAny(TEXT("ABC"), {TEXT("ABC"), TEXT("BC")}, TEXT(""), TEXT(""));
		TestSplitFirstOfAny(TEXT("ABC"), {TEXT("abc"), TEXT("bc")}, TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitFirstOfAny(TEXT(""), {TEXT("A"), TEXT("B")}, Left, Right));
		CHECK_FALSE(SplitFirstOfAny(TEXT("AB"), {TEXT("ABC"), TEXT("ABD")}, Left, Right));

		TestSplitFirstOfAny("AbCABCAbCABC", {"CABc", "ABC"}, "AbC", "AbCABC");

		CHECK_FALSE(SplitFirstOfAny(FStringView(nullptr, 0), {TEXT("ABC"), TEXT("ABD")}, Left, Right)); //-V575
		CHECK_FALSE(SplitFirstOfAny(FStringView(), {TEXT("ABC"), TEXT("ABD")}, Left, Right));
	}

	SECTION("SplitLastOfAny")
	{
		const auto TestSplitLastOfAny = []<typename T>(const T* View, TConstArrayView<TStringView<T>> Search, const T* ExpectedLeft, const T* ExpectedRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
		{
			TStringView<T> ActualLeft, ActualRight;
			//CAPTURE(View, Search, SearchCase);
			CHECK(SplitLastOfAny(View, Search, ActualLeft, ActualRight, SearchCase));
			//CAPTURE(ActualLeft, ExpectedLeft, ActualRight, ExpectedRight);
			CHECK(ActualLeft.Equals(ExpectedLeft));
			CHECK(ActualRight.Equals(ExpectedRight));
		};

		TestSplitLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("a"), TEXT("b")}, TEXT("AbCABCA"), TEXT("cABC"));
		TestSplitLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("a"), TEXT("b")}, TEXT("AbCABCAbcA"), TEXT("C"), ESearchCase::IgnoreCase);
		TestSplitLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("b")}, TEXT("AbCABCA"), TEXT("cABC"));
		TestSplitLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("c"), TEXT("B")}, TEXT("AbCABCAbcA"), TEXT("C"));
		TestSplitLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("c"), TEXT("B")}, TEXT("AbCABCAbcAB"), TEXT(""), ESearchCase::IgnoreCase);
		TestSplitLastOfAny(TEXT("AbCABCAbcABD"), {TEXT("E"), TEXT("D")}, TEXT("AbCABCAbcAB"), TEXT(""));
		TestSplitLastOfAny(TEXT("AbCABCAbcABD"), {TEXT("E"), TEXT("d")}, TEXT("AbCABCAbcAB"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("D"), TEXT("a")}, Left, Right));
		CHECK_FALSE(SplitLastOfAny(TEXT("AbCABCAbcABC"), {TEXT("E"), TEXT("D")}, Left, Right, ESearchCase::IgnoreCase));

		TestSplitLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("AbC")}, TEXT("AbCABC"), TEXT("ABC"));
		TestSplitLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("CABc"), TEXT("ABC")}, TEXT("AbCABCAbC"), TEXT(""));
		TestSplitLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("ABD"), TEXT("Bc")}, TEXT("AbCABCAbCA"), TEXT(""), ESearchCase::IgnoreCase);
		TestSplitLastOfAny(TEXT("AbCABCAbCABD"), {TEXT("BD"), TEXT("CABB")}, TEXT("AbCABCAbCA"), TEXT(""));
		TestSplitLastOfAny(TEXT("AbCABCAbCABD"), {TEXT("Bd"), TEXT("CABB")}, TEXT("AbCABCAbCA"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("bc"), TEXT("ab")}, Left, Right));
		CHECK_FALSE(SplitLastOfAny(TEXT("AbCABCAbCABC"), {TEXT("DA"), TEXT("CD")}, Left, Right, ESearchCase::IgnoreCase));

		TestSplitLastOfAny(TEXT("A"), {TEXT("A"), TEXT("B")}, TEXT(""), TEXT(""));
		TestSplitLastOfAny(TEXT("A"), {TEXT("A"), TEXT("B")}, TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		TestSplitLastOfAny(TEXT("ABC"), {TEXT("ABC"), TEXT("BC")}, TEXT("A"), TEXT(""));
		TestSplitLastOfAny(TEXT("ABC"), {TEXT("abc"), TEXT("bc")}, TEXT("A"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitLastOfAny(TEXT(""), {TEXT("A"), TEXT("B")}, Left, Right));
		CHECK_FALSE(SplitLastOfAny(TEXT("AB"), {TEXT("ABC"), TEXT("ABD")}, Left, Right));

		TestSplitLastOfAny("AbCABCAbCABC", {"CABc", "ABC"}, "AbCABCAbC", "");

		CHECK_FALSE(SplitLastOfAny(FStringView(nullptr, 0), {TEXT("ABC"), TEXT("ABD")}, Left, Right)); //-V575
		CHECK_FALSE(SplitLastOfAny(FStringView(), {TEXT("ABC"), TEXT("ABD")}, Left, Right));
	}

	SECTION("SplitFirstChar")
	{
		const auto TestSplitFirstChar = []<typename T>(const T* View, T Search, const T* ExpectedLeft, const T* ExpectedRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
		{
			TStringView<T> ActualLeft, ActualRight;
			//CAPTURE(View, Search, SearchCase);
			CHECK(SplitFirstChar(View, Search, ActualLeft, ActualRight, SearchCase));
			//CAPTURE(ActualLeft, ExpectedLeft, ActualRight, ExpectedRight);
			CHECK(ActualLeft.Equals(ExpectedLeft));
			CHECK(ActualRight.Equals(ExpectedRight));
		};

		TestSplitFirstChar(TEXT("AbCABCAbCABC"), TEXT('b'), TEXT("A"), TEXT("CABCAbCABC"));
		TestSplitFirstChar(TEXT("AbCABCAbCABC"), TEXT('B'), TEXT("AbCA"), TEXT("CAbCABC"));
		TestSplitFirstChar(TEXT("AbCABCAbCABC"), TEXT('B'), TEXT("A"), TEXT("CABCAbCABC"), ESearchCase::IgnoreCase);
		TestSplitFirstChar(TEXT("AbCABCAbCABD"), TEXT('D'), TEXT("AbCABCAbCAB"), TEXT(""));
		TestSplitFirstChar(TEXT("AbCABCAbCABD"), TEXT('d'), TEXT("AbCABCAbCAB"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitFirstChar(TEXT("AbCABCAbCABC"), TEXT('a'), Left, Right));
		CHECK_FALSE(SplitFirstChar(TEXT("AbCABCAbCABC"), TEXT('D'), Left, Right, ESearchCase::IgnoreCase));

		TestSplitFirstChar(TEXT("A"), TEXT('A'), TEXT(""), TEXT(""));
		TestSplitFirstChar(TEXT("A"), TEXT('a'), TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitFirstChar(TEXT(""), TEXT('A'), Left, Right));

		TestSplitFirstChar("AbCABCAbCABC", 'B', "AbCA", "CAbCABC");

		CHECK_FALSE(SplitFirstChar(FStringView(nullptr, 0), TEXT('A'), Left, Right)); //-V575
		CHECK_FALSE(SplitFirstChar(FStringView(), TEXT('A'), Left, Right));
	}

	SECTION("SplitLastChar")
	{
		const auto TestSplitLastChar = []<typename T>(const T* View, T Search, const T* ExpectedLeft, const T* ExpectedRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
		{
			TStringView<T> ActualLeft, ActualRight;
			//CAPTURE(View, Search, SearchCase);
			CHECK(SplitLastChar(View, Search, ActualLeft, ActualRight, SearchCase));
			//CAPTURE(ActualLeft, ExpectedLeft, ActualRight, ExpectedRight);
			CHECK(ActualLeft.Equals(ExpectedLeft));
			CHECK(ActualRight.Equals(ExpectedRight));
		};

		TestSplitLastChar(TEXT("AbCABCAbCABC"), TEXT('b'), TEXT("AbCABCA"), TEXT("CABC"));
		TestSplitLastChar(TEXT("AbCABCAbCABC"), TEXT('B'), TEXT("AbCABCAbCA"), TEXT("C"));
		TestSplitLastChar(TEXT("AbCABCAbCABC"), TEXT('b'), TEXT("AbCABCAbCA"), TEXT("C"), ESearchCase::IgnoreCase);
		TestSplitLastChar(TEXT("AbCABCAbCABD"), TEXT('D'), TEXT("AbCABCAbCAB"), TEXT(""));
		TestSplitLastChar(TEXT("AbCABCAbCABD"), TEXT('d'), TEXT("AbCABCAbCAB"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitLastChar(TEXT("AbCABCAbCABC"), TEXT('a'), Left, Right));
		CHECK_FALSE(SplitLastChar(TEXT("AbCABCAbCABC"), TEXT('D'), Left, Right, ESearchCase::IgnoreCase));

		TestSplitLastChar(TEXT("A"), TEXT('A'), TEXT(""), TEXT(""));
		TestSplitLastChar(TEXT("A"), TEXT('a'), TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitLastChar(TEXT(""), TEXT('A'), Left, Right));

		TestSplitLastChar("AbCABCAbCABC", 'B', "AbCABCAbCA", "C");

		CHECK_FALSE(SplitLastChar(FStringView(nullptr, 0), TEXT('A'), Left, Right)); //-V575
		CHECK_FALSE(SplitLastChar(FStringView(), TEXT('A'), Left, Right));
	}

	SECTION("SplitFirstOfAnyChar")
	{
		const auto TestSplitFirstOfAnyChar = []<typename T>(const T* View, TConstArrayView<T> Search, const T* ExpectedLeft, const T* ExpectedRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
		{
			TStringView<T> ActualLeft, ActualRight;
			//CAPTURE(View, Search, SearchCase);
			CHECK(SplitFirstOfAnyChar(View, Search, ActualLeft, ActualRight, SearchCase));
			//CAPTURE(ActualLeft, ExpectedLeft, ActualRight, ExpectedRight);
			CHECK(ActualLeft.Equals(ExpectedLeft));
			CHECK(ActualRight.Equals(ExpectedRight));
		};

		TestSplitFirstOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('b')}, TEXT("A"), TEXT("CABCAbcABC"));
		TestSplitFirstOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('c'), TEXT('B')}, TEXT("AbCA"), TEXT("CAbcABC"));
		TestSplitFirstOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('c'), TEXT('B')}, TEXT("A"), TEXT("CABCAbcABC"), ESearchCase::IgnoreCase);
		TestSplitFirstOfAnyChar(TEXT("AbCABCAbcABD"), {TEXT('E'), TEXT('D')}, TEXT("AbCABCAbcAB"), TEXT(""));
		TestSplitFirstOfAnyChar(TEXT("AbCABCAbcABD"), {TEXT('E'), TEXT('d')}, TEXT("AbCABCAbcAB"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitFirstOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('D'), TEXT('a')}, Left, Right));
		CHECK_FALSE(SplitFirstOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('E'), TEXT('D')}, Left, Right, ESearchCase::IgnoreCase));

		TestSplitFirstOfAnyChar(TEXT("A"), {TEXT('A'), TEXT('B')}, TEXT(""), TEXT(""));
		TestSplitFirstOfAnyChar(TEXT("A"), {TEXT('A'), TEXT('B')}, TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitFirstOfAnyChar(TEXT(""), {TEXT('A'), TEXT('B')}, Left, Right));

		TestSplitFirstOfAnyChar("AbCABCAbcABC", {'c', 'B'}, "AbCA", "CAbcABC");

		CHECK_FALSE(SplitFirstOfAnyChar(FStringView(nullptr, 0), {TEXT('A'), TEXT('B')}, Left, Right)); //-V575
		CHECK_FALSE(SplitFirstOfAnyChar(FStringView(), {TEXT('A'), TEXT('B')}, Left, Right));
	}

	SECTION("SplitLastOfAnyChar")
	{
		const auto TestSplitLastOfAnyChar = []<typename T>(const T* View, TConstArrayView<T> Search, const T* ExpectedLeft, const T* ExpectedRight, ESearchCase::Type SearchCase = ESearchCase::CaseSensitive)
		{
			TStringView<T> ActualLeft, ActualRight;
			//CAPTURE(View, Search, SearchCase);
			CHECK(SplitLastOfAnyChar(View, Search, ActualLeft, ActualRight, SearchCase));
			//CAPTURE(ActualLeft, ExpectedLeft, ActualRight, ExpectedRight);
			CHECK(ActualLeft.Equals(ExpectedLeft));
			CHECK(ActualRight.Equals(ExpectedRight));
		};

		TestSplitLastOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('b')}, TEXT("AbCABCA"), TEXT("cABC"));
		TestSplitLastOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('c'), TEXT('B')}, TEXT("AbCABCAbcA"), TEXT("C"));
		TestSplitLastOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('c'), TEXT('B')}, TEXT("AbCABCAbcAB"), TEXT(""), ESearchCase::IgnoreCase);
		TestSplitLastOfAnyChar(TEXT("AbCABCAbcABD"), {TEXT('E'), TEXT('D')}, TEXT("AbCABCAbcAB"), TEXT(""));
		TestSplitLastOfAnyChar(TEXT("AbCABCAbcABD"), {TEXT('E'), TEXT('d')}, TEXT("AbCABCAbcAB"), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitLastOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('D'), TEXT('a')}, Left, Right));
		CHECK_FALSE(SplitLastOfAnyChar(TEXT("AbCABCAbcABC"), {TEXT('E'), TEXT('D')}, Left, Right, ESearchCase::IgnoreCase));

		TestSplitLastOfAnyChar(TEXT("A"), {TEXT('A'), TEXT('B')}, TEXT(""), TEXT(""));
		TestSplitLastOfAnyChar(TEXT("A"), {TEXT('A'), TEXT('B')}, TEXT(""), TEXT(""), ESearchCase::IgnoreCase);
		CHECK_FALSE(SplitLastOfAnyChar(TEXT(""), {TEXT('A'), TEXT('B')}, Left, Right));

		TestSplitLastOfAnyChar("AbCABCAbcABC", {'c', 'B'}, "AbCABCAbcA", "C");

		CHECK_FALSE(SplitLastOfAnyChar(FStringView(nullptr, 0), {TEXT('A'), TEXT('B')}, Left, Right)); //-V575
		CHECK_FALSE(SplitLastOfAnyChar(FStringView(), {TEXT('A'), TEXT('B')}, Left, Right));
	}
}

} // UE::String

#endif // WITH_TESTS
