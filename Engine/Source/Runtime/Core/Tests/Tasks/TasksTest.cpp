// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "Tests/Benchmark.h"
#include "Tasks/Task.h"
#include "Tasks/Pipe.h"
#include "Tasks/TaskConcurrencyLimiter.h"
#include "HAL/Thread.h"
#include "Experimental/Misc/ExecutionResource.h"
#include "Async/ParallelFor.h"
#include "Async/ManualResetEvent.h"
#include "Tests/TestHarnessAdapter.h"
#include "Containers/UnrealString.h"

#include <atomic>

#if WITH_TESTS

namespace UE { namespace TasksTests
{
	using namespace Tasks;

	void DummyFunc()
	{
	}

	TEST_CASE_NAMED(FTasksBasicTest, "System::Core::Tasks::Basic", "[.][ApplicationContextMask][EngineFilter]")
	{
		if (!FPlatformProcess::SupportsMultithreading())
		{
			// the new API doesn't support single-threaded execution (`-nothreading`) until it's feature-compatible with the old API and completely replaces it
			return;
		}

		{	// basic example, fire and forget a high-pri task
			Launch(
				UE_SOURCE_LOCATION, // debug name
				[] {}, // task body
				Tasks::ETaskPriority::High /* task priority, `Normal` by default */
			);
		}

		{	// launch a task and wait till it's executed
			Launch(UE_SOURCE_LOCATION, [] {}).Wait();
		}

		{	// FTaskEvent asserts on destruction if it wasn't triggered. uncomment the code to verify
			//FTaskEvent Event{ UE_SOURCE_LOCATION };
		}

		{	// FTaskEvent blocks execution until it's signalled
			FTaskEvent Event{ UE_SOURCE_LOCATION };
			check(!Event.IsCompleted());

			// check that waiting blocks
			FTask Task = Launch(UE_SOURCE_LOCATION, [&Event] { Event.Wait(); });
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Event.Trigger();
			check(Event.IsCompleted());
			verify(Event.Wait(FTimespan::Zero()));
		}

		{	// FTaskEvent can be triggered multiple times
			FTaskEvent Event{ UE_SOURCE_LOCATION };
			Event.Trigger();
			check(Event.IsCompleted());
			Event.Trigger();
			Event.Trigger();
			check(Event.IsCompleted());
		}

		{	// same but using busy-waiting
			FTaskEvent Event{ UE_SOURCE_LOCATION };
			check(!Event.IsCompleted());

			// check that waiting blocks
			FTask Task = Launch(UE_SOURCE_LOCATION, [Event]() mutable { Event.Wait(); });
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Event.Trigger();
			check(Event.IsCompleted());
			verify(Event.Wait(FTimespan::Zero()));
		}

		{	// waiting for multiple tasks
			TArray<FTask> Tasks
			{
				Launch(UE_SOURCE_LOCATION,[] {}),
				Launch(UE_SOURCE_LOCATION,[] {})
			};
			Wait(Tasks);
		}

		{	// basic use-case, postpone waiting so the task is executed first
			std::atomic<bool> Done{ false };
			FTask Task = Launch(UE_SOURCE_LOCATION, [&Done] { Done = true; });
			while (!Task.IsCompleted())
			{
				FPlatformProcess::Yield();
			}
			Task.Wait();
			check(Done);
		}

		{	// basic use-case with result, postpone execution so waiting kicks in first
			TTask<int> Task = Launch(UE_SOURCE_LOCATION, [] { FPlatformProcess::Sleep(0.1f);  return 42; });
			verify(Task.GetResult() == 42);
		}

		{	// basic use-case with result, postpone waiting so the task is executed first
			TTask<int> Task = Launch(UE_SOURCE_LOCATION, [] { return 42; });
			while (!Task.IsCompleted())
			{
				FPlatformProcess::Yield();
			}
			verify(Task.GetResult() == 42);
		}

		{ // using an "empty" prerequisites
			FTask EmptyPrereq;
			FTask NonEmptyPrereq = Launch(UE_SOURCE_LOCATION, [] {});
			FTask Task = Launch(UE_SOURCE_LOCATION, [] {}, Prerequisites(EmptyPrereq, NonEmptyPrereq));
			check(Task.Wait(FTimespan::FromMilliseconds(100)));
		}

		{	// check that movable-only result types are supported, that only single instance of result is created and that it's destroyed
			static std::atomic<uint32> ConstructionsNum{ 0 }; // `static` to make it visible to `FMoveConstructable`
			static std::atomic<uint32> DestructionsNum{ 0 };

			struct FMoveConstructable
			{
				FORCENOINLINE FMoveConstructable()
				{
					ConstructionsNum.fetch_add(1, std::memory_order_relaxed);
				}

				FMoveConstructable(FMoveConstructable&&)
					: FMoveConstructable()
				{
				}

				FMoveConstructable(const FMoveConstructable&) = delete;
				FMoveConstructable& operator=(FMoveConstructable&&) = delete;
				FMoveConstructable& operator=(const FMoveConstructable&) = delete;

				FORCENOINLINE ~FMoveConstructable()
				{
					DestructionsNum.fetch_add(1, std::memory_order_relaxed);
				}
			};

			{
				Launch(UE_SOURCE_LOCATION, [] { return FMoveConstructable{}; }).GetResult();
			}

#if 0	// unreliable test, destruction can happen on a worker thread, after the task is flagged as completed and so the check can be hit before the destruction
			uint32 LocalConstructionsNum = ConstructionsNum.load(std::memory_order_relaxed);
			uint32 LocalDestructionsNum = DestructionsNum.load(std::memory_order_relaxed);
			checkf(LocalConstructionsNum == 1, TEXT("%d result instances were created but one was expected: the value stored in the task"), LocalConstructionsNum);
			checkf(LocalConstructionsNum == LocalDestructionsNum, TEXT("Mismatched number of constructions (%d) and destructions (%d)"), LocalConstructionsNum, LocalDestructionsNum);

			ConstructionsNum = 0;
			DestructionsNum = 0;
#endif

			{
				FMoveConstructable Res{ MoveTemp(Launch(UE_SOURCE_LOCATION, [] { return FMoveConstructable{}; }).GetResult()) }; // consume the result
			}

#if 0	// unreliable test, destruction can happen on a worker thread, after the task is flagged as completed and so the check can be hit before the destruction
			LocalConstructionsNum = ConstructionsNum.load(std::memory_order_relaxed);
			LocalDestructionsNum = DestructionsNum.load(std::memory_order_relaxed);
			checkf(LocalConstructionsNum == 2, TEXT("%d result instances were created but 2 was expected: the value stored in the task"), LocalConstructionsNum);
			checkf(LocalConstructionsNum == LocalDestructionsNum, TEXT("Mismatched number of constructions (%d) and destructions (%d)"), LocalConstructionsNum, LocalDestructionsNum);

			ConstructionsNum = 0;
			DestructionsNum = 0;
#endif
		}

		// fire and forget: launch a task w/o keeping its reference
		if (LowLevelTasks::FScheduler::Get().GetNumWorkers() != 0)
		{
			std::atomic<bool> bDone{ false };
			Launch(UE_SOURCE_LOCATION, [&bDone] { bDone = true; });
			while (!bDone)
			{
				FPlatformProcess::Yield();
			}
		}

		{	// mutable lambda, compilation check
			Launch(UE_SOURCE_LOCATION, []() mutable {}).Wait();
			Launch(UE_SOURCE_LOCATION, []() mutable { return false; }).GetResult();
		}

		{	// free memory occupied by task, can be required if task instance is held as a member var
			FTask Task = Launch(UE_SOURCE_LOCATION, [] {});
			Task.Wait();
			Task = {};
		}

		{	// accessing the task from inside its execution
			FTask Task;
			Task.Launch(UE_SOURCE_LOCATION, [&Task] { check(!Task.IsCompleted()); });
			Task.Wait();
		}

		///////////////////////////////////////////////////////////////////////////////
		// nested tasks
		///////////////////////////////////////////////////////////////////////////////

		{	// one nested task
			FTaskEvent FinishSignal{ UE_SOURCE_LOCATION };
			FTaskEvent ExecutedSignal{ UE_SOURCE_LOCATION };
			FTask Task{ Launch(UE_SOURCE_LOCATION,
				[&FinishSignal, &ExecutedSignal]
				{
					AddNested(FinishSignal);
					ExecutedSignal.Trigger();
				}
			) };

			verify(!Task.Wait(FTimespan::FromMilliseconds(100)));
			verify(ExecutedSignal.IsCompleted());
			FinishSignal.Trigger();
			Task.Wait();
		}

		{	// nested task completed before the parent finishes its execution
			Launch(UE_SOURCE_LOCATION,
				[]
				{
					FTask NestedTask = Launch(UE_SOURCE_LOCATION, [] {});
					AddNested(NestedTask);
					NestedTask.Wait();
				}
			).Wait();
		}

		{	// multiple nested tasks
			FTaskEvent Signal1{ UE_SOURCE_LOCATION };
			FTaskEvent Signal2{ UE_SOURCE_LOCATION };
			FTaskEvent Signal3{ UE_SOURCE_LOCATION };

			FTask Task{ Launch(UE_SOURCE_LOCATION,
				[&Signal1, &Signal2, &Signal3]
				{
					AddNested(Signal1);
					AddNested(Signal2);
					AddNested(Signal3);
				}
			) };

			verify(!Task.Wait(FTimespan::FromMilliseconds(100)));
			Signal1.Trigger();
			verify(!Task.Wait(FTimespan::FromMilliseconds(100)));
			Signal2.Trigger();
			verify(!Task.Wait(FTimespan::FromMilliseconds(100)));
			Signal3.Trigger();
			Task.Wait();
		}

		{	// nested in square
			FTaskEvent Signal{ UE_SOURCE_LOCATION };
			FTask Task{ Launch(UE_SOURCE_LOCATION,
				[&Signal]
				{
					AddNested(Signal);

					FTaskEvent NestedSignal{ UE_SOURCE_LOCATION };
					FTask NestedTask{ Launch(UE_SOURCE_LOCATION,
						[&NestedSignal]
						{
							AddNested(NestedSignal);
						}
					) };

					verify(!NestedTask.Wait(FTimespan::FromMilliseconds(100)));
					NestedSignal.Trigger();
					NestedTask.Wait();
				}
			)};

			verify(!Task.Wait(FTimespan::FromMilliseconds(200)));
			Signal.Trigger();
			Task.Wait();
		}

		{	// nested task should block pipe execution, also an example of pipe suspension

			struct FPipeSuspensionScope
			{
				explicit FPipeSuspensionScope(FPipe& Pipe)
				{
					Pipe.Launch(UE_SOURCE_LOCATION,
						[this]
						{
							AddNested(ResumeSignal);
							SuspendSignal.Trigger();
						}
					);

					SuspendSignal.Wait();
				}

				~FPipeSuspensionScope()
				{
					ResumeSignal.Trigger();
				}

				FTaskEvent SuspendSignal{ UE_SOURCE_LOCATION };
				FTaskEvent ResumeSignal{ UE_SOURCE_LOCATION };
			};

			FPipe Pipe{ UE_SOURCE_LOCATION };
			FTask Task;
			{
				FPipeSuspensionScope Suspension(Pipe);
				Task = Pipe.Launch(UE_SOURCE_LOCATION, [] {});
				verify(!Task.Wait(FTimespan::FromMilliseconds(100)));
			}
			Task.Wait();
			Pipe.WaitUntilEmpty();
		}

		{	// test basic functionality of TTask::IsAwaitable()
			FTask Task;
			Task.Launch(UE_SOURCE_LOCATION,
				[&Task] 
				{
					check(!Task.IsAwaitable()); // Task.Wait() would deadlock if called here inside its execution
				}
			);
			check(Task.IsAwaitable());
			Task.Wait();
		}


		{	// TTask::IsAwaitable() returns false when called from its inner task execution
			FTask Outer;
			Outer.Launch(UE_SOURCE_LOCATION,
				[&Outer]
				{
					// the inner task is executed "inline" (synchronously)
					FTask Inner = Launch
					(
						UE_SOURCE_LOCATION, 
						[&Outer] { check(!Outer.IsAwaitable()); /* still inside `Outer` execution */ }, 
						ETaskPriority::Default, 
						EExtendedTaskPriority::Inline
					);
					check(Inner.IsCompleted());
				}
			);
			Outer.Wait();
		}

		{ // check that `Prerequisites()` support containers
			Prerequisites(TArray<FTask>{});
			Prerequisites(TArrayView<FTask>{});
		}

		{	// a basic test for a named thread task
			FTask GTTask = Launch
			(
				UE_SOURCE_LOCATION, 
				[] { check(IsInGameThread()); }, 
				ETaskPriority::Default, 
				EExtendedTaskPriority::GameThreadNormalPri
			);
			GTTask.Wait();
		}

		{	// a basic test for multiple named thread tasks
			// This will exercise retraction on named thread
			TArray<FTask> GTTasks;
			for (int32 Index = 0; Index < 4; ++Index)
			{
				GTTasks.Add(Launch
				(
					UE_SOURCE_LOCATION,
					[] { check(IsInGameThread()); },
					ETaskPriority::Default,
					EExtendedTaskPriority::GameThreadNormalPri
				));
			}
			UE::Tasks::Wait(GTTasks);
		}

		//{	// Cancelled task is completed only after nested
		//	FTaskEvent ParentBlocker{ UE_SOURCE_LOCATION };
		//	FTaskEvent NestedBlocker{ UE_SOURCE_LOCATION };
		//	FTask Parent = Launch(UE_SOURCE_LOCATION,
		//		[&NestedBlocker]
		//		{
		//			FTask Nested{ Launch(UE_SOURCE_LOCATION, [] {}, NestedBlocker) };
		//			AddNested(Nested);
		//			Nested.Wait();
		//			// the only reference left is hold by Parent because of `AddNested` dependency
		//		},
		//		ParentBlocker
		//	);
		//	verify(Parent.TryCancel());
		//	ParentBlocker.Trigger(); // unblock `Parent` so it can notice that it's been cancelled
		//	check(Parent.Wait(FTimespan::FromMilliseconds(100))); // but it's still not complete because of the nested task is blocked
		//	NestedBlocker.Trigger();
		//	Parent.Wait();
		//}

	}

	template<uint32 SpawnerGroupsNum, uint32 SpawnersPerGroupNum, uint32 RepeatNum>
	void BasicStressTest()
	{
		TArray<FTask> SpawnerGroups;
		SpawnerGroups.Reserve(SpawnerGroupsNum);

		constexpr uint32 TasksNum = SpawnerGroupsNum * SpawnersPerGroupNum;
		TArray<FTask> Spawners;
		Spawners.AddDefaulted(TasksNum);
		TArray<FTask> Tasks;
		Tasks.AddDefaulted(TasksNum);

		for (uint32 RepeatIt = 0; RepeatIt != RepeatNum; ++RepeatIt)
		{
			std::atomic<uint32> TasksExecutedNum{ 0 };

			for (uint32 GroupIndex = 0; GroupIndex != SpawnerGroupsNum; ++GroupIndex)
			{
				SpawnerGroups.Add(Launch(UE_SOURCE_LOCATION,
					[
						Spawners = &Spawners[GroupIndex * SpawnersPerGroupNum],
						Tasks = &Tasks[GroupIndex * SpawnersPerGroupNum],
						&TasksExecutedNum
					]
					{
						for (uint32 SpawnerIndex = 0; SpawnerIndex != SpawnersPerGroupNum; ++SpawnerIndex)
						{
							Spawners[SpawnerIndex] = Launch(UE_SOURCE_LOCATION,
								[
									Task = &Tasks[SpawnerIndex],
									&TasksExecutedNum
								]
								{
									*Task = Launch(UE_SOURCE_LOCATION,
										[&TasksExecutedNum]
										{
											++TasksExecutedNum;
										}
									);
								}
							);
						}
					}
				));
			}

			Wait(SpawnerGroups);
			Wait(Spawners);
			Wait(Tasks);

			check(TasksExecutedNum == TasksNum);
		}
	}

	TEST_CASE_NAMED(FTasksBasicStressTest, "System::Core::Tasks::BasicStressTest", "[ApplicationContextMask][EngineFilter]")
	{
		UE_BENCHMARK(5, BasicStressTest<100, 100, 100>);
	}

	template<uint32 SpawnerGroupsNum, uint32 SpawnersPerGroupNum>
	void PipeStressTest();

	TEST_CASE_NAMED(FTasksPipeTest, "System::Core::Tasks::Pipe", "[.][ApplicationContextMask][EngineFilter]")
	{
		if (!FPlatformProcess::SupportsMultithreading())
		{
			// the new API doesn't support single-threaded execution (`-nothreading`) until it's feature-compatible with the new API and completely replaces it
			return;
		}

		{	// a basic usage example
			Tasks::FPipe Pipe{ UE_SOURCE_LOCATION }; // a debug name, user-provided or 
				// `UE_SOURCE_LOCATION` - source file name and line number
			// launch two tasks in the pipe, they will be executed sequentially, but in parallel
			// with other tasks (including TaskGraph's old API tasks)
			Tasks::FTask Task1 = Pipe.Launch(UE_SOURCE_LOCATION, [] {});
			Tasks::FTask Task2 = Pipe.Launch(UE_SOURCE_LOCATION, [] {});
			Task2.Wait(); // wait for `Task2` completion
			Pipe.WaitUntilEmpty();
		}

		{	// an example of thread-safe async interface, kind of a primitive "actor"
			class FAsyncClass
			{
			public:
				TTask<bool> DoSomething()
				{
					return Pipe.Launch(TEXT("DoSomething()"), [this] { return DoSomethingImpl(); });
				}

				FTask DoSomethingElse()
				{
					return Pipe.Launch(TEXT("DoSomethingElse()"), [this] { DoSomethingElseImpl(); });
				}

			private:
				bool DoSomethingImpl() { return false; }
				void DoSomethingElseImpl() {}

			private:
				FPipe Pipe{ UE_SOURCE_LOCATION };
			};

			// access the same instance from multiple threads
			FAsyncClass AsyncInstance;
			bool bRes = AsyncInstance.DoSomething().GetResult();
			AsyncInstance.DoSomethingElse().Wait();
		}

		{	// basic
			FPipe Pipe{ UE_SOURCE_LOCATION };
			Pipe.Launch(UE_SOURCE_LOCATION, [] {});
			Pipe.Launch(UE_SOURCE_LOCATION, [] {}).Wait();
			Pipe.WaitUntilEmpty();
		}

		{	// launching a piped task with pointer to a function
			FPipe Pipe{ UE_SOURCE_LOCATION };
			Pipe.Launch(UE_SOURCE_LOCATION, &DummyFunc).Wait();
			Pipe.WaitUntilEmpty();
		}

		{	// launching a piped task with a functor object
			struct FFunctor
			{
				void operator()()
				{
				}
			};

			FPipe Pipe{ UE_SOURCE_LOCATION };
			Pipe.Launch(UE_SOURCE_LOCATION, FFunctor{}).Wait();
			Pipe.WaitUntilEmpty();
		}

		{	// hold the first piped task execution until the next one is piped to test for non-concurrent execution
			FPipe Pipe{ UE_SOURCE_LOCATION };
			bool bTask1Done = false;
			FTask Task1 = Pipe.Launch(UE_SOURCE_LOCATION, 
				[&bTask1Done] 
				{ 
					FPlatformProcess::Sleep(0.1f); 
					bTask1Done = true; 
				}
			);
			// we can't just check if `Task1` is completed because pipe gets unblocked and so the next piped task can start execution before the
			// previous piped task's comlpetion flag is set
			Pipe.Launch(UE_SOURCE_LOCATION, [&bTask1Done] { check(bTask1Done); }).Wait();
			Pipe.WaitUntilEmpty();
		}

		{	// piping another task after the previous one is completed and destroyed
			FPipe Pipe{ UE_SOURCE_LOCATION };

			Pipe.Launch(UE_SOURCE_LOCATION, [] {}).Wait();
			Pipe.Launch(UE_SOURCE_LOCATION, [] {}).Wait();
			Pipe.WaitUntilEmpty();
		}

		{	// an example of blocking a pipe
			FPipe Pipe{ UE_SOURCE_LOCATION };
			std::atomic<bool> bBlocked;
			FTaskEvent Event{ UE_SOURCE_LOCATION };
			FTask Task = Pipe.Launch(UE_SOURCE_LOCATION,
				[&bBlocked, &Event]
				{
					bBlocked = true;
					Event.Wait(); 
				}
			);
			while (!bBlocked)
			{
			}
			// now it's blocked
			ensure(!Task.Wait(FTimespan::FromMilliseconds(100)));

			Event.Trigger(); // unblock
			Task.Wait();
			Pipe.WaitUntilEmpty();
		}

		{	// multiple "inline" tasks piped in one go, check that pipe maintains FIFO order in this case
			FPipe Pipe{ UE_SOURCE_LOCATION };
			FTaskEvent Block{ UE_SOURCE_LOCATION };
			bool bFirstDone = false;
			bool bSecondDone = false;
			FTask Task1 = Pipe.Launch(UE_SOURCE_LOCATION, [&] { check(!bSecondDone); bFirstDone = true; }, Prerequisites(Block), ETaskPriority::Normal, EExtendedTaskPriority::Inline);
			FTask Task2 = Pipe.Launch(UE_SOURCE_LOCATION, [&] { check(bFirstDone); bSecondDone = true; }, Prerequisites(Block), ETaskPriority::Normal, EExtendedTaskPriority::Inline);
			Block.Trigger();
			Wait(TArray{ Task1, Task2 });
			check(bFirstDone && bSecondDone);
			Pipe.WaitUntilEmpty();
		}

		UE_BENCHMARK(5, PipeStressTest<200, 100>);

	}

	// stress test for named thread tasks, checks spawning a large number of tasks from multiple threads and executing them
	template<uint32 SpawnerGroupsNum, uint32 SpawnersPerGroupNum>
	void PipeStressTest()
	{
		TArray<FTask> SpawnerGroups;
		SpawnerGroups.Reserve(SpawnerGroupsNum);

		constexpr uint32 TasksNum = SpawnerGroupsNum * SpawnersPerGroupNum;
		TArray<FTask> Spawners;
		Spawners.AddDefaulted(TasksNum);
		TArray<FTask> Tasks;
		Tasks.AddDefaulted(TasksNum);

		std::atomic<bool> bExecuting{ false };
		std::atomic<uint32> TasksExecutedNum{ 0 };

		FPipe Pipe{ UE_SOURCE_LOCATION };

		for (uint32 GroupIndex = 0; GroupIndex != SpawnerGroupsNum; ++GroupIndex)
		{
			SpawnerGroups.Add(Launch(UE_SOURCE_LOCATION,
				[
					Spawners = &Spawners[GroupIndex * SpawnersPerGroupNum],
					Tasks = &Tasks[GroupIndex * SpawnersPerGroupNum],
					&bExecuting,
					&TasksExecutedNum,
					&Pipe
				]
				{
					for (uint32 SpawnerIndex = 0; SpawnerIndex != SpawnersPerGroupNum; ++SpawnerIndex)
					{
						Spawners[SpawnerIndex] = Launch(UE_SOURCE_LOCATION,
							[
								Task = &Tasks[SpawnerIndex],
								&bExecuting,
								&TasksExecutedNum,
								&Pipe
							]
							{
								*Task = Pipe.Launch(UE_SOURCE_LOCATION,
									[&bExecuting, &TasksExecutedNum]
									{
										check(!bExecuting);
										bExecuting = true;
										++TasksExecutedNum;
										bExecuting = false;
									}
								);
							}
						);
					}
				}
				));
		}

		Wait(SpawnerGroups);
		Wait(Spawners);
		Wait(Tasks);

		check(TasksExecutedNum == TasksNum);
		Pipe.WaitUntilEmpty();
	}

	TEST_CASE_NAMED(FTasksPipeWaitUntilEmptyTest, "System::Core::Tasks::Pipe::WaitUntilEmpty", "[ApplicationContextMask][EngineFilte]")
	{
		//for (int i = 0; i != 100000; ++i)
		{	// waiting until an empty pipe is empty
			FPipe Pipe{ UE_SOURCE_LOCATION };
			Pipe.WaitUntilEmpty();
		}

		{	// waiting until a not empty pipe is empty with prereq
			FPipe Pipe{ UE_SOURCE_LOCATION };
			
			FTaskEvent Prereq{ UE_SOURCE_LOCATION };

			check(!Pipe.HasWork());

			FTask Task1{ Pipe.Launch(UE_SOURCE_LOCATION, [] {}, Prereq) };

			//Make sure the pipe knows about the task even if it has prereq
			check(Pipe.HasWork());

			check(!Task1.IsCompleted());

			check(!Pipe.WaitUntilEmpty(FTimespan::FromMilliseconds(100)));

			FTask Task2{ Pipe.Launch(UE_SOURCE_LOCATION, [] {}) };
			check(Task2.Wait(FTimespan::FromMilliseconds(100)));

			check(!Pipe.WaitUntilEmpty(FTimespan::FromMilliseconds(100)));

			check(!Task1.IsCompleted());
			check(Task2.IsCompleted());

			Prereq.Trigger();

			check(Pipe.WaitUntilEmpty(FTimespan::FromMilliseconds(100)));
			check(Task1.IsCompleted());
			check(Task2.IsCompleted());
		}

		{	// waiting until a not empty pipe is empty
			FPipe Pipe{ UE_SOURCE_LOCATION };
			Pipe.Launch(UE_SOURCE_LOCATION, [] {});
			Pipe.WaitUntilEmpty();
		}

		if (false) // disabled as it's an invalid case, but keep it compiling
		//for (int i = 0; i != 100000; ++i)
		{	// waiting until a not empty pipe is empty, while new tasks are piped. just an example, this would assert in `Pipe.WaitUntilEmtpy()`
			// as no more tasks can be piped after that

			FPipe Pipe{ UE_SOURCE_LOCATION };

			std::atomic<bool> bQuit{ false };

			auto TaskBody = [&Pipe, &bQuit](auto& TaskBodyRef) -> void
			{
				if (!bQuit)
				{
					Pipe.Launch(UE_SOURCE_LOCATION, [&TaskBodyRef] { TaskBodyRef(TaskBodyRef); });
				}
			};

			// start the chain
			TaskBody(TaskBody);

			check(!Pipe.WaitUntilEmpty(FTimespan::FromMilliseconds(100))); // it's not empty until we break the chain

			// break the chain
			bQuit = true;

			check(Pipe.WaitUntilEmpty());
		}
	}

	struct FAutoTlsSlot
	{
		uint32 Slot;

		FAutoTlsSlot()
			: Slot(FPlatformTLS::AllocTlsSlot())
		{
		}

		~FAutoTlsSlot()
		{
			FPlatformTLS::FreeTlsSlot(Slot);
		}
	};

	template<uint64 Num>
	void UeTlsStressTest()
	{
		static FAutoTlsSlot Slot;
		double Dummy = 0;
		for (uint64 i = 0; i != Num; ++i)
		{
			Dummy += (double)(uintptr_t)(FPlatformTLS::GetTlsValue(Slot.Slot));
			double Now = FPlatformTime::Seconds();
			FPlatformTLS::SetTlsValue(Slot.Slot, (void*)(uintptr_t)(Now));
		}
		FPlatformTLS::SetTlsValue(Slot.Slot, (void*)(uintptr_t)(Dummy));
	}

	template<uint64 Num>
	void ThreadLocalStressTest()
	{
		static thread_local double TlsValue;
		double Dummy = 0;
		for (uint64 i = 0; i != Num; ++i)
		{
			Dummy += TlsValue;
			double Now = FPlatformTime::Seconds();
			TlsValue = Now;
		}
		TlsValue = Dummy;
	}

	TEST_CASE_NAMED(FTlsTest, "System::Core::Tls", "[ApplicationContextMask][EngineFilter]")
	{
		UE_BENCHMARK(5, UeTlsStressTest<10000000>);
		UE_BENCHMARK(5, ThreadLocalStressTest<10000000>);
	}

	template<uint64 NumBranches, uint64 NumTasks>
	void DependenciesPerfTest()
	{

		TArray<TTask<FTaskEvent>> Branches;
		Branches.Reserve(NumBranches);
		for (uint64 BranchIndex = 0; BranchIndex != NumBranches; ++BranchIndex)
		{
			auto Branch = []
			{
				TArray<FTask> Tasks;
				Tasks.Reserve(NumTasks);
				for (uint64 TaskIndex = 0; TaskIndex != NumTasks; ++TaskIndex)
				{
					Tasks.Add(Launch(UE_SOURCE_LOCATION, [] { /*FPlatformProcess::YieldCycles(100000);*/ }));
				}
				FTaskEvent Joiner{ UE_SOURCE_LOCATION };
				Joiner.AddPrerequisites(Tasks);
				Joiner.Trigger();
				return Joiner;
			};

			Branches.Add(Launch(UE_SOURCE_LOCATION, MoveTemp(Branch)));
		}
		TArray<FTaskEvent> BranchTasks;
		BranchTasks.Reserve(NumBranches);
		for (TTask<FTaskEvent>& Task : Branches)
		{
			BranchTasks.Add(Task.GetResult());
		}
		Wait(BranchTasks);
	}

	TEST_CASE_NAMED(FTasksEventTest, "System::Core::Tasks::TriggerTaskEventConcurrently", "[ApplicationContextMask][EngineFilter]")
	{
		// Regression test to ensure we do not assert when triggering an event from more than one thread at a time.
		for (int i = 0; i < 100000; ++i)
		{
			FTaskEvent Event{ UE_SOURCE_LOCATION };
			FTask DeferredTask{ Launch(UE_SOURCE_LOCATION, [] {}, Event) };
			ParallelFor(64, [Event, DeferredTask] (int32) mutable
			{
				Event.Trigger();
				DeferredTask.Wait();
			});
		}
	}

	TEST_CASE_NAMED(FTasksDependenciesTest, "System::Core::Tasks::Dependencies", "[ApplicationContextMask][EngineFilter]")
	{
		{	// a task is not executed until its prerequisite (FTaskEvent) is completed
			FTaskEvent Prereq{ UE_SOURCE_LOCATION };

			FTask Task{ Launch(UE_SOURCE_LOCATION, [] {}, Prereq) };
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Prereq.Trigger();
			Task.Wait();
		}

		{	// a task is not executed until its prerequisite (FTaskEvent) is completed. with explicit task priority
			FTaskEvent Prereq{ UE_SOURCE_LOCATION };

			FTask Task{ Launch(UE_SOURCE_LOCATION, [] {}, Prereq, ETaskPriority::Normal) };
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Prereq.Trigger();
			Task.Wait();
		}

		{	// a task is not executed until its prerequisite (FTask) is completed
			FTaskEvent Event{ UE_SOURCE_LOCATION };
			FTask Prereq{ Launch(UE_SOURCE_LOCATION, [&Event] { Event.Wait(); }) };
			FTask Task{ Launch(UE_SOURCE_LOCATION, [] {}, Prereq) };
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Event.Trigger();
			Task.Wait();
		}

		{	// compilation test of an iterable collection as prerequisites
			TArray<FTask> Prereqs{ Launch(UE_SOURCE_LOCATION, [] {}), FTaskEvent{ UE_SOURCE_LOCATION } };

			FTask Task{ Launch(UE_SOURCE_LOCATION, [] {}, Prereqs) };
			((FTaskEvent&)Prereqs[1]).Trigger();
			Task.Wait();
		}

		{	// compilation test of an initializer list as prerequisites
			auto Prereqs = { Launch(UE_SOURCE_LOCATION, [] {}), Launch(UE_SOURCE_LOCATION, [] {}) };
			Launch(UE_SOURCE_LOCATION, [] {}, Prereqs).Wait();
			// the following line doesn't compile because the compiler can't deduce the initializer list type
			//Launch(UE_SOURCE_LOCATION, [] {}, { Launch(UE_SOURCE_LOCATION, [] {}), Launch(UE_SOURCE_LOCATION, [] {}) }).Wait();
		}

		{	// a task is not executed until all its prerequisites (FTask and FTaskEvent instances) are completed
			FTaskEvent Prereq1{ UE_SOURCE_LOCATION };
			FTaskEvent Event{ UE_SOURCE_LOCATION };
			FTask Prereq2{ Launch(UE_SOURCE_LOCATION, [&Event] { Event.Wait(); }) };

			TTask<void> Task{ Launch(UE_SOURCE_LOCATION, [] {}, Prerequisites(Prereq1, Prereq2)) };
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Prereq1.Trigger();
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Event.Trigger();
			Task.Wait();
		}

		{	// a task is not executed until all its prerequisites (FTask and FTaskEvent instances) are completed. with explicit task priority
			FTaskEvent Prereq1{ UE_SOURCE_LOCATION };
			FTaskEvent Event{ UE_SOURCE_LOCATION };
			FTask Prereq2{ Launch(UE_SOURCE_LOCATION, [&Event] { Event.Wait(); }) };
			TArray<FTask> Prereqs{ Prereq1, Prereq2 }; // to check if a random iterable container works as a prerequisite collection

			TTask<void> Task{ Launch(UE_SOURCE_LOCATION, [] {}, Prereqs, ETaskPriority::Normal) };
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Prereq1.Trigger();
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Event.Trigger();
			Task.Wait();
		}

		{	// a piped task blocked by a prerequisites doesn't block the pipe
			FPipe Pipe{ UE_SOURCE_LOCATION };
			FTaskEvent Prereq{ UE_SOURCE_LOCATION };

			FTask Task1{ Pipe.Launch(UE_SOURCE_LOCATION, [] {}, Prereq) };
			FPlatformProcess::Sleep(0.1f);
			check(!Task1.IsCompleted());

			FTask Task2{ Pipe.Launch(UE_SOURCE_LOCATION, [] {}) };
			Task2.Wait();

			Prereq.Trigger();
			Task1.Wait();

			Pipe.WaitUntilEmpty();
		}

		{	// a piped task with multiple prerequisites
			FPipe Pipe{ UE_SOURCE_LOCATION };
			FTaskEvent Prereq1{ UE_SOURCE_LOCATION };
			FTaskEvent Event{ UE_SOURCE_LOCATION };
			FTask Prereq2{ Launch(UE_SOURCE_LOCATION, [&Event] { Event.Wait(); }) };

			FTask Task{ Pipe.Launch(UE_SOURCE_LOCATION, [] {}, Prerequisites(Prereq1, Prereq2)) };
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Prereq1.Trigger();
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Event.Trigger();
			Task.Wait();

			Pipe.WaitUntilEmpty();
		}

		{	// a piped task with multiple prerequisites. with explicit task priority
			FPipe Pipe{ UE_SOURCE_LOCATION };
			FTaskEvent Prereq1{ UE_SOURCE_LOCATION };
			FTaskEvent Event{ UE_SOURCE_LOCATION };
			FTask Prereq2{ Launch(UE_SOURCE_LOCATION, [&Event] { Event.Wait(); }) };

			FTask Task{ Pipe.Launch(UE_SOURCE_LOCATION, [] {}, Prerequisites(Prereq1, Prereq2), ETaskPriority::Normal) };
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Prereq1.Trigger();
			FPlatformProcess::Sleep(0.1f);
			check(!Task.IsCompleted());

			Event.Trigger();
			Task.Wait();

			Pipe.WaitUntilEmpty();
		}

		UE_BENCHMARK(5, DependenciesPerfTest<150, 150>);
	}

	// blocks all workers until given event is triggered. Returns blocking tasks.
	TArray<FTask> BlockWorkers(FTaskEvent& ResumeEvent, uint32 NumWorkers = LowLevelTasks::FScheduler::Get().GetNumWorkers())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BlockWorkers);

		TArray<FTask> WorkerBlockers; // tasks that block worker threads
		WorkerBlockers.Reserve(NumWorkers);

		std::atomic<uint32>   NumWorkersBlocked{ 0 };
		UE::FManualResetEvent AllWorkersBlocked;

		for (int32 Index = 0; Index != NumWorkers; ++Index)
		{
			WorkerBlockers.Emplace(
				UE::Tasks::Launch(
					TEXT("WorkerBlocker"),
					[&NumWorkersBlocked, &NumWorkers, &ResumeEvent, &AllWorkersBlocked]
					{
						if (++NumWorkersBlocked == NumWorkers)
						{
							AllWorkersBlocked.Notify();
						}
						TRACE_CPUPROFILER_EVENT_SCOPE(BlockWorkers_Blocked);
						ResumeEvent.Wait();
					}
			));
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(WaitingUntilAllWorkersBlocked);
		check(AllWorkersBlocked.WaitFor(UE::FMonotonicTimeSpan::FromSeconds(1)));

		return WorkerBlockers;
	}

	// two levels of prerequisites and two levels of nested tasks
	void TwoLevelsDeepRetractionTest()
	{
		FTask P11{ Launch(TEXT("P11"), [] {}) };
		FTask P12{ Launch(TEXT("P12"), [] {}) };
		FTask P21{ Launch(TEXT("P21"), [] {}, Prerequisites(P11, P12)) };
		FTask P22{ Launch(TEXT("P22"), [] {}) };
		FTask N11, N12, N21, N22;
		FTask Task = Launch(UE_SOURCE_LOCATION,
			[&N11, &N12, &N21, &N22]
			{
				AddNested(N11 = Launch(TEXT("N11"),
					[&N21, &N22]
					{
						AddNested(N21 = Launch(TEXT("N21"), [] {}));
						AddNested(N22 = Launch(TEXT("N22"), [] {}));
					}
				));
				AddNested(N12 = Launch(TEXT("N12"), [] {}));
			},
			Prerequisites(P21, P22)
		);
		Task.Wait();
		check(P11.IsCompleted() && P12.IsCompleted() && P21.IsCompleted() && P22.IsCompleted() &&
			N11.IsCompleted() && N12.IsCompleted() && N21.IsCompleted() && N22.IsCompleted());
	}

	TEST_CASE_NAMED(FTasksDeepRetractionTest, "System::Core::Tasks::DeepRetraction", "[.][ApplicationContextMask][EngineFilter]")
	{
		FPlatformProcess::Sleep(0.1f); // give workers time to fall asleep

		FTaskEvent ResumeEvent{ UE_SOURCE_LOCATION };
		TArray<FTask> WorkerBlockers = BlockWorkers(ResumeEvent);

		{	// basic retraction, no dependencies
			bool bDone = false;
			Launch(UE_SOURCE_LOCATION, [&bDone] { bDone = true; }).Wait();
			check(bDone);
		}

		{	// basic deep retraction: single prerequisite
			bool bPrerequisiteDone = false;
			bool bTaskDone = false;
			FTask Prerequisite{ Launch(UE_SOURCE_LOCATION, 
				[&bPrerequisiteDone, &bTaskDone] 
				{
					check(!bPrerequisiteDone);
					check(!bTaskDone);
					bPrerequisiteDone = true;
				}
			)};
			Launch(UE_SOURCE_LOCATION, 
				[&bPrerequisiteDone, &bTaskDone] 
				{ 
					check(bPrerequisiteDone);
					check(!bTaskDone);
					bTaskDone = true; 
				},
				Prerequisite
			).Wait();
			check(bPrerequisiteDone);
			check(bTaskDone);
		}

		{	// basic deep retraction: single nested task
			bool bParentDone = false;
			bool bNestedDone = false;
			FTask NestedTask;
			Launch(UE_SOURCE_LOCATION,
				[&bParentDone, &bNestedDone, &NestedTask]
				{
					check(!bParentDone);
					check(!bNestedDone);
					NestedTask = Launch(UE_SOURCE_LOCATION,
						[&bParentDone, &bNestedDone]
						{
							check(bParentDone); // due to single-threaded execution (cos all workers are blocked), nested task can't be executed
							// until the parent is
							check(!bNestedDone);
							bNestedDone = true;
						}
					);
					AddNested(NestedTask);
					check(!bNestedDone);
					bParentDone = true;
				}
			).Wait();
			check(bParentDone);
			check(bNestedDone);
			check(NestedTask.IsCompleted());
		}

		TwoLevelsDeepRetractionTest();

		{	// retraction of a piped task
			FPipe Pipe{ UE_SOURCE_LOCATION };
			FTask PipedTask = Pipe.Launch(UE_SOURCE_LOCATION, [] {});
			PipedTask.Wait(); // the pipe is not blocked, so the task is retracted successfully
		}

		{	// deep retraction of a piped task: waiting for a piped task when the pipe has other incomplete tasks before it
			FPipe Pipe{ UE_SOURCE_LOCATION };
			FTask PipedTask1 = Pipe.Launch(UE_SOURCE_LOCATION, [] {});
			FTask PipedTask2 = Pipe.Launch(UE_SOURCE_LOCATION, [] {});
			PipedTask2.Wait(); // the pipe is not blocked, deep retraction first executes `PipedTask1`, and then `PipedTask2`
			Pipe.WaitUntilEmpty();
		}

		// an example of a deadlock caused by waiting for a piped task from inside execution of another task of the same pipe
		//{	// retraction of a piped task from inside that pipe
		//	FPipe Pipe{ UE_SOURCE_LOCATION };
		//	FTask PipedOuterTask = Pipe.Launch(UE_SOURCE_LOCATION, 
		//		[&Pipe] () mutable
		//		{
		//			FTask PipedInnerTask = Pipe.Launch(UE_SOURCE_LOCATION, [] {});
		//			PipedInnerTask.Wait(); // deadlock as the pipe is blocked (we are inside it)
		//		}
		//	);
		//	PipedOuterTask.Wait();
		//}

		ResumeEvent.Trigger();
		Wait(WorkerBlockers);
	}

	template<uint32 Num>
	void DeepRetractionStressTest()
	{
		for (uint32 i = 0; i != Num; ++i)
		{
			TwoLevelsDeepRetractionTest();
		}
	}

	TEST_CASE_NAMED(FTasksDeepRetractionStressTest, "System::Core::Tasks::DeepRetraction::Stress", "[ApplicationContextMask][EngineFilter]")
	{
		UE_BENCHMARK(5, DeepRetractionStressTest<1000>);
	}

	template<uint64 Num>
	void NestedTasksStressTest()
	{
		for (uint64 i = 0; i != Num; ++i)
		{
			FTask Nested;
			FTask Parent = Launch(TEXT("Parent"),
				[&Nested]
				{
					AddNested(Nested = Launch(TEXT("Nested"), [] {}));
				}
			);
			Parent.Wait();
			check(Nested.IsCompleted() && Parent.IsCompleted());
		}
	}

	TEST_CASE_NAMED(FTasksNestedTasksStressTest, "System::Core::Tasks::NestedTasks::Stress", "[ApplicationContextMask][EngineFilter]")
	{
		UE_BENCHMARK(5, NestedTasksStressTest<10000>);
	}

	template<int NumTasks>
	void TestPerfBasic()
	{
		TArray<FTask> Tasks;
		Tasks.Reserve(NumTasks);

		for (int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			Tasks.Emplace(Launch(UE_SOURCE_LOCATION, [] {}));
		}

		Wait(Tasks);
	}

	template<int32 NumTasks, int32 BatchSize>
	void TestPerfBatch()
	{
		static_assert(NumTasks % BatchSize == 0, "`NumTasks` must be divisible by `BatchSize`");
		constexpr int32 NumBatches = NumTasks / BatchSize;

		TArray<FTask> Batches;
		Batches.Reserve(NumBatches);
		TArray<FTask> Tasks;
		Tasks.AddDefaulted(NumTasks);

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			Batches.Add(Launch(UE_SOURCE_LOCATION,
				[&Tasks, BatchIndex]
				{
					for (int32 TaskIndex = 0; TaskIndex < BatchSize; ++TaskIndex)
					{
						Tasks[BatchIndex * BatchSize + TaskIndex] = Launch(UE_SOURCE_LOCATION, [] {});
					}
				}
			));
		}

		Wait(Batches);
		Wait(Tasks);
	}

	template<int32 NumTasks, int32 BatchSize>
	void TestPerfBatchOptimised()
	{
		static_assert(NumTasks % BatchSize == 0, "`NumTasks` must be divisible by `BatchSize`");
		constexpr int32 NumBatches = NumTasks / BatchSize;

		FTaskEvent SpawnSignal{ UE_SOURCE_LOCATION };
		TArray<FTask> AllDone;

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			AllDone.Add(Launch(UE_SOURCE_LOCATION,
				[]
				{
					FTaskEvent RunSignal{ UE_SOURCE_LOCATION };
					for (int32 TaskIndex = 0; TaskIndex < BatchSize; ++TaskIndex)
					{
						//AddNested(Launch(UE_SOURCE_LOCATION, [] {}, RunSignal));
					}
					RunSignal.Trigger();
				},
				SpawnSignal
			));
		}

		SpawnSignal.Trigger();
		Wait(AllDone);
	}

	template<int NumTasks>
	void TestLatency()
	{
		for (uint32 TaskIndex = 0; TaskIndex != NumTasks; ++TaskIndex)
		{
			Launch(UE_SOURCE_LOCATION, [] {}).Wait();
		}
	}

	template<uint32 NumTasks>
	void TestFGraphEventPerf()
	{
		FTaskEvent Prereq{ UE_SOURCE_LOCATION };
		std::atomic<uint32> CompletedTasks{ 0 };

		TArray<FTask> Tasks;
		for (int i = 0; i != NumTasks; ++i)
		{
			Tasks.Add(Launch(UE_SOURCE_LOCATION,
				[&Prereq, &CompletedTasks]()
				{
					Prereq.Wait();
					++CompletedTasks;
				}
			));
		}

		Prereq.Trigger();
		Wait(Tasks);

		check(CompletedTasks == NumTasks);
	}

	template<int NumTasks>
	void TestSpawning()
	{
		{
			TArray<FTask> Tasks;
			Tasks.Reserve(NumTasks);
			//double StartTime = FPlatformTime::Seconds();
			for (uint32 TaskNo = 0; TaskNo != NumTasks; ++TaskNo)
			{
				Tasks.Add(Launch(UE_SOURCE_LOCATION, [] {}));
			}

			//double Duration = FPlatformTime::Seconds() - StartTime;
			//UE_LOGF(LogTemp, Display, "Spawning %d empty trackable tasks took %f secs", NumTasks, Duration);

			Wait(Tasks);
		}
		{
			double StartTime = FPlatformTime::Seconds();
			for (uint32 TaskNo = 0; TaskNo != NumTasks; ++TaskNo)
			{
				Launch(UE_SOURCE_LOCATION, [] {});
			}

			//double Duration = FPlatformTime::Seconds() - StartTime;
			//UE_LOGF(LogTemp, Display, "Spawning %d empty non-trackable tasks took %f secs", NumTasks, Duration);
		}
	}

	template<int NumTasks>
	void TestBatchSpawning()
	{
		//double StartTime = FPlatformTime::Seconds();
		FTaskEvent Prereq{ UE_SOURCE_LOCATION };
		TArray<FTask> Tasks;
		Tasks.Reserve(NumTasks);
		for (uint32 TaskNo = 0; TaskNo != NumTasks; ++TaskNo)
		{
			Tasks.Add(Launch(UE_SOURCE_LOCATION, [] {}, Prereq));
		}

		//double SpawnedTime = FPlatformTime::Seconds();
		Prereq.Trigger();

		//double EndTime = FPlatformTime::Seconds();
		//UE_LOGF(LogTemp, Display, "Spawning %d empty non-trackable tasks took %f secs total, %f secs spawning and %f secs dispatching", NumTasks, EndTime - StartTime, SpawnedTime - StartTime, EndTime - SpawnedTime);

		Wait(Tasks);
	}

	template<int64 NumBatches, int64 NumTasksPerBatch>
	void TestWorkStealing()
	{
		TArray<FTask> Batches;
		Batches.Reserve(NumBatches);

		TArray<FTask> Tasks[NumBatches];
		for (int32 BatchIndex = 0; BatchIndex != NumBatches; ++BatchIndex)
		{
			Tasks[BatchIndex].Reserve(NumTasksPerBatch);
			Batches.Add(Launch(UE_SOURCE_LOCATION,
				[&Tasks, BatchIndex]()
				{
					for (int32 TaskIndex = 0; TaskIndex < NumTasksPerBatch; ++TaskIndex)
					{
						Tasks[BatchIndex].Add(Launch(UE_SOURCE_LOCATION, [] {}));
					}
				}
			));
		}

		Wait(Batches);
		for (int32 BatchIndex = 0; BatchIndex != NumBatches; ++BatchIndex)
		{
			Wait(Tasks[BatchIndex]);
		}
	}

	TEST_CASE_NAMED(FTasksPerfTest, "System::Core::Tasks::PerfTest", "[ApplicationContextMask][EngineFilter]")
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TaskGraphTests_PerfTest);

		UE_BENCHMARK(5, TestPerfBasic<10000>);
		UE_BENCHMARK(5, TestPerfBatch<10000, 100>);
		UE_BENCHMARK(5, TestPerfBatchOptimised<10000, 100>);
		UE_BENCHMARK(5, TestLatency<1000>);
		UE_BENCHMARK(5, TestWorkStealing<100, 100>);
		UE_BENCHMARK(5, TestSpawning<10000>);
		UE_BENCHMARK(5, TestBatchSpawning<10000>);
	}

	TEST_CASE_NAMED(FTasksPriorityCVarTest, "System::Core::Tasks::PriorityCVar", "[ApplicationContextMask][EngineFilter]")
	{
		// set every combination of task priority and extended task priority
		const TCHAR* CVarName = TEXT("TasksPriorityTestCVar");
		UE::Tasks::FTaskPriorityCVar CVar{ CVarName, TEXT("CVarHelp"), ETaskPriority::Normal, EExtendedTaskPriority::None };
		IConsoleVariable* CVarPtr = IConsoleManager::Get().FindConsoleVariable(CVarName);
		for (int Priority = 0; Priority != (int)ETaskPriority::Count; ++Priority)
		{
			for (int ExtendedPriority = 0; ExtendedPriority != (int)EExtendedTaskPriority::Count; ++ExtendedPriority)
			{
				TStringBuilder<1024> TaskPriorities;
				TaskPriorities.Append(ToString((ETaskPriority)Priority));
				TaskPriorities.Append(TEXT(" "));
				TaskPriorities.Append(ToString((EExtendedTaskPriority)ExtendedPriority));

				CVarPtr->Set(*TaskPriorities);

				check(CVar.GetTaskPriority() == (ETaskPriority)Priority);
				check(CVar.GetExtendedTaskPriority() == (EExtendedTaskPriority)ExtendedPriority);
				check(CVarPtr->GetString() == *TaskPriorities);
			}
		}

		// test setting only task priority
		CVarPtr->Set(TEXT("High"));
		check(CVar.GetTaskPriority() == ETaskPriority::High);
		check(CVar.GetExtendedTaskPriority() == EExtendedTaskPriority::None);
		CVarPtr->Set(TEXT("Normal"));
		check(CVar.GetTaskPriority() == ETaskPriority::Normal);

		// usage example
		CVarPtr->Set(TEXT("Normal"));
		Launch(UE_SOURCE_LOCATION, [] {}, CVar.GetTaskPriority(), CVar.GetExtendedTaskPriority()).Wait();

	}

	// test Pipe support for `-nothreading` config (`FPlatformProcess::SupportsMultithreading()` is false)
	TEST_CASE_NAMED(FTasksPipeNoThreadingTest, "System::Core::Tasks::PipeNoThreading", "[ApplicationContextMask][EngineFilter]")
	{
		if (FPlatformProcess::SupportsMultithreading())
		{
			return; // run with `-nothreading` for this test to make any sense
		}

		FPipe Pipe{ UE_SOURCE_LOCATION };
		Pipe.Launch(UE_SOURCE_LOCATION, [] {});
		FTask FinalTask = Pipe.Launch(UE_SOURCE_LOCATION, [] {});
		check(FinalTask.Wait(FTimespan::FromMilliseconds(100)));
		Pipe.WaitUntilEmpty();

	}

	TEST_CASE_NAMED(FTasksMakeCompletedTask, "System::Core::Tasks::MakeCompletedTask", "[.][ApplicationContextMask][EngineFilter]")
	{
		{	// basic
			TTask<int> Task = MakeCompletedTask<int>(42);
			check(Task.GetResult() == 42);
		}

		{	// move-only result
			TTask<TUniquePtr<int>> Task = MakeCompletedTask<TUniquePtr<int>>(MakeUnique<int>(42));
			check(*Task.GetResult() == 42);
		}

		{	// copy-only result
			struct FCopyOnly
			{
				FCopyOnly() = default;
				FCopyOnly(const FCopyOnly&) = default;
				FCopyOnly& operator=(const FCopyOnly&) = default;
			};

			TTask<FCopyOnly> Task = MakeCompletedTask<FCopyOnly>(FCopyOnly{});
		}

		{ // non-movable result
			struct FNonMovable
			{
				UE_NONCOPYABLE(FNonMovable);
				FNonMovable() = default;
			};

			TTask<FNonMovable> Task = MakeCompletedTask<FNonMovable>();
		}

		{	// result construction from multiple args
			struct FDummy
			{
				int Int;
				FString Str;

				FDummy(int InInt, const FString& InStr)
					: Int(InInt)
					, Str(InStr)
				{}
			};

			TTask<FDummy> Task = MakeCompletedTask<FDummy>(42, TEXT("Test"));
			check(Task.GetResult().Int == 42 && Task.GetResult().Str == TEXT("Test"));
		}
	}

	TEST_CASE_NAMED(FTasksCancellation, "System::Core::Tasks::Cancellation", "[.][ApplicationContextMask][EngineFilter]")
	{
		{
			FCancellationToken CancellationToken;
			FTaskEvent BlockExecution{ UE_SOURCE_LOCATION };

			// check that a task sees cancellation request
			FTask Task1 = Launch(UE_SOURCE_LOCATION,
				[&CancellationToken, BlockExecution]
				{
					BlockExecution.Wait();
					verify(CancellationToken.IsCanceled());
				}
			);
			// same token can be used with multiple tasks to cancel them all
			// a task can ignore cancellation request
			FTask Task2 = Launch(UE_SOURCE_LOCATION, [&CancellationToken] {});

			CancellationToken.Cancel();
			BlockExecution.Trigger();

			Wait(TArray{ Task1, Task2 });
		}
	}

	TEST_CASE_NAMED(FTasksWaitAny, "System::Core::Async::Tasks::WaitAny", "[.][ApplicationContextMask][EngineFilter]")
	{
		{	// blocks if none of tasks is completed
			FTaskEvent Blocker{ UE_SOURCE_LOCATION }; // blocks all tasks

			TArray<FTask> Tasks
			{
				Launch(UE_SOURCE_LOCATION, [] {}, Prerequisites(Blocker)),
				Launch(UE_SOURCE_LOCATION, [] {}, Prerequisites(Blocker))
			};

			verify(WaitAny(Tasks, FTimespan::FromMilliseconds(1.0)) == INDEX_NONE);

			Blocker.Trigger();

			verify(WaitAny(Tasks) != INDEX_NONE);
		}

		{	// doesn't wait for all tasks
			FTaskEvent Blocker{ UE_SOURCE_LOCATION };

			TArray<FTask> Tasks
			{
				Launch(UE_SOURCE_LOCATION, [] {}),
				Launch(UE_SOURCE_LOCATION, [] {}, Prerequisites(Blocker)) // is blocked
			};

			verify(WaitAny(Tasks) == 0);

			Blocker.Trigger();
		}
	}

	TEST_CASE_NAMED(FTasksAny, "System::Core::Async::Tasks::Any", "[.][ApplicationContextMask][EngineFilter]")
	{
		{	// blocks if none of tasks is completed
			FTaskEvent Blocker{ UE_SOURCE_LOCATION }; // blocks all tasks

			TArray<FTask> Tasks
			{
				Launch(UE_SOURCE_LOCATION, [] {}, Prerequisites(Blocker)),
				Launch(UE_SOURCE_LOCATION, [] {}, Prerequisites(Blocker))
			};

			verify(!Any(Tasks).Wait(FTimespan::FromMilliseconds(0.1)));

			Blocker.Trigger();

			Any(Tasks).Wait();
		}

		{	// doesn't wait for all tasks
			FTaskEvent Blocker{ UE_SOURCE_LOCATION };

			TArray<FTask> Tasks
			{
				Launch(UE_SOURCE_LOCATION, [] {}),
				Launch(UE_SOURCE_LOCATION, [] {}, Prerequisites(Blocker)) // is blocked
			};

			Any(Tasks).Wait();

			Blocker.Trigger();
		}
	}

	// produces work for FTaskConcurrencyLimiter from multiple threads to check it's thread-safety, checks that:
	// * it doesn't go over max concurrency
	// * can check (not 100% reliably but still useful for local runs) that max concurrency is actually reached
	// * slots don't overlap
	template<uint32 MaxConcurrency, uint32 NumItems, uint32 NumPushingTasks>
	void TaskConcurrencyLimiterStressTest()
	{
		static_assert(NumItems % NumPushingTasks == 0);

		std::atomic<uint32> CurrentConcurrency = 0;
		std::atomic<uint32> ActualMaxConcurrency = 0;
		std::atomic<uint32> NumProcessed = 0;

		std::atomic<bool> Slots[MaxConcurrency] = {};

		TArray<FTask> PushingTasks;
		PushingTasks.Reserve(NumPushingTasks);

		FTaskConcurrencyLimiter TaskConcurrencyLimiter(MaxConcurrency);

		for (uint32 i = 0; i != NumPushingTasks; ++i)
		{
			PushingTasks.Add(Launch(UE_SOURCE_LOCATION, 
				[&TaskConcurrencyLimiter, &CurrentConcurrency, &ActualMaxConcurrency, &Slots, &NumProcessed]
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(PushTasks);
					for (uint32 i = 0; i < NumItems / NumPushingTasks; ++i)
					{
						TaskConcurrencyLimiter.Push(UE_SOURCE_LOCATION,
							[&CurrentConcurrency, &ActualMaxConcurrency, &Slots, &NumProcessed](uint32 Slot)
							{
								TRACE_CPUPROFILER_EVENT_SCOPE(Task);
								check(Slot < MaxConcurrency);
								check(!Slots[Slot].load(std::memory_order_relaxed));
								Slots[Slot].store(true, std::memory_order_relaxed);

								uint32 CurrentConcurrencyLocal = CurrentConcurrency.fetch_add(1, std::memory_order_relaxed) + 1;
								check(CurrentConcurrencyLocal <= MaxConcurrency);
								uint32 ActualMaxConcurrencyLocal = ActualMaxConcurrency.load(std::memory_order_relaxed);
								while (ActualMaxConcurrencyLocal < CurrentConcurrencyLocal &&
									!ActualMaxConcurrency.compare_exchange_weak(ActualMaxConcurrencyLocal, CurrentConcurrencyLocal, std::memory_order_relaxed, std::memory_order_relaxed))
								{
									check(ActualMaxConcurrencyLocal <= MaxConcurrency);
								}

								FPlatformProcess::YieldCycles(10000);

								CurrentConcurrencyLocal = CurrentConcurrency.fetch_sub(1, std::memory_order_relaxed) - 1;
								check(CurrentConcurrencyLocal >= 0);

								Slots[Slot].store(false, std::memory_order_relaxed);

								NumProcessed.fetch_add(1, std::memory_order_release);
							}
						);
					}
				}
			));
		}

		Wait(PushingTasks);
		TaskConcurrencyLimiter.Wait();
		check(NumProcessed.load(std::memory_order_acquire) == NumItems);

		// unreliable check that MaxConcurrency is actually reached, but reliable enough for testing locally
		//check(ActualMaxConcurrency.load(std::memory_order_relaxed) == MaxConcurrency);
	}

	TEST_CASE_NAMED(FTasksConcurrencyLimiterTest, "System::Core::Async::Tasks::TaskConcurrencyLimiter", "[.][ApplicationContextMask][EngineFilter]")
	{
		UE_BENCHMARK(5, TaskConcurrencyLimiterStressTest<8, 1'000'000, 10>);
	}

	template<uint32 RepeatCount>
	void TaskConcurrencyLimiter_WaitingStressTest()
	{
		for (uint32 RepeatIndex = 0; RepeatIndex < RepeatCount; ++RepeatIndex)
		{
			FTaskConcurrencyLimiter TaskConcurrencyLimiter{ 1 };
			FEventRef Blocker;
			std::atomic<bool> bDone{ false };
			TaskConcurrencyLimiter.Push(UE_SOURCE_LOCATION, 
				[&Blocker, &bDone](uint32) 
				{ 
					TRACE_CPUPROFILER_EVENT_SCOPE(Blocked);
					Blocker->Wait(); 
					bDone.store(true, std::memory_order_relaxed); 
				}
			);
			const uint32 NumWaitingTasks = 10;
			TArray<FTask> WaitingTasks;
			WaitingTasks.Reserve(NumWaitingTasks);
			for (uint32 WaitIndex = 0; WaitIndex < NumWaitingTasks; ++WaitIndex)
			{
				WaitingTasks.Add(Launch(UE_SOURCE_LOCATION, 
					[&TaskConcurrencyLimiter, &bDone] 
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(Waiting);
						TaskConcurrencyLimiter.Wait(); 
						check(bDone.load(std::memory_order_relaxed)); 
					}
				));
			}
			Blocker->Trigger();
			Wait(WaitingTasks);
		}
	}

	TEST_CASE_NAMED(FTasksConcurrencyLimiterWaitingTest, "System::Core::Async::Tasks::TaskConcurrencyLimiter::Waiting", "[.][ApplicationContextMask][EngineFilter]")
	{
		UE_BENCHMARK(5, TaskConcurrencyLimiter_WaitingStressTest<10'000>);
	}

	template<uint32 RepeatCount>
	void TaskConcurrencyLimiter_WaitingRetractionStressTest()
	{
		for (uint32 RepeatIndex = 0; RepeatIndex < RepeatCount; ++RepeatIndex)
		{
			// Schedule more blocking work than we have task threads to run them on so we are forced to retract work in the limiter to proceed
			const uint32 NumWaitingTasks = LowLevelTasks::FScheduler::Get().GetMaxNumWorkers() * 2;
			TArray<FEventRef> Blockers;
			TArray<FTask> WaitingTasks;
			Blockers.SetNum(NumWaitingTasks);
			WaitingTasks.Reserve(NumWaitingTasks);

			std::atomic<int32> WaitCount = 0;
			for (uint32 WaitIndex = 0; WaitIndex < NumWaitingTasks; ++WaitIndex)
			{
				WaitingTasks.Add(Launch(UE_SOURCE_LOCATION,
					[&Blockers, WaitIndex, &WaitCount]
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(Waiting);
						WaitCount.fetch_add(1, std::memory_order_relaxed);
						Blockers[WaitIndex]->Wait();
					}));
			}

			// Ensure we have waited long enough that we have started blocking tasks on all worker threads
			while (WaitCount.load(std::memory_order_relaxed) < (int32)LowLevelTasks::FScheduler::Get().GetNumWorkers())
			{
				FPlatformProcess::Sleep(0.0f);
			}

			FTaskConcurrencyLimiter TaskConcurrencyLimiter{ 1 };
			for (uint32 WaitIndex = 0; WaitIndex < NumWaitingTasks; ++WaitIndex)
			{
				TaskConcurrencyLimiter.Push(UE_SOURCE_LOCATION,
					[&Blockers, WaitIndex](uint32)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(Blocked);
						Blockers[WaitIndex]->Trigger();
					}
				);
			}

			// Without retractions we will never run any of the queued task work
			TaskConcurrencyLimiter.Wait();
			Wait(WaitingTasks);
		}
	}

	TEST_CASE_NAMED(FTasksConcurrencyLimiterWaitingRetractionTest, "System::Core::Async::Tasks::TaskConcurrencyLimiter::WaitingRetraction", "[.][ApplicationContextMask][EngineFilter]")
	{
		UE_BENCHMARK(5, TaskConcurrencyLimiter_WaitingRetractionStressTest<10'000>);
	}

	void BenchmarkBlockUnblockWorkers(uint32 NumWorkers)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BenchmarkBlockUnblockWorkers);

		for (int Index = 0; Index < 1000; ++Index)
		{
			FTaskEvent ResumeEvent{ UE_SOURCE_LOCATION };
			TArray<FTask> WorkerBlockers = BlockWorkers(ResumeEvent, NumWorkers);

			ResumeEvent.Trigger();
			Wait(WorkerBlockers);
		}
	}

	TEST_CASE_NAMED(FTasksBenchmarkBlockUnblockWorkers, "System::Core::Async::Tasks::BenchmarkBlockUnblockWorkers", "[.][ApplicationContextMask][EngineFilter]")
	{
		FPlatformProcess::Sleep(0.1f); // give workers time to fall asleep, to avoid any reserve worker messing around

		uint32 NumWorkers = LowLevelTasks::FScheduler::Get().GetNumWorkers();

		for (uint32 Index = 1; Index <= NumWorkers; Index *= 2)
		{
			FString Name = FString::Printf(TEXT("BenchmarkBlockUnblockWorkers(%u)"), Index);

			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Name);
			Benchmark<5>(*Name, [&Index]() { BenchmarkBlockUnblockWorkers(Index); });
		}
	}

	TEST_CASE_NAMED(FTasksLoneStandbyWorker, "System::Core::Async::Tasks::LoneStandbyWorker", "[.][ApplicationContextMask][EngineFilter]")
	{
		// We absolutely need oversubscription to kick in to test this.
		// So only use a single worker to make sure that happens.
		LowLevelTasks::FScheduler::Get().RestartWorkers(1, 0);

		UE::FManualResetEvent OversubscribeeReadyEvent;
		UE::FManualResetEvent OversubscriberReadyEvent;
		UE::FManualResetEvent OversubscribeeDoneEvent;
		UE::FManualResetEvent OversubscriberDoneEvent;
		UE::FManualResetEvent LocalQueueEvent;

		UE::Tasks::FTask Oversubscriber =
			UE::Tasks::Launch(
				TEXT("Oversubscriber"),
				[&]()
				{
					LowLevelTasks::FOversubscriptionScope _;
					OversubscriberReadyEvent.Notify();
					OversubscriberDoneEvent.Wait();
				}
		);

		// Wait until the oversubscription scope is active
		OversubscriberReadyEvent.Wait();

		UE::Tasks::FTask Oversubscribee = UE::Tasks::Launch(
			TEXT("Oversubscribee"),
			[&]()
			{
				OversubscribeeReadyEvent.Notify();
				OversubscribeeDoneEvent.Wait();
			});

		// The first subsequent of a task is sent to the local queue
		// so setup ourself to be the subsequent of the oversubscribee.
		UE::Tasks::Launch(
			TEXT("LocalQueueTask"),
			[&]()
			{
				LocalQueueEvent.Notify();
			},
			UE::Tasks::Prerequisites(Oversubscribee)
		);

		// Wait until the oversubscribee task is launched.
		OversubscribeeReadyEvent.Wait();
		// Now close the oversubscription scope while the oversubcribee is still executing.
		OversubscriberDoneEvent.Notify();
		// Wait until the oversubscriber has closed it's oversubscription scope.
		Oversubscriber.Wait();
		// Now release the oversubscribee so it finishes executing and release its subsequent.
		// The first subsequent will be sent to the local queue of the standby thread.
		OversubscribeeDoneEvent.Notify();
		// Now we're left with only a standby thread executing. If that standby thread
		// enqueues the first subsequent to its local queue without waking up another worker
		// and goes immediately to sleep after its execution because the oversubscription
		// period is over, we could deadlock. The fix for this was to always enqueue
		// to the global queue for standby workers and to always perform a wakeup
		// so that normal workers can pick up the work if the standby worker can't.
		// Enqueuing to the global queue is not stricly required as just performing
		// a wake up would be sufficient, but the other thread would have to perform
		// stealing on all threads to find our task, which is less efficient.

		// Verify that we did not timeout (i.e. deadlock).
		verify(LocalQueueEvent.WaitFor(UE::FMonotonicTimeSpan::FromSeconds(5)));

		LowLevelTasks::FScheduler::Get().RestartWorkers();
	}

	// StressTest RestartWorkers while having task performing oversubscription
	TEST_CASE_NAMED(FTaskGraphOversubscribeDuringRestartWorkers, "System::Core::Async::Tasks::RestartWorkersAndOversubscription", "[.][ApplicationContextMask][EngineFilter]")
	{
		for (int32 Index = 0; Index < 10; ++Index)
		{
			FTask Task = Launch(TEXT("Task"),
				[&]()
				{
					// Just trigger oversubscription
					LowLevelTasks::FOversubscriptionScope _;
				});

			// For the repro to work we need to trigger oversubscription right between the time
			// we acquire the critical section and before the waiting queues are shut down.
			// This reproed very reliably before the fix.
			LowLevelTasks::FScheduler::Get().RestartWorkers(0, 4);

			Task.Wait();
		}

		// Restore original worker count
		LowLevelTasks::FScheduler::Get().RestartWorkers();
	}

	// StressTest RestartWorkers while having a thread trying to launch tasks on workers
	TEST_CASE_NAMED(FTaskGraphLaunchFromThreadDuringRestartWorkers, "System::Core::Async::Tasks::RestartWorkersAndExternalThreads", "[.][ApplicationContextMask][EngineFilter]")
	{
		std::atomic<bool> bExit = false;
		UE::FManualResetEvent Done;
		Async(EAsyncExecution::Thread,
			[&Done, &bExit]()
			{
				while (!bExit)
				{
					Launch(TEXT("Task"), [&]() {}).Wait();
				}

				Done.Notify();
			}
		);

		for (int32 Index = 0; Index < 100; ++Index)
		{
			// For the repro to work we need to launch tasks to try to start new workers while
			// we're shutting down and restarting workers.
			LowLevelTasks::FScheduler::Get().RestartWorkers(0, 4);
		}

		bExit = true;
		Done.Wait();

		// Restore original worker count
		LowLevelTasks::FScheduler::Get().RestartWorkers();
	}

	// StressTest RestartWorkers while a standby worker is actively executing tasks.
	// ConditionalStandby is called after each task execution in StandbyLoop.
	// The race: a standby worker can push itself onto the standby stack (via CAS)
	// after StartShutdown has drained the stack but before it does StandbyState = StackMask.
	// The worker blocks on Event->Wait() with nobody to wake it, and StopWorkers hangs on Join().
	TEST_CASE_NAMED(FConditionalStandbyShutdownRace, "System::Core::Async::Tasks::ConditionalStandbyShutdownRace", "[.][ApplicationContextMask][EngineFilter]")
	{
		for (int32 Index = 0; Index < 100; ++Index)
		{
			// Single foreground worker so oversubscription reliably creates a standby worker.
			LowLevelTasks::FScheduler::Get().RestartWorkers(1, 0);

			UE::FManualResetEvent BlockerReady;
			UE::FManualResetEvent BlockerRelease;

			// Block the only normal worker and hold oversubscription open.
			// This forces creation of a standby worker that must handle subsequent tasks.
			FTask Blocker = Launch(TEXT("Blocker"),
				[&]()
				{
					LowLevelTasks::FOversubscriptionScope _;
					BlockerReady.Notify();
					BlockerRelease.Wait();
					// Scope ends here: oversubscription decrements.
					// The standby worker's next ConditionalStandby will enter
					// the while loop (activeCount > ThreadCount + 0).
				});

			// Wait for oversubscription to be active and the standby worker to exist.
			BlockerReady.Wait();

			// Feed tasks for the standby worker. After each execution it calls
			// ConditionalStandby. Once oversubscription ends, that call will
			// try to push itself onto the standby stack.
			for (int32 i = 0; i < 8; ++i)
			{
				Launch(TEXT("StandbyWork"), []() {});
			}

			// Release the blocker (ends oversubscription) and immediately restart.
			// The standby worker may be mid-ConditionalStandby when StartShutdown runs.
			BlockerRelease.Notify();
			LowLevelTasks::FScheduler::Get().RestartWorkers(0, 4);

			Blocker.Wait();
		}

		LowLevelTasks::FScheduler::Get().RestartWorkers();
	}

	// This test will force the scheduling of a background task from the normal task which exercise some code in the scheduler
	// to send the task to the global queue and always wake up a worker since background tasks can't run on normal threads.
	TEST_CASE_NAMED(FTaskGraphBackgroundWithNormalAsPrereq, "System::Core::Async::Tasks::BackgroundWithNormalAsPrereq", "[.][ApplicationContextMask][EngineFilter]")
	{
		for (int32 Index = 0; Index < 100; ++Index)
		{
			UE::Tasks::FTask Task =
				UE::Tasks::Launch(
					TEXT("Task"),
					[] { },
					UE::Tasks::Launch(TEXT("Task"), [] { }), // Launch Normal task first and use it as prereq for the background one
					LowLevelTasks::ETaskPriority::BackgroundNormal
				);

			double StartTime = FPlatformTime::Seconds();
			while (!Task.IsCompleted())
			{
				if (FPlatformTime::Seconds() - StartTime > 10.0f)
				{
					FAIL_CHECK(TEXT("Test is likely deadlocked, aborting."));
					return;
				}
			}
		}
	}

	TEST_CASE_NAMED(FTaskGraphLongEventChain, "System::Core::Async::Tasks::LongEventChain", "[ApplicationContextMask][EngineFilter]")
	{
		UE::Tasks::FTaskEvent RootEvent(TEXT("RootEvent"));

		UE::Tasks::FPipe Pipe(TEXT("Pipe"));

		std::atomic<int32> CurrentIndex = 0;

		// Launch a long event chain with RootEvent not triggered yet.
		// Each event will unlock a task to put in the pipe, and then we can observe the unlock order.
		UE::Tasks::FTaskEvent LastEvent = RootEvent;
		for (int32 Index = 0; Index < 100000; ++Index)
		{
			UE::Tasks::FTaskEvent Event(TEXT("Event"));
			Event.AddPrerequisites(LastEvent);

			Pipe.Launch(
				TEXT("Task"),
				[Index, &CurrentIndex]
				{
					CHECK(Index == CurrentIndex++);
				},
				Event
			);

			// RootEvent is not triggered yet, so this does launch the task yet
			Event.Trigger();
			LastEvent = Event;
		}

		// Without the overflow handling in Close(), this would probably cause a stack overflow
		RootEvent.Trigger();

		Pipe.WaitUntilEmpty();
	}

#if PLATFORM_SUPPORTS_ASYMMETRIC_FENCES
	TEST_CASE_NAMED(FAsymmetricThreadFence, "System::Core::Async::AsymmetricThreadFence", "[.][ApplicationContextMask][EngineFilter]")
	{
		for (int32 Index = 0; Index < 10000; ++Index)
		{
			std::atomic<int> A = 0;
			std::atomic<int> B = 0;
			std::atomic<int> R0 = 0;
			std::atomic<int> R1 = 0;

			// If you use AsymmetricThreadFenceLight in both threads, you'll see you can end up with a race
			// because nothing publishes the A value before reading the B value. Using an heavy fence
			// on one side ensures that the test can't fail even when one side is only using relaxed operations
			// along with a simple compiler barrier.
			UE::Tasks::FTask T1 =
				UE::Tasks::Launch(
					TEXT("T1"),
					[&A, &B, &R0]()
					{
						A.store(1, std::memory_order_relaxed);
						FPlatformMisc::AsymmetricThreadFenceHeavy();
						R0.store(B.load(std::memory_order_relaxed), std::memory_order_relaxed);
					}
				);

			B.store(1, std::memory_order_relaxed);
			FPlatformMisc::AsymmetricThreadFenceLight();
			R1.store(A.load(std::memory_order_relaxed), std::memory_order_relaxed);

			T1.Wait();

			// Both value should never be 0 but it can happen when no synchronization is used due to the presence of a store buffer.
			verify(R1 == 1 || R0 == 1);
		}
	}
#endif

}}

#endif // WITH_TESTS
