// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"

#include "Tests/TestHarnessAdapter.h"

namespace
{
	template <typename T>
	struct TInstanceTracker
	{
		static TSet<T*> Constructed;
		static TSet<T*> Destructed;

		TInstanceTracker()
		{
			Constructed.Add(static_cast<T*>(this));
		}

		TInstanceTracker(const TInstanceTracker&)
		{
			Constructed.Add(static_cast<T*>(this));
		}

		TInstanceTracker& operator=(const TInstanceTracker&) = default;

		~TInstanceTracker()
		{
			Destructed.Add(static_cast<T*>(this));
		}

		static void ResetInstances()
		{
			Constructed.Reset();
			Destructed.Reset();
		}

		static bool CheckCurrentlyInstantiated(std::initializer_list<const T*> CurrentInstances)
		{
			TSet<T*> LocalConstructed = Constructed;
			TSet<T*> LocalDestructed = Destructed;

			for (const T* Instance : CurrentInstances)
			{
				if (!LocalConstructed.Contains(Instance))
				{
					return false;
				}

				LocalConstructed.Remove(Instance);
			}

			// The remaining objects should have been destructed

			LocalConstructed.Sort([](T& Lhs, T& Rhs){ return &Lhs < &Rhs; });
			LocalDestructed.Sort([](T& Lhs, T& Rhs){ return &Lhs < &Rhs; });

			if (LocalConstructed.Num() != LocalDestructed.Num())
			{
				return false;
			}

			auto ConstructedIt = LocalConstructed.CreateConstIterator();
			auto DestructedIt = LocalConstructed.CreateConstIterator();
			for (; ConstructedIt; ++ConstructedIt, ++DestructedIt)
			{
				if (*ConstructedIt != *DestructedIt)
				{
					return false;
				}
			}

			return true;
		}
	};

	template <typename T>
	TSet<T*> TInstanceTracker<T>::Constructed;

	template <typename T>
	TSet<T*> TInstanceTracker<T>::Destructed;
}

TEST_CASE_NAMED(FRelocateConstructItemsTest, "UE::Core::RelocateConstructItems", "[Core][Smoke]")
{
	SECTION("Relocate overlapping memory earlier in memory")
	{
		alignas(FString) char Buffer[sizeof(FString) * 5];

		FString* TypedBuffer = (FString*)Buffer;

		new ((void*)(TypedBuffer + 2)) FString("String A1");
		new ((void*)(TypedBuffer + 3)) FString("String B1");
		new ((void*)(TypedBuffer + 4)) FString("String C1");

		RelocateConstructItems<FString>((void*)Buffer, TypedBuffer + 2, 3);

		CHECK(TypedBuffer[0] == TEXT("String A1"));
		CHECK(TypedBuffer[1] == TEXT("String B1"));
		CHECK(TypedBuffer[2] == TEXT("String C1"));

		TypedBuffer[2].~FString();
		TypedBuffer[1].~FString();
		TypedBuffer[0].~FString();
	}

	SECTION("Relocate overlapping memory later in memory")
	{
		alignas(FString) char Buffer[sizeof(FString) * 5];

		FString* TypedBuffer = (FString*)Buffer;

		new ((void*)(TypedBuffer + 0)) FString("String A2");
		new ((void*)(TypedBuffer + 1)) FString("String B2");
		new ((void*)(TypedBuffer + 2)) FString("String C2");

		RelocateConstructItems<FString>((void*)(TypedBuffer + 2), TypedBuffer, 3);

		CHECK(TypedBuffer[2] == TEXT("String A2"));
		CHECK(TypedBuffer[3] == TEXT("String B2"));
		CHECK(TypedBuffer[4] == TEXT("String C2"));

		TypedBuffer[4].~FString();
		TypedBuffer[3].~FString();
		TypedBuffer[2].~FString();
	}

	SECTION("Relocate between different types (cannot overlap)")
	{
		struct FStringSource : TInstanceTracker<FStringSource>
		{
			UE_NONCOPYABLE(FStringSource)

			FStringSource(const TCHAR* InCh)
				: Str(InCh)
			{
			}

			~FStringSource() = default;

			FString Str;
			int32 DummyFieldToMakeItADifferentSizeFromFStringDest = 0;
		};

		struct FStringDest : TInstanceTracker<FStringDest>
		{
			UE_NONCOPYABLE(FStringDest)

			// This is FStringSource&& to ensure that the relocation uses moves instead of copies
			FStringDest(FStringSource&& InStr)
				: Str(MoveTemp(InStr.Str))
			{
			}

			~FStringDest() = default;

			FString Str;
		};

		FStringSource::ResetInstances();
		FStringDest::ResetInstances();

		alignas(FStringSource) char SrcBuffer[sizeof(FStringSource) * 3];
		alignas(FStringDest) char DestBuffer[sizeof(FStringDest) * 3];

		FStringSource* TypedSrcBuffer = (FStringSource*)SrcBuffer;
		FStringDest* TypedDestBuffer = (FStringDest*)DestBuffer;

		new ((void*)(TypedSrcBuffer + 0)) FStringSource(TEXT("String A3"));
		new ((void*)(TypedSrcBuffer + 1)) FStringSource(TEXT("String B3"));
		new ((void*)(TypedSrcBuffer + 2)) FStringSource(TEXT("String C3"));

		CHECK(FStringSource::CheckCurrentlyInstantiated({ TypedSrcBuffer + 0, TypedSrcBuffer + 1, TypedSrcBuffer + 2 }));
		CHECK(FStringDest::CheckCurrentlyInstantiated({}));

		RelocateConstructItems<FStringDest>((void*)TypedDestBuffer, TypedSrcBuffer, 3);

		CHECK(FStringSource::CheckCurrentlyInstantiated({}));
		CHECK(FStringDest::CheckCurrentlyInstantiated({ TypedDestBuffer + 0, TypedDestBuffer + 1, TypedDestBuffer + 2 }));

		CHECK(TypedDestBuffer[0].Str == TEXT("String A3"));
		CHECK(TypedDestBuffer[1].Str == TEXT("String B3"));
		CHECK(TypedDestBuffer[2].Str == TEXT("String C3"));

		TypedDestBuffer[2].~FStringDest();
		TypedDestBuffer[1].~FStringDest();
		TypedDestBuffer[0].~FStringDest();

		CHECK(FStringSource::CheckCurrentlyInstantiated({}));
		CHECK(FStringDest::CheckCurrentlyInstantiated({}));
	}
}

#endif
