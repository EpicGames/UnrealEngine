// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Templates/NoDestroy.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

struct FNoDestroyTestType
{
    int32 Value = 0;

    static inline int32 ConstructorCount = 0;
    static inline int32 DestructorCount = 0;

    FNoDestroyTestType(int32 InValue)
        : Value(InValue)
    {
        ++ConstructorCount;
    }

    explicit consteval FNoDestroyTestType(EConstEval, int32 InValue)
        : Value(InValue)
    {
    }

    ~FNoDestroyTestType()
    {
        ++DestructorCount;
    }
};

constinit TNoDestroy<FNoDestroyTestType> NoDestroyTestGlobal{NoDestroyConstEval, ConstEval, 17};

static_assert(sizeof(FNoDestroyTestType) == sizeof(TNoDestroy<FNoDestroyTestType>), "TNoDestroy should be same size as inner type");

TEST_CASE_NAMED(FNoDestroyTest, "System::Core::Templates::NoDestroy", "[Core][Templates][SmokeFilter]")
{
    CHECK_EQUALS("Global value is initialized", 17, NoDestroyTestGlobal->Value);
    CHECK_EQUALS("Address of global is equal to address from overridden address operator", 
        (void*)std::addressof(NoDestroyTestGlobal), (void*)&NoDestroyTestGlobal);

    {
        FNoDestroyTestType::ConstructorCount = 0;
        FNoDestroyTestType::DestructorCount = 0;
        TNoDestroy<FNoDestroyTestType> Temporary{21};

        CHECK_EQUALS("Inner object constructed in place", 1, FNoDestroyTestType::ConstructorCount);
        CHECK_EQUALS("Inner object value initialized", 21, Temporary->Value);
    }
    CHECK_EQUALS("Inner object not destroyed", 0, FNoDestroyTestType::DestructorCount);
}

#endif