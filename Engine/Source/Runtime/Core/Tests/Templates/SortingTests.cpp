// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/Sorting.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FTemplatesSortingTest, "System::Core::Templates::Sorting", "[Core][Templates][SmokeFilter]")
{
	struct FRadixSort64Test
	{
		uint64 Key;
		FString Value;
		
		bool operator==(const FRadixSort64Test&) const = default;
		bool operator!=(const FRadixSort64Test&) const = default;
	};
	struct SortKey
	{
		FORCEINLINE uint64 operator()(const FRadixSort64Test& Value) const
		{
			return Value.Key;
		}
	};

	SECTION("RadixSort64 low bits")
	{
		TArray<uint64> Array{ 3, 1, 2 };
		RadixSort64(Array.GetData(), Array.Num());

		TArray<uint64> Compare{ 1, 2, 3 };
		CHECK_MESSAGE(TEXT("List incorrectly sorted."), Array == Compare);
	}

	SECTION("RadixSort64 high bits")
	{
		TArray<uint64> Array
		{
			0xaabbccdd00112233,
				0xeeff998800112233,
				0x9988776600112233
		};
		RadixSort64(Array.GetData(), Array.Num());

		TArray<uint64> Compare
		{
			0x9988776600112233,
			0xaabbccdd00112233,
			0xeeff998800112233
		};
		CHECK_MESSAGE(TEXT("List incorrectly sorted."), Array == Compare);
	}

	SECTION("RadixSort64 all")
	{
		TArray<uint64> Array
		{
			0xaabbccdd00112233,
			0xeeff998800887766,
			0x9988776600443322
		};
		RadixSort64(Array.GetData(), Array.Num());

		TArray<uint64> Compare
		{
			0x9988776600443322,
			0xaabbccdd00112233,
			0xeeff998800887766
		};
		CHECK_MESSAGE(TEXT("List incorrectly sorted."), Array == Compare);
	}

	SECTION("RadixSort64 all with buffer")
	{
		TArray<uint64> Array
		{
			0xaabbccdd00112233,
			0xeeff998800887766,
			0x9988776600443322
		};
		TArray<uint64> Buffer;
		Buffer.AddDefaulted(Array.Num());
		RadixSort64<ERadixSortBufferState::IsInitialized>(Array.GetData(), Buffer.GetData(), Array.Num());

		TArray<uint64> Compare
		{
			0x9988776600443322,
			0xaabbccdd00112233,
			0xeeff998800887766
		};
		CHECK_MESSAGE(TEXT("List incorrectly sorted."), Array == Compare);
	}

	SECTION("RadixSort64 custom sort key")
	{
		TArray<FRadixSort64Test> Array
		{
			FRadixSort64Test{ .Key = 2, .Value = TEXT("B") },
			FRadixSort64Test{ .Key = 1, .Value = TEXT("A") },
			FRadixSort64Test{ .Key = 3, .Value = TEXT("C") },
		};
		RadixSort64(Array.GetData(), Array.Num(), SortKey());

		TArray<FRadixSort64Test> Compare
		{
			FRadixSort64Test{.Key = 1, .Value = TEXT("A") },
			FRadixSort64Test{.Key = 2, .Value = TEXT("B") },
			FRadixSort64Test{.Key = 3, .Value = TEXT("C") },
		};
		CHECK_MESSAGE(TEXT("List incorrectly sorted."), Array == Compare);
	}

	SECTION("RadixSort64 custom sort key and provided buffer.")
	{
		TArray<FRadixSort64Test> Array
		{
			FRadixSort64Test{.Key = 2, .Value = TEXT("B") },
			FRadixSort64Test{.Key = 1, .Value = TEXT("A") },
			FRadixSort64Test{.Key = 3, .Value = TEXT("C") },
		};
		TArray<FRadixSort64Test> Buffer;
		Buffer.AddDefaulted(Array.Num());
		RadixSort64<ERadixSortBufferState::IsInitialized>(Array.GetData(), Buffer.GetData(), Array.Num(), SortKey());

		TArray<FRadixSort64Test> Compare
		{
			FRadixSort64Test{.Key = 1, .Value = TEXT("A") },
			FRadixSort64Test{.Key = 2, .Value = TEXT("B") },
			FRadixSort64Test{.Key = 3, .Value = TEXT("C") },
		};
		CHECK_MESSAGE(TEXT("List incorrectly sorted."), Array == Compare);
	}
}

#endif //WITH_TESTS
