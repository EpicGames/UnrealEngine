// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Templates/ValueOrError.h"

#include "Tests/TestHarnessAdapter.h"

#include <type_traits>

TEST_CASE_NAMED(FTemplatesValueOrErrorTest, "System::Core::Templates::ValueOrError", "[Core][Templates][SmokeFilter]")
{
	SECTION("Static")
	{
		STATIC_REQUIRE(!std::is_constructible<TValueOrError<int, int>>::value);

		STATIC_REQUIRE(std::is_copy_constructible<TValueOrError<int, int>>::value);
		STATIC_REQUIRE(std::is_move_constructible<TValueOrError<int, int>>::value);

		STATIC_REQUIRE(std::is_copy_assignable<TValueOrError<int, int>>::value);
		STATIC_REQUIRE(std::is_move_assignable<TValueOrError<int, int>>::value);
	}

	static int ValueCount = 0;
	static int ErrorCount = 0;

	struct FTestValue
	{
		FTestValue() { Value = ++ValueCount; }
		FTestValue(const FTestValue&) { Value = ++ValueCount; }
		FTestValue(FTestValue&& Other) { Value = Other.Value; ++ValueCount; }
		FTestValue(int InValue1, int InValue2, int InValue3) { Value = InValue1 + InValue2 + InValue3; ++ValueCount; }
		~FTestValue() { --ValueCount; }
		FTestValue& operator=(const FTestValue& Other) = delete;
		FTestValue& operator=(FTestValue&& Other) = delete;
		int Value;
	};

	struct FTestError
	{
		FTestError() { Error = ++ErrorCount; }
		FTestError(const FTestError&) { Error = ++ErrorCount; }
		FTestError(FTestError&& Other) { Error = Other.Error; ++ErrorCount; }
		FTestError(int InError1, int InError2) { Error = InError1 + InError2; ++ErrorCount; }
		~FTestError() { --ErrorCount; }
		FTestError& operator=(const FTestError& Other) = delete;
		FTestError& operator=(FTestError&& Other) = delete;
		int Error;
	};

	struct FTestMoveOnly
	{
		FTestMoveOnly() = default;
		FTestMoveOnly(int32 InValue) : Value(InValue) {}
		FTestMoveOnly(const FTestMoveOnly&) = delete;
		FTestMoveOnly(FTestMoveOnly&&) = default;
		int Value = 0;
	};

	using FTestType = TValueOrError<FTestValue, FTestError>;

	SECTION("MakeValue Move")
	{
		FTestType ValueOrError = MakeValue(FTestValue());
		CHECK(ValueCount == 1);
		CHECK(ValueOrError.TryGetValue() == &ValueOrError.GetValue());
		CHECK(ValueOrError.GetValue().Value == 1);
		CHECK(ValueOrError.HasValue());
		CHECK(ValueOrError.TryGetError() == nullptr);
		CHECK_FALSE(ValueOrError.HasError());
	}

	SECTION("MakeValue Proxy")
	{
		FTestType ValueOrError = MakeValue(2, 6, 8);
		CHECK(ValueCount == 1);
		CHECK(ValueOrError.TryGetValue() == &ValueOrError.GetValue());
		CHECK(ValueOrError.GetValue().Value == 16);
		CHECK(ValueOrError.HasValue());
		CHECK(ValueOrError.TryGetError() == nullptr);
		CHECK_FALSE(ValueOrError.HasError());
	}

	SECTION("StealValue")
	{
		FTestType ValueOrError = MakeValue(FTestValue());
		FTestValue Value = ValueOrError.StealValue();
		CHECK(ValueCount == 1);
		CHECK(Value.Value == 1);
		CHECK_FALSE(ValueOrError.HasError());
		CHECK_FALSE(ValueOrError.HasValue());
	}

	SECTION("MakeError Move")
	{
		FTestType ValueOrError = MakeError(FTestError());
		CHECK(ErrorCount == 1);
		CHECK(ValueOrError.TryGetError() == &ValueOrError.GetError());
		CHECK(ValueOrError.GetError().Error == 1);
		CHECK(ValueOrError.HasError());
		CHECK(ValueOrError.TryGetValue() == nullptr);
		CHECK_FALSE(ValueOrError.HasValue());
	}

	SECTION("MakeError Proxy")
	{
		FTestType ValueOrError = MakeError(4, 12);
		CHECK(ErrorCount == 1);
		CHECK(ValueOrError.TryGetError() == &ValueOrError.GetError());
		CHECK(ValueOrError.GetError().Error == 16);
		CHECK(ValueOrError.HasError());
		CHECK(ValueOrError.TryGetValue() == nullptr);
		CHECK_FALSE(ValueOrError.HasValue());
	}

	SECTION("StealError")
	{
		FTestType ValueOrError = MakeError();
		FTestError Error = ValueOrError.StealError();
		CHECK(ErrorCount == 1);
		CHECK(Error.Error == 1);
		CHECK_FALSE(ValueOrError.HasValue());
		CHECK_FALSE(ValueOrError.HasError());
	}

	SECTION("Assignment")
	{
		FTestType ValueOrError = MakeValue();
		ValueOrError = MakeValue();
		CHECK(ValueCount == 1);
		CHECK(ValueOrError.GetValue().Value == 2);
		CHECK(ErrorCount == 0);
		ValueOrError = MakeError();
		CHECK(ValueCount == 0);
		CHECK(ErrorCount == 1);
		ValueOrError = MakeError();
		CHECK(ValueCount == 0);
		CHECK(ValueOrError.GetError().Error == 2);
		CHECK(ErrorCount == 1);
		ValueOrError = MakeValue();
		CHECK(ValueCount == 1);
		CHECK(ErrorCount == 0);
		FTestType UnsetValueOrError = MakeValue();
		(void)UnsetValueOrError.StealValue();
		ValueOrError = MoveTemp(UnsetValueOrError);
		CHECK(ValueCount == 0);
		CHECK(ErrorCount == 0);
		CHECK_FALSE(ValueOrError.HasValue());
		CHECK_FALSE(ValueOrError.HasError());
	}

	SECTION("Move-Only Value/Error")
	{
		TValueOrError<FTestMoveOnly, FTestMoveOnly> Value = MakeValue(FTestMoveOnly(1));
		TValueOrError<FTestMoveOnly, FTestMoveOnly> Error = MakeError(FTestMoveOnly(1));
		FTestMoveOnly MovedValue = MoveTemp(Value).GetValue();
		FTestMoveOnly MovedError = MoveTemp(Error).GetError();
		CHECK(MovedValue.Value == 1);
		CHECK(MovedError.Value == 1);
	}

	SECTION("Integer Value/Error")
	{
		TValueOrError<int32, int32> ValueOrError = MakeValue();
		CHECK(ValueOrError.GetValue() == 0);
		ValueOrError = MakeValue(1);
		CHECK(ValueOrError.GetValue() == 1);
		ValueOrError = MakeError();
		CHECK(ValueOrError.GetError() == 0);
		ValueOrError = MakeError(1);
		CHECK(ValueOrError.GetError() == 1);
	}

	SECTION("Value/Void Value")
	{
		TValueOrError<FTestValue, void> ValueOrError = MakeValue(1, 2, 3);
		CHECK(ValueCount == 1);
		CHECK(ValueOrError.HasValue());
		CHECK_FALSE(ValueOrError.HasError());
		CHECK(ValueOrError.GetValue().Value == 6);
		CHECK(ValueOrError.TryGetValue() == &ValueOrError.GetValue());
		FTestValue Value = ValueOrError.StealValue();
		CHECK_FALSE(ValueOrError.HasValue());
		CHECK(Value.Value == 6);
	}

	SECTION("Value/Void Error")
	{
		TValueOrError<FTestValue, void> ValueOrError = MakeError();
		CHECK(ValueCount == 0);
		CHECK(ValueOrError.HasError());
		CHECK_FALSE(ValueOrError.HasValue());
		CHECK(ValueOrError.TryGetValue() == nullptr);
	}

	SECTION("Void/Error Value")
	{
		TValueOrError<void, FTestError> ValueOrError = MakeValue();
		CHECK(ErrorCount == 0);
		CHECK(ValueOrError.HasValue());
		CHECK_FALSE(ValueOrError.HasError());
		CHECK(ValueOrError.TryGetError() == nullptr);
	}

	SECTION("Void/Error Error")
	{
		TValueOrError<void, FTestError> ValueOrError = MakeError(1, 2);
		CHECK(ErrorCount == 1);
		CHECK(ValueOrError.HasError());
		CHECK_FALSE(ValueOrError.HasValue());
		CHECK(ValueOrError.GetError().Error == 3);
		CHECK(ValueOrError.TryGetError() == &ValueOrError.GetError());
		FTestError Error = ValueOrError.StealError();
		CHECK_FALSE(ValueOrError.HasError());
		CHECK(Error.Error == 3);
	}

	SECTION("Void/Void Value")
	{
		TValueOrError<void, void> ValueOrError = MakeValue();
		CHECK(ValueOrError.HasValue());
		CHECK_FALSE(ValueOrError.HasError());
	}

	SECTION("Void/Void Error")
	{
		TValueOrError<void, void> ValueOrError = MakeError();
		CHECK(ValueOrError.HasError());
		CHECK_FALSE(ValueOrError.HasValue());
	}

	CHECK(ValueCount == 0);
	CHECK(ErrorCount == 0);
}

#endif //WITH_TESTS