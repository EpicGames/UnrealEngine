// Copyright Epic Games, Inc. All Rights Reserved.

#if !EXPLICIT_TESTS_TARGET && WITH_LOW_LEVEL_TESTS

// Will be removed, test group events for implicitly built tests must be placed in the same folder as a *.Target.cs file

#endif // !EXPLICIT_TESTS_TARGET && WITH_LOW_LEVEL_TESTS
