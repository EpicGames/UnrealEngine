// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Utf8/Utf8Iterator.h"

#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "Tests/TestHarnessAdapter.h"

// ---------------------------------------------------------------------------
// Helpers
// ---------------------------------------------------------------------------

namespace
{
	/** Collect all codepoints produced by iterating over a raw byte span. */
	TArray<UTF32CHAR> Decode(FUtf8StringView View)
	{
		TArray<UTF32CHAR> Result;
		for (FUtf8Iterator It(View); It; ++It)
		{
			Result.Add(*It);
		}
		return Result;
	}

	/** Collect all codepoints from a UTF-8 string literal. */
	UE_REWRITE TArray<UTF32CHAR> Decode(const UTF8CHAR* Str, int32 Len)
	{
		return Decode(FUtf8StringView(Str, Len));
	}

	/** Collect all codepoints from a UTF-8 string literal. */
	UE_REWRITE TArray<UTF32CHAR> Decode(const ANSICHAR* Str)
	{
		return Decode(FUtf8StringView(Str));
	}
}

// ---------------------------------------------------------------------------
// Tests
// ---------------------------------------------------------------------------

TEST_CASE_NAMED(FUtf8IteratorTest, "System::Core::Utf8::Utf8Iterator", "[Core][Utf8][SmokeFilter]")
{
	// -----------------------------------------------------------------------
	SECTION("EmptyView")
	{
		FUtf8Iterator It(FUtf8StringView{});
		CHECK_FALSE((bool)It);
	}

	// -----------------------------------------------------------------------
	SECTION("EmptyViewFromLiteral")
	{
		FUtf8Iterator It(UTF8TEXTVIEW(""));
		CHECK_FALSE((bool)It);
	}

	// -----------------------------------------------------------------------
	SECTION("ASCIIString")
	{
		// Every byte in 0x00–0x7F is a single-codepoint sequence
		TArray<UTF32CHAR> Got = Decode("Hello");
		TArray<UTF32CHAR> Exp = { 'H', 'e', 'l', 'l', 'o' };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("NullByteIsNotSpecial")
	{
		// Zero bytes must not terminate iteration; the view length governs the range
		const UTF8CHAR Bytes[] = { 'A', 0x00, 'B' };
		TArray<UTF32CHAR> Got = Decode(Bytes, 3);
		TArray<UTF32CHAR> Exp = { (UTF32CHAR)'A', (UTF32CHAR)0x00u, (UTF32CHAR)'B' };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("TwoByteSequence")
	{
		// U+00C9 'É'  →  0xC3 0x89
		const UTF8CHAR Bytes[] = { 0xC3, 0x89 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 2);
		TArray<UTF32CHAR> Exp = { (UTF32CHAR)0x00C9u };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("TwoByteSequenceMinimum")
	{
		// U+0080 — the smallest codepoint requiring a two-byte encoding
		// Encoded as: 0xC2 0x80
		const UTF8CHAR Bytes[] = { 0xC2, 0x80 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 2);
		TArray<UTF32CHAR> Exp = { (UTF32CHAR)0x0080u };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("ThreeByteSequence")
	{
		// U+4E2D '中'  →  0xE4 0xB8 0xAD
		const UTF8CHAR Bytes[] = { 0xE4, 0xB8, 0xAD };
		TArray<UTF32CHAR> Got = Decode(Bytes, 3);
		TArray<UTF32CHAR> Exp = { (UTF32CHAR)0x4E2Du };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("FourByteSequence")
	{
		// U+1F600 '😀'  →  0xF0 0x9F 0x98 0x80
		const UTF8CHAR Bytes[] = { 0xF0, 0x9F, 0x98, 0x80 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 4);
		TArray<UTF32CHAR> Exp = { (UTF32CHAR)0x1F600u };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("FourByteSequenceMaximum")
	{
		// U+10FFFF — the maximum valid Unicode codepoint
		// Encoded as: 0xF4 0x8F 0xBF 0xBF
		const UTF8CHAR Bytes[] = { 0xF4, 0x8F, 0xBF, 0xBF };
		TArray<UTF32CHAR> Got = Decode(Bytes, 4);
		TArray<UTF32CHAR> Exp = { (UTF32CHAR)0x10FFFFu };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("MixedMultibyteString")
	{
		// "A中B" → 0x41, 0xE4 0xB8 0xAD, 0x42
		const UTF8CHAR Bytes[] = { 0x41, 0xE4, 0xB8, 0xAD, 0x42 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 5);
		TArray<UTF32CHAR> Exp = { (UTF32CHAR)'A', (UTF32CHAR)0x4E2Du, (UTF32CHAR)'B' };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("InvalidContinuationByteInLeadPosition")
	{
		// A bare continuation byte (0x80–0xBF) in lead position is invalid.
		// Each such byte should produce one replacement character.
		const UTF8CHAR Bytes[] = { 0x80, 0xBF };
		TArray<UTF32CHAR> Got = Decode(Bytes, 2);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("OverlongTwoByteSequence")
	{
		// 0xC0 0x80 — 0xC0 can never start a valid sequence, so it is consumed
		// alone.  0x80 is then a bare continuation byte and produces its own
		// replacement character.
		const UTF8CHAR Bytes[] = { 0xC0, 0x80 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 2);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("OverlongThreeByteSequence")
	{
		// 0xE0 requires a second byte of 0xA0–0xBF; 0x80 is out of that range, so
		// 0xE0 is consumed alone.  Each of the two 0x80 continuation bytes then
		// produces its own replacement character.
		const UTF8CHAR Bytes[] = { 0xE0, 0x80, 0x80 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 3);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("OverlongFourByteSequence")
	{
		// 0xF0 requires a second byte of 0x90–0xBF; 0x80 is out of that range, so
		// 0xF0 is consumed alone.  Each of the three 0x80 continuation bytes then
		// produces its own replacement character.
		const UTF8CHAR Bytes[] = { 0xF0, 0x80, 0x80, 0x80 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 4);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("SurrogateCodepoint")
	{
		// 0xED requires a second byte of 0x80–0x9F; 0xA0 is out of that range, so
		// 0xED is consumed alone.  Each of the remaining continuation bytes then
		// produces its own replacement character.
		// U+D800 (high surrogate): 0xED 0xA0 0x80
		const UTF8CHAR HighSurrogate[] = { 0xED, 0xA0, 0x80 };
		TArray<UTF32CHAR> Got = Decode(HighSurrogate, 3);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);

		// U+DFFF (low surrogate): 0xED 0xBF 0xBF
		const UTF8CHAR LowSurrogate[] = { 0xED, 0xBF, 0xBF };
		Got = Decode(LowSurrogate, 3);
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("OutOfRangeFourByteSequence")
	{
		// 0xF4 requires a second byte of 0x80–0x8F; 0x90 is out of that range, so
		// 0xF4 is consumed alone.  Each of the three remaining continuation bytes
		// then produces its own replacement character.
		// (Would encode U+110000 — one above the maximum valid codepoint.)
		const UTF8CHAR Bytes[] = { 0xF4, 0x90, 0x80, 0x80 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 4);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("IllegalFiveByteSequence")
	{
		// 0xF8 0x80 0x80 0x80 0x80 — five-byte lead, illegal per RFC 3629.
		// The lead byte is consumed alone; each of the four continuation bytes then
		// produces its own replacement character.
		const UTF8CHAR Bytes[] = { 0xF8, 0x80, 0x80, 0x80, 0x80 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 5);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("IllegalSixByteSequence")
	{
		// 0xFC 0x80 0x80 0x80 0x80 0x80 — six-byte lead, illegal per RFC 3629.
		// The lead byte is consumed alone; each of the five continuation bytes then
		// produces its own replacement character.
		const UTF8CHAR Bytes[] = { 0xFC, 0x80, 0x80, 0x80, 0x80, 0x80 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 6);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("TruncatedTwoByteAtEnd")
	{
		// 0xC3 with no continuation — truncated, produces one replacement and ends.
		const UTF8CHAR Bytes[] = { 0xC3 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 1);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("TruncatedThreeByteOneContinuationAtEnd")
	{
		// 0xE4 0xB8 with no third byte — truncated after one continuation.
		const UTF8CHAR Bytes[] = { 0xE4, 0xB8 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 2);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("TruncatedThreeByteNoContinuationAtEnd")
	{
		// 0xE4 alone — truncated with no continuations at all.
		const UTF8CHAR Bytes[] = { 0xE4 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 1);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("TruncatedFourByteAtEnd")
	{
		// 0xF0 0x9F 0x98 with the final continuation missing.
		const UTF8CHAR Bytes[] = { 0xF0, 0x9F, 0x98 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 3);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("BrokenSequenceNonContinuationNotConsumed")
	{
		// 0xC2 followed by 'A' (0x41) — 'A' is not a continuation byte.
		// The iterator must NOT consume 'A' as part of the invalid sequence;
		// it should be decoded normally on the next step.
		const UTF8CHAR Bytes[] = { 0xC2, 0x41 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 2);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, (UTF32CHAR)'A' };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("BrokenThreeByteFirstContinuationInvalid")
	{
		// 0xE0 0x41 0x80 — second byte is not a continuation.
		// Replacement for 0xE0, then 'A' decoded normally, then 0x80 (bare
		// continuation) yields another replacement.
		const UTF8CHAR Bytes[] = { 0xE0, 0x41, 0x80 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 3);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, (UTF32CHAR)'A', FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("BrokenThreeByteSecondContinuationInvalid")
	{
		// 0xE4 0xB8 0x41 — third byte is not a continuation.
		// Replacement for 0xE4 0xB8, then 'A' decoded normally.
		const UTF8CHAR Bytes[] = { 0xE4, 0xB8, 0x41 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 3);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, (UTF32CHAR)'A' };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("FiveByteLeadFollowedByASCII")
	{
		// 0xF8 0x41 — 5-byte lead followed by 'A' (not a continuation).
		// Only the lead byte should be consumed as the ill-formed sequence;
		// 'A' must survive to the next iteration.
		const UTF8CHAR Bytes[] = { 0xF8, 0x41 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 2);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, (UTF32CHAR)'A' };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("MixedValidAndInvalid")
	{
		// "A" + invalid 0x80 + U+4E2D '中' + 0xFE (always-invalid byte) + "B"
		const UTF8CHAR Bytes[] = { 0x41, 0x80, 0xE4, 0xB8, 0xAD, 0xFE, 0x42 };
		TArray<UTF32CHAR> Got = Decode(Bytes, 7);
		TArray<UTF32CHAR> Exp = { (UTF32CHAR)'A', FUtf8Iterator::ReplacementCharacter, (UTF32CHAR)0x4E2Du, FUtf8Iterator::ReplacementCharacter, (UTF32CHAR)'B' };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("AllInvalidSequences")
	{
		// Four bare continuation bytes in a row: each should yield one replacement.
		const UTF8CHAR Bytes[] = { 0x80, 0x90, 0xA0, 0xBF };
		TArray<UTF32CHAR> Got = Decode(Bytes, 4);
		TArray<UTF32CHAR> Exp = { FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter, FUtf8Iterator::ReplacementCharacter };
		CHECK(Got == Exp);
	}

	// -----------------------------------------------------------------------
	SECTION("ReplacementCharacterConstant")
	{
		CHECK(FUtf8Iterator::ReplacementCharacter == (UTF32CHAR)0xFFFDu);
	}
}

#endif // WITH_TESTS
